[GameFeaturePlugins]
bGFPAreAlwaysResident=True
