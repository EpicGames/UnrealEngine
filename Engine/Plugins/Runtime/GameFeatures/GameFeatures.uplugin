{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Game Features",
	"Description": "Support for modular Game Feature Plugins",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "GameFeatures",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "GameFeaturesEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ModularGameplay",
			"Enabled": true
		},
		{
			"Name": "DataRegistry",
			"Enabled": true
		},
		{
			"Name": "AssetReferenceRestrictions",
			"Enabled": true
		},
		{
			"Name": "PluginUtils",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "DataValidation",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}
