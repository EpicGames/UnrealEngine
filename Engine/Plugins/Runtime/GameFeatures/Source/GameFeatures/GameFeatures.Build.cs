// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameFeatures : ModuleRules
	{
		public GameFeatures(ReadOnlyTargetRules Target) : base(Target)
		{
			bTreatAsEngineModule = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"ModularGameplay",
					"DataRegistry"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"GameplayTags",
					"InstallBundleManager",
					"IoStoreOnDemandCore",
					"IrisCore",
					"Json",
					"JsonUtilities",
					"PakFile",
					"Projects",
					"RenderCore", // required for FDeferredCleanupInterface
					"TraceLog",
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"IoStoreOnDemandCore",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"PlacementMode",
						"PluginUtils",
					}
				);
			}
		}
	}
}
