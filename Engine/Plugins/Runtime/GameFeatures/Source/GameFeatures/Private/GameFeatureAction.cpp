// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction.h"
#include "GameFeatureData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction)

UGameFeatureData* UGameFeatureAction::GetGameFeatureData() const
{
	for (UObject* Obj = GetOuter(); Obj; Obj = Obj->GetOuter())
	{
		if (UGameFeatureData* GFD = Cast<UGameFeatureData>(Obj))
		{
			return GFD;
		}
	}

	return nullptr;
}

void UGameFeatureAction::OnGameFeatureActivating(FGameFeatureActivatingContext& Context)
{
	// Call older style if not overridden
	OnGameFeatureActivating();
}

bool UGameFeatureAction::IsGameFeaturePluginRegistered(bool bCheckForRegistering  /*= false*/) const
{
	UGameFeatureData* GameFeatureData = GetGameFeatureData();
	return !!GameFeatureData ? GameFeatureData->IsGameFeaturePluginRegistered(bCheckForRegistering) : false;
}

bool UGameFeatureAction::IsGameFeaturePluginActive(bool bCheckForActivating /*= false*/) const
{
	UGameFeatureData* GameFeatureData = GetGameFeatureData();
	return !!GameFeatureData ? GameFeatureData->IsGameFeaturePluginActive(bCheckForActivating) : false;
}
