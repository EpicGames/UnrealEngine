// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AddActorFactory.h"
#include "GameFeatureData.h"
#include "Misc/MessageDialog.h"

#if WITH_EDITOR
#include "IPlacementModeModule.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AddActorFactory)

#define LOCTEXT_NAMESPACE "GameFeatures"

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddActorFactory

DEFINE_LOG_CATEGORY_STATIC(LogAddActorFactory, Log, All);

void UGameFeatureAction_AddActorFactory::OnGameFeatureRegistering()
{
	AddActorFactory();
}

void UGameFeatureAction_AddActorFactory::OnGameFeatureUnregistering()
{
	RemoveActorFactory();
}

#if WITH_EDITOR
void UGameFeatureAction_AddActorFactory::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	// If OldOuter is not GetTransientPackage(), but GetOuter() is GetTransientPackage(), then you were trashed.
	const UObject* MyOuter = GetOuter();
	const UPackage* TransientPackage = GetTransientPackage();
	if (OldOuter != TransientPackage && MyOuter == TransientPackage)
	{
		RemoveActorFactory();
	}
}

void UGameFeatureAction_AddActorFactory::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGameFeatureAction_AddActorFactory, ActorFactory))
	{
		RemoveActorFactory();
		AddActorFactory();
	}
}
#endif // WITH_EDITOR

void UGameFeatureAction_AddActorFactory::AddActorFactory()
{
#if WITH_EDITOR
	if (ActorFactory.IsNull())
	{
		UE_LOGF(LogAddActorFactory, Warning, "ActorFactory is null. Unable to add factory");
		return;
	}
	if (UClass* FactoryClass = ActorFactory.LoadSynchronous())
	{
		if (!FactoryClass->IsChildOf(UActorFactory::StaticClass()))
		{
			UE_LOGF(LogAddActorFactory, Error, "ActorFactory (%ls) was not an ActorFactory class", *FactoryClass->GetName());
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AddActorFactory_BadSubclass", "Selected class was not an ActorFactory class."));
			ActorFactory.Reset();
			return;
		}
		UE_LOGF(LogAddActorFactory, Verbose, "Adding actor factory %ls", *FactoryClass->GetName());

		UActorFactory* NewFactory = NewObject<UActorFactory>(GetTransientPackage(), FactoryClass);
		if (NewFactory->bShouldAutoRegister)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AddActorFactory_AutoRegister", "The selected actor factory is set to auto register. Set the config variable bShouldAutoRegister to false before using this action."));
			ActorFactory.Reset();
			return;
		}
		GEditor->ActorFactories.Add(NewFactory);
		AddedFactory = NewFactory;

		if (IPlacementModeModule::IsAvailable())
		{
			IPlacementModeModule::Get().RegenerateItemsForCategory(FBuiltInPlacementCategories::AllClasses());
		}
	}
#endif // WITH_EDITOR
}

void UGameFeatureAction_AddActorFactory::RemoveActorFactory()
{
#if WITH_EDITOR
	if (UActorFactory* FactoryToRemove = Cast<UActorFactory>(AddedFactory.Get()))
	{
		UE_LOGF(LogAddActorFactory, Verbose, "Removing actor factory %ls", *FactoryToRemove->GetName());

		GEditor->ActorFactories.Remove(FactoryToRemove);
		AddedFactory.Reset();

		if (IPlacementModeModule::IsAvailable())
		{
			IPlacementModeModule::Get().RegenerateItemsForCategory(FBuiltInPlacementCategories::AllClasses());
		}
	}
#endif // WITH_EDITOR
}

//////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
