// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AddCheats.h"
#include "GameFramework/CheatManager.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AddCheats)

#define LOCTEXT_NAMESPACE "GameFeatures"

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddCheats

void UGameFeatureAction_AddCheats::OnGameFeatureActivating()
{
#if UE_WITH_CHEAT_MANAGER
	bIsActive = true;
	CheatManagerRegistrationHandle = UCheatManager::RegisterForOnCheatManagerCreated(FOnCheatManagerCreated::FDelegate::CreateUObject(this, &ThisClass::OnCheatManagerCreated));
#endif // UE_WITH_CHEAT_MANAGER
}

void UGameFeatureAction_AddCheats::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
#if UE_WITH_CHEAT_MANAGER
	UCheatManager::UnregisterFromOnCheatManagerCreated(CheatManagerRegistrationHandle);

	for (TWeakObjectPtr<UCheatManagerExtension> ExtensionPtr : SpawnedCheatManagers)
	{
		if (UCheatManagerExtension* Extension = ExtensionPtr.Get())
		{
			UCheatManager* CheatManager = CastChecked<UCheatManager>(Extension->GetOuter());
			CheatManager->RemoveCheatManagerExtension(Extension);
		}
	}
	SpawnedCheatManagers.Empty();
	bIsActive = false;
#endif // UE_WITH_CHEAT_MANAGER
}

#if WITH_EDITOR
EDataValidationResult UGameFeatureAction_AddCheats::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	int32 EntryIndex = 0;
	for (const TSoftClassPtr<UCheatManagerExtension>& CheatManagerClassPtr : CheatManagers)
	{
		if (CheatManagerClassPtr.IsNull())
		{
			Result = EDataValidationResult::Invalid;
			Context.AddError(FText::Format(LOCTEXT("CheatEntryIsNull", "Null entry at index {0} in CheatManagers"), FText::AsNumber(EntryIndex)));
		}
		++EntryIndex;
	}

	return Result;
}
#endif

void UGameFeatureAction_AddCheats::OnCheatManagerCreated(UCheatManager* CheatManager)
{
#if UE_WITH_CHEAT_MANAGER
	// First clean out any stale pointers
	for (int32 ManagerIdx = SpawnedCheatManagers.Num() - 1; ManagerIdx >= 0; --ManagerIdx)
	{
		if (!SpawnedCheatManagers[ManagerIdx].IsValid())
		{
			SpawnedCheatManagers.RemoveAtSwap(ManagerIdx);
		}
	}

	for (const TSoftClassPtr<UCheatManagerExtension>& CheatManagerClassPtr : CheatManagers)
	{
		if (!CheatManagerClassPtr.IsNull())
		{
			TSubclassOf<UCheatManagerExtension> CheatManagerClass = CheatManagerClassPtr.Get();
			if (CheatManagerClass != nullptr)
			{
				// The class is in memory. Spawn now.
				SpawnCheatManagerExtension(CheatManager, CheatManagerClass);
			}
			else if (bLoadCheatManagersAsync)
			{
				// The class is not in memory and we want to load async. Start async load now.
				TWeakObjectPtr<UGameFeatureAction_AddCheats> WeakThis(this);
				TWeakObjectPtr<UCheatManager> WeakCheatManager(CheatManager);
				LoadPackageAsync(CheatManagerClassPtr.GetLongPackageName(), FLoadPackageAsyncDelegate::CreateLambda(
					[WeakThis, WeakCheatManager, CheatManagerClassPtr](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
				{
					if (Result == EAsyncLoadingResult::Succeeded)
					{
						UGameFeatureAction_AddCheats* StrongThis = WeakThis.Get();
						UCheatManager* StrongCheatManager = WeakCheatManager.Get();
						if (StrongThis && StrongThis->bIsActive && StrongCheatManager)
						{
							if (TSubclassOf<UCheatManagerExtension> LoadedCheatManagerClass = CheatManagerClassPtr.Get())
							{
								StrongThis->SpawnCheatManagerExtension(StrongCheatManager, LoadedCheatManagerClass);
							}
						}
					}
				}
				));
			}
			else
			{
				// The class is not in memory and we want to sync load. Load and spawn immediately.
				CheatManagerClass = CheatManagerClassPtr.LoadSynchronous();
				if (CheatManagerClass != nullptr)
				{
					SpawnCheatManagerExtension(CheatManager, CheatManagerClass);
				}
			}
		}
	}
#endif // UE_WITH_CHEAT_MANAGER
};

void UGameFeatureAction_AddCheats::SpawnCheatManagerExtension(UCheatManager* CheatManager, const TSubclassOf<UCheatManagerExtension>& CheatManagerClass)
{
#if UE_WITH_CHEAT_MANAGER
	if ((CheatManagerClass->ClassWithin == nullptr) || CheatManager->IsA(CheatManagerClass->ClassWithin))
	{
		UCheatManagerExtension* Extension = NewObject<UCheatManagerExtension>(CheatManager, CheatManagerClass);
		SpawnedCheatManagers.Add(Extension);
		CheatManager->AddCheatManagerExtension(Extension);
	}
#endif // UE_WITH_CHEAT_MANAGER
}

//////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

