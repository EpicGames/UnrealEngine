// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AddChunkOverride.h"
#include "Engine/AssetManager.h"
#include "Engine/AssetManagerSettings.h"
#include "GameFeatureData.h"
#include "Misc/MessageDialog.h"
#include "Misc/PathViews.h"

#if WITH_EDITOR
#include "Misc/ChunkDependencyInfo.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AddChunkOverride)

#define LOCTEXT_NAMESPACE "GameFeatures"

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddChunkOverride

DEFINE_LOG_CATEGORY_STATIC(LogAddChunkOverride, Log, All);

namespace GameFeatureAction_AddChunkOverride
{
	static TMap<int32, TArray<FString>> ChunkIdToPluginMap;
	static TMap<FString, int32> PluginToChunkId;
}

UGameFeatureAction_AddChunkOverride::FShouldAddChunkOverride UGameFeatureAction_AddChunkOverride::ShouldAddChunkOverride;

void UGameFeatureAction_AddChunkOverride::OnGameFeatureRegistering()
{
	const bool bShouldAddChunkOverride = ShouldAddChunkOverride.IsBound() ? ShouldAddChunkOverride.Execute(GetTypedOuter<UGameFeatureData>()) : true;
	if (bShouldAddChunkOverride)
	{
		TWeakObjectPtr<UGameFeatureAction_AddChunkOverride> WeakThis(this);
		UAssetManager::CallOrRegister_OnCompletedInitialScan(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UGameFeatureAction_AddChunkOverride::AddChunkIdOverride));
	}
}

void UGameFeatureAction_AddChunkOverride::OnGameFeatureUnregistering()
{
	RemoveChunkIdOverride();
}

#if WITH_EDITOR
TOptional<int32> UGameFeatureAction_AddChunkOverride::GetChunkForPackage(const FString& PackageName)
{
	if (GameFeatureAction_AddChunkOverride::PluginToChunkId.Num() == 0)
	{
		return TOptional<int32>();
	}

	static const FString EngineDir(TEXT("/Engine/"));
	static const FString GameDir(TEXT("/Game/"));
	if (PackageName.StartsWith(EngineDir, ESearchCase::CaseSensitive))
	{
		return TOptional<int32>();
	}
	else if (PackageName.StartsWith(GameDir, ESearchCase::CaseSensitive))
	{
		return TOptional<int32>();
	}
	else
	{
		FString MountPointName = FString(FPathViews::GetMountPointNameFromPath(PackageName));
		if (GameFeatureAction_AddChunkOverride::PluginToChunkId.Contains(MountPointName))
		{
			const int32 ExpectedChunkId = GameFeatureAction_AddChunkOverride::PluginToChunkId[MountPointName];
			return TOptional<int32>(ExpectedChunkId);
		}
	}
	return TOptional<int32>();
}

TArray<FString> UGameFeatureAction_AddChunkOverride::GetPluginNameFromChunkID(int32 ChunkID)
{
	return GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap.FindRef(ChunkID);
}

void UGameFeatureAction_AddChunkOverride::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	// If OldOuter is not GetTransientPackage(), but GetOuter() is GetTransientPackage(), then you were trashed.
	const UObject* MyOuter = GetOuter();
	const UPackage* TransientPackage = GetTransientPackage();
	if (OldOuter != TransientPackage && MyOuter == TransientPackage)
	{
		RemoveChunkIdOverride();
	}
}

void UGameFeatureAction_AddChunkOverride::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGameFeatureAction_AddChunkOverride, bShouldOverrideChunk))
	{
		RemoveChunkIdOverride();
		// Generate a new value if we have an invalid chunkId
		if (bShouldOverrideChunk && ChunkId < 0)
		{
			UE_LOGF(LogAddChunkOverride, Log, "Detected invalid ChunkId autogenerating new ID based on PluginName");
			ChunkId = GenerateUniqueChunkId();
		}
		if (ChunkId >= 0)
		{
			AddChunkIdOverride();
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UGameFeatureAction_AddChunkOverride, ChunkId))
	{
		RemoveChunkIdOverride();
		AddChunkIdOverride();
	}
}

int32 UGameFeatureAction_AddChunkOverride::GetLowestAllowedChunkId()
{
	if (const UGameFeatureAction_AddChunkOverride* Action = UGameFeatureAction_AddChunkOverride::StaticClass()->GetDefaultObject<UGameFeatureAction_AddChunkOverride>())
	{
		return Action->LowestAllowedChunkIndexForAutoGeneration;
	}
	else
	{
		ensureMsgf(false, TEXT("Unable to get class default object for UGameFeatureAction_AddChunkOverride"));
		return INDEX_NONE;
	}
}

#endif // WITH_EDITOR

void UGameFeatureAction_AddChunkOverride::AddChunkIdOverride()
{
#if WITH_EDITOR
	if (!bShouldOverrideChunk)
	{
		return;
	}
	if (ChunkId < 0)
	{
		UE_LOGF(LogAddChunkOverride, Error, "ChunkId is negative. Unable to override to a negative chunk");
		return;
	}

	if (UGameFeatureData* GameFeatureData = GetTypedOuter<UGameFeatureData>())
	{
		UChunkDependencyInfo* DependencyInfo = GetMutableDefault<UChunkDependencyInfo>();
		

		if (FChunkDependency* ExistingDep = DependencyInfo->DependencyArray.FindByPredicate([CheckChunk = ChunkId](const FChunkDependency& ChunkDep)
			{
				return ChunkDep.ChunkID == CheckChunk;
			}))
		{
			// If we found this chunk it might have been auto generated. Update this instead of adding ours.
			if (ExistingDep->ParentChunkID == 0)
			{
				ExistingDep->ParentChunkID = ParentChunk;
			}
		}
		else
		{
			FChunkDependency NewChunkDependency;
			NewChunkDependency.ChunkID = ChunkId;
			NewChunkDependency.ParentChunkID = ParentChunk;
			DependencyInfo->DependencyArray.Add(NewChunkDependency);
		}
		DependencyInfo->GetOrBuildChunkDependencyGraph(ChunkId, true);

		FString PluginName;
		GameFeatureData->GetPluginName(PluginName);
		TArray<FString>& PluginsInChunk = GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap.FindOrAdd(ChunkId);
		PluginsInChunk.Add(PluginName);
		GameFeatureAction_AddChunkOverride::PluginToChunkId.Add(PluginName, ChunkId);
		UE_LOGF(LogAddChunkOverride, Log, "Plugin(%ls) will cook assets into chunk(%d)", *PluginName, ChunkId);

		UAssetManager& Manager = UAssetManager::Get();

		FPrimaryAssetRules GFDRules;
		GFDRules.ChunkId = ChunkId;
		Manager.SetPrimaryAssetRules(GameFeatureData->GetPrimaryAssetId(), GFDRules);

		for (const FPrimaryAssetTypeInfo& AssetTypeInfo : GameFeatureData->GetPrimaryAssetTypesToScan())
		{
			FPrimaryAssetRulesCustomOverride Override;
			Override.PrimaryAssetType = FPrimaryAssetType(AssetTypeInfo.PrimaryAssetType);
			Override.FilterDirectory.Path = FString::Printf(TEXT("/%s"), *PluginName);
			Override.Rules.ChunkId = ChunkId;
			Manager.ApplyCustomPrimaryAssetRulesOverride(Override);
		}
	}
#endif // WITH_EDITOR
}

void UGameFeatureAction_AddChunkOverride::RemoveChunkIdOverride()
{
#if WITH_EDITOR
	// Remove primary asset rules by setting the override the default.
	if (UGameFeatureData* GameFeatureData = GetTypedOuter<UGameFeatureData>())
	{
		FString PluginName;
		GameFeatureData->GetPluginName(PluginName);
		if (!GameFeatureAction_AddChunkOverride::PluginToChunkId.Contains(PluginName))
		{
			UE_LOGF(LogAddChunkOverride, Verbose, "No chunk override found for (%ls) Skipping override removal", *PluginName);
			return;
		}

		const int32 ChunkIdOverride = GameFeatureAction_AddChunkOverride::PluginToChunkId[PluginName];
		if (GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap.Contains(ChunkIdOverride))
		{
			GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap[ChunkIdOverride].Remove(PluginName);
			if (GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap[ChunkIdOverride].IsEmpty())
			{
				GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap.Remove(ChunkIdOverride);
			}
		}
		UE_LOGF(LogAddChunkOverride, Log, "Removing ChunkId override (%d) for Plugin (%ls)", ChunkIdOverride, *PluginName);

		UAssetManager& Manager = UAssetManager::Get();

		Manager.SetPrimaryAssetRules(GameFeatureData->GetPrimaryAssetId(), FPrimaryAssetRules());
		for (const FPrimaryAssetTypeInfo& AssetTypeInfo : GameFeatureData->GetPrimaryAssetTypesToScan())
		{
			FPrimaryAssetRulesCustomOverride Override;
			Override.PrimaryAssetType = FPrimaryAssetType(AssetTypeInfo.PrimaryAssetType);
			Override.FilterDirectory.Path = FString::Printf(TEXT("/%s"), *PluginName);
			Manager.ApplyCustomPrimaryAssetRulesOverride(Override);
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
int32 UGameFeatureAction_AddChunkOverride::GenerateUniqueChunkId() const
{
	// Holdover auto-generation function until we can allow for Chunks to be specified by string name
	int32 NewChunkId = -1;
	UGameFeatureData* GameFeatureData = GetTypedOuter<UGameFeatureData>();
	if (ensure(GameFeatureData))
	{
		FString PluginName;
		GameFeatureData->GetPluginName(PluginName);

		uint32 NewId = GetTypeHash(PluginName);
		int16 SignedId = NewId;
		if (SignedId < 0)
		{
			SignedId = -SignedId;
		}
		NewChunkId = SignedId;
	}

	if (NewChunkId < LowestAllowedChunkIndexForAutoGeneration)
	{
		UE_LOGF(LogAddChunkOverride, Warning, "Autogenerated ChunkId(%d) is lower than the config specified LowestAllowedChunkIndexForAutoGeneration(%d)", NewChunkId, LowestAllowedChunkIndexForAutoGeneration);
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AddChunkOverride_InvalidId", "Autogenerated ChunkID is lower than config specified LowestAllowedChunkIndexForAutoGeneration. Please manually assign a valid Chunk Id"));
		NewChunkId = -1;
	}
	else if (GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap.Contains(NewChunkId))
	{
		UE_LOGF(LogAddChunkOverride, Warning, "ChunkId(%d) is in use by %ls. Unable to autogenerate unique id. Lowest allowed ChunkId(%d)", NewChunkId, *FString::Join(GameFeatureAction_AddChunkOverride::ChunkIdToPluginMap[ChunkId], TEXT(",")), LowestAllowedChunkIndexForAutoGeneration);
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AddChunkOverride_UsedChunkId", "Unable to auto generate unique valid Chunk Id. Please manually assign a valid Chunk Id"));
		NewChunkId = -1;
	}

	return NewChunkId;
}
#endif // WITH_EDITOR

//////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

