// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AddComponents.h"
#include "AssetRegistry/AssetBundleData.h"
#include "Components/GameFrameworkComponentManager.h"
#include "Engine/GameInstance.h"
#include "GameFeaturesSubsystemSettings.h"
#include "Engine/AssetManager.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AddComponents)

static TAutoConsoleVariable<bool> CVarUseNewWorldTracking(
	TEXT("GameFeaturePlugin.AddComponentsAction.UseNewWorldTracking"),
	true,
	TEXT("If true, the AddComponents GFA will keep track of world changes and update added components when NetMode changes."),
	ECVF_Default);

#define LOCTEXT_NAMESPACE "GameFeatures"

//////////////////////////////////////////////////////////////////////
// FGameFeatureComponentEntry
FGameFeatureComponentEntry::FGameFeatureComponentEntry()
	: bClientComponent(true)
	, bServerComponent(true)
	, AdditionFlags(static_cast<uint8>(EGameFrameworkAddComponentFlags::None))
{
}

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddComponents

void UGameFeatureAction_AddComponents::OnGameFeatureActivating(FGameFeatureActivatingContext& Context)
{
	if (CVarUseNewWorldTracking.GetValueOnAnyThread())
	{
		// Bind once to static GameInstance delegates
		if (ActivationContextDataMap.Num() == 0)
		{
			GameInstanceStartHandle = FWorldDelegates::OnStartGameInstance.AddUObject(this, &UGameFeatureAction_AddComponents::HandleGameInstanceStart_NewWorldTracking);
			GameInstanceWorldChangedHandle = FWorldDelegates::OnGameInstanceWorldChanged.AddUObject(this, &UGameFeatureAction_AddComponents::HandleGameInstanceWorldChanged);
		}

		// Keep track of this activation for GameInstances that start later
		FActivationContextData& ActivationContextData = ActivationContextDataMap.Add(Context);

		// Process all existing WorldContexts that apply to this activation
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (Context.ShouldApplyToWorldContext(WorldContext))
			{
				if (WorldContext.OwningGameInstance)
				{
					AddGameInstanceForActivation(WorldContext.OwningGameInstance, ActivationContextData);
				}
			}
		}
	}
	else
	{
		FContextHandles& Handles = ContextHandles.FindOrAdd(Context);

		Handles.GameInstanceStartHandle = FWorldDelegates::OnStartGameInstance.AddUObject(this,
			&UGameFeatureAction_AddComponents::HandleGameInstanceStart, FGameFeatureStateChangeContext(Context));

		ensure(Handles.ComponentRequestHandles.Num() == 0);

		// Add to any worlds with associated game instances that have already been initialized
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (Context.ShouldApplyToWorldContext(WorldContext))
			{
				AddToWorld(WorldContext, Handles);
			}
		}
	}
}

void UGameFeatureAction_AddComponents::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	if (CVarUseNewWorldTracking.GetValueOnAnyThread())
	{
		ActivationContextDataMap.Remove(Context);
		
		if (ActivationContextDataMap.Num() == 0)
		{
			FWorldDelegates::OnStartGameInstance.Remove(GameInstanceStartHandle);
			GameInstanceStartHandle.Reset();

			FWorldDelegates::OnGameInstanceWorldChanged.Remove(GameInstanceWorldChangedHandle);
			GameInstanceWorldChangedHandle.Reset();
		}
	}
	else
	{
		FContextHandles& Handles = ContextHandles.FindOrAdd(Context);

		FWorldDelegates::OnStartGameInstance.Remove(Handles.GameInstanceStartHandle);

		// Releasing the handles will also remove the components from any registered actors too
		Handles.ComponentRequestHandles.Empty();
	}
}

#if WITH_EDITORONLY_DATA
void UGameFeatureAction_AddComponents::AddAdditionalAssetBundleData(FAssetBundleData& AssetBundleData)
{
	if (UAssetManager::IsInitialized())
	{
		for (const FGameFeatureComponentEntry& Entry : ComponentList)
		{
			if (Entry.bClientComponent)
			{
				AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateClient, Entry.ComponentClass.ToSoftObjectPath().GetAssetPath());
			}
			if (Entry.bServerComponent)
			{
				AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateServer, Entry.ComponentClass.ToSoftObjectPath().GetAssetPath());
			}
		}
	}
}
#endif

#if WITH_EDITOR
EDataValidationResult UGameFeatureAction_AddComponents::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	int32 EntryIndex = 0;
	for (const FGameFeatureComponentEntry& Entry : ComponentList)
	{
		if (Entry.ActorClass.IsNull())
		{
			Result = EDataValidationResult::Invalid;
			Context.AddError(FText::Format(LOCTEXT("ComponentEntryHasNullActor", "Null ActorClass at index {0} in ComponentList"), FText::AsNumber(EntryIndex)));
		}

		if (Entry.ComponentClass.IsNull())
		{
			Result = EDataValidationResult::Invalid;
			Context.AddError(FText::Format(LOCTEXT("ComponentEntryHasNullComponent", "Null ComponentClass at index {0} in ComponentList"), FText::AsNumber(EntryIndex)));
		}

		++EntryIndex;
	}

	return Result;
}
#endif

void UGameFeatureAction_AddComponents::AddToWorld(const FWorldContext& WorldContext, FContextHandles& Handles)
{
	UWorld* World = WorldContext.World();
	UGameInstance* GameInstance = WorldContext.OwningGameInstance;

	if ((GameInstance != nullptr) && (World != nullptr) && World->IsGameWorld())
	{
		if (UGameFrameworkComponentManager* GFCM = UGameInstance::GetSubsystem<UGameFrameworkComponentManager>(GameInstance))
		{
			const ENetMode NetMode = World->GetNetMode();
			const bool bIsServer = NetMode != NM_Client;
			const bool bIsClient = NetMode != NM_DedicatedServer;

			UE_LOGF(LogGameFeatures, Verbose, "Adding components for %ls to world %ls (client: %d, server: %d)", *GetPathNameSafe(this), *World->GetDebugDisplayName(), bIsClient ? 1 : 0, bIsServer ? 1 : 0);
			
			for (const FGameFeatureComponentEntry& Entry : ComponentList)
			{
				const bool bShouldAddRequest = (bIsServer && Entry.bServerComponent) || (bIsClient && Entry.bClientComponent);
				if (bShouldAddRequest)
				{
					if (!Entry.ActorClass.IsNull())
					{
						UE_LOGF(LogGameFeatures, VeryVerbose, "Adding component to world %ls (%ls)", *World->GetDebugDisplayName(), *Entry.ComponentClass.ToString());
						UE_SCOPED_ENGINE_ACTIVITY(TEXT("Adding component to world %s (%s)"), *World->GetDebugDisplayName(), *Entry.ComponentClass.ToString());
						TSubclassOf<UActorComponent> ComponentClass = Entry.ComponentClass.LoadSynchronous();
						if (ComponentClass)
						{
							Handles.ComponentRequestHandles.Add(GFCM->AddComponentRequest(Entry.ActorClass, ComponentClass, static_cast<EGameFrameworkAddComponentFlags>(Entry.AdditionFlags)));
						}
						else if (!Entry.ComponentClass.IsNull())
						{
							UE_LOGF(LogGameFeatures, Error, "[GameFeatureData %ls]: Failed to load component class %ls. Not applying component.", *GetPathNameSafe(this), *Entry.ComponentClass.ToString());
						}
					}
				}
			}
		}
	}
}

void UGameFeatureAction_AddComponents::HandleGameInstanceStart(UGameInstance* GameInstance, FGameFeatureStateChangeContext ChangeContext)
{
	if (FWorldContext* WorldContext = GameInstance->GetWorldContext())
	{
		if (ChangeContext.ShouldApplyToWorldContext(*WorldContext))
		{
			FContextHandles* Handles = ContextHandles.Find(ChangeContext);
			if (ensure(Handles))
			{
				AddToWorld(*WorldContext, *Handles);
			}
		}
	}
}

void UGameFeatureAction_AddComponents::HandleGameInstanceStart_NewWorldTracking(UGameInstance* GameInstance)
{
	if (FWorldContext* WorldContext = GameInstance->GetWorldContext())
	{
		FObjectKey GameInstanceKey(GameInstance);

		// Add this GameInstance to all activation contexts that it applies to
		for (TPair<FGameFeatureStateChangeContext, FActivationContextData>& ActivationContextPair : ActivationContextDataMap)
		{
			const FGameFeatureStateChangeContext& ActivationContext = ActivationContextPair.Key;
			FActivationContextData& ActivationContextData = ActivationContextPair.Value;

			if (ActivationContext.ShouldApplyToWorldContext(*WorldContext))
			{
				AddGameInstanceForActivation(GameInstance, ActivationContextData);
			}
		}
	}
}

void UGameFeatureAction_AddComponents::HandleGameInstanceWorldChanged(UGameInstance* GameInstance, UWorld* OldWorld, UWorld* NewWorld)
{
	FObjectKey GameInstanceKey(GameInstance);

	for (TPair<FGameFeatureStateChangeContext, FActivationContextData>& ActivationContextPair : ActivationContextDataMap)
	{
		if (FGameInstanceData* GameInstanceData = ActivationContextPair.Value.GameInstanceDataMap.Find(FObjectKey(GameInstance)))
		{
			UGameFrameworkComponentManager* GFCM = UGameInstance::GetSubsystem<UGameFrameworkComponentManager>(GameInstance);
			if (NewWorld && NewWorld->IsGameWorld() && GFCM)
			{
				// New world may have a different NetMode, update component requests
				UpdateComponentsOnManager(NewWorld, GFCM, *GameInstanceData);
			}
			else
			{
				// World set to null, reset component requests
				GameInstanceData->WorldNetMode = NM_MAX;
				GameInstanceData->ComponentRequestHandles.Reset();
			}
		}
		
	}
}

void UGameFeatureAction_AddComponents::AddGameInstanceForActivation(TNotNull<UGameInstance*> GameInstance, FActivationContextData& ActivationContextData)
{
	FGameInstanceData& GameInstanceData = ActivationContextData.GameInstanceDataMap.FindOrAdd(FObjectKey(GameInstance));

	UWorld* World = GameInstance->GetWorld();
	UGameFrameworkComponentManager* GFCM = UGameInstance::GetSubsystem<UGameFrameworkComponentManager>(GameInstance);
	if (!World || !World->IsGameWorld() || !GFCM)
	{
		return;
	}

	UpdateComponentsOnManager(World, GFCM, GameInstanceData);
}

void UGameFeatureAction_AddComponents::UpdateComponentsOnManager(TNotNull<UWorld*> World, TNotNull<UGameFrameworkComponentManager*> Manager, FGameInstanceData& GameInstanceData)
{
	const bool bInitialAdd = (GameInstanceData.WorldNetMode == NM_MAX);
	const bool bWasServer = !bInitialAdd && (GameInstanceData.WorldNetMode != NM_Client);
	const bool bWasClient = !bInitialAdd && (GameInstanceData.WorldNetMode != NM_DedicatedServer);

	const ENetMode NetMode = World->GetNetMode();
	const bool bIsServer = NetMode != NM_Client;
	const bool bIsClient = NetMode != NM_DedicatedServer;
	GameInstanceData.WorldNetMode = NetMode;

	// No change in NetMode that affected client/server conditions, no components to update
	if ((bWasServer == bIsServer) && (bWasClient == bIsClient))
	{
		return;
	}

	if (bInitialAdd)
	{
		// Fill our handle array with null entries to start
		GameInstanceData.ComponentRequestHandles.AddDefaulted(ComponentList.Num());

		UE_LOGF(LogGameFeatures, Verbose, "Adding components for %ls to world %ls (client: %d, server: %d)", *GetPathNameSafe(this), *World->GetDebugDisplayName(), bIsClient ? 1 : 0, bIsServer ? 1 : 0);
	}
	else
	{
		UE_LOGF(LogGameFeatures, Verbose, "Updating components for %ls to world %ls (client: %d->%d, server: %d->%d)", *GetPathNameSafe(this), *World->GetDebugDisplayName(),
			bWasClient ? 1 : 0, bIsClient ? 1 : 0, bWasServer ? 1 : 0, bIsServer ? 1 : 0);
	}

	// Verify arrays are of equal length
	if (!ensure(ComponentList.Num() == GameInstanceData.ComponentRequestHandles.Num()))
	{
		return;
	}
	
	for (int32 i = 0; i < ComponentList.Num(); ++i)
	{
		const FGameFeatureComponentEntry& Entry = ComponentList[i];
		TSharedPtr<FComponentRequestHandle>& RequestHandle = GameInstanceData.ComponentRequestHandles[i];

		const bool bShouldAddRequest = (bIsServer && Entry.bServerComponent) || (bIsClient && Entry.bClientComponent);
		if (bShouldAddRequest && !RequestHandle.IsValid())
		{
			RequestHandle = AddComponentRequest(World, Manager, Entry);
		}
		else if (!bShouldAddRequest && RequestHandle.IsValid())
		{
			UE_LOGF(LogGameFeatures, VeryVerbose, "Removing component to world %ls (%ls)", *World->GetDebugDisplayName(), *Entry.ComponentClass.ToString());
			UE_SCOPED_ENGINE_ACTIVITY(TEXT("Removing component from world %s (%s)"), *World->GetDebugDisplayName(), *Entry.ComponentClass.ToString());
			RequestHandle.Reset();
		}
	}
}

TSharedPtr<FComponentRequestHandle> UGameFeatureAction_AddComponents::AddComponentRequest(TNotNull<UWorld*> World, TNotNull<UGameFrameworkComponentManager*> Manager, const FGameFeatureComponentEntry& Entry)
{
	if (!Entry.ActorClass.IsNull())
	{
		UE_LOGF(LogGameFeatures, VeryVerbose, "Adding component to world %ls (%ls)", *World->GetDebugDisplayName(), *Entry.ComponentClass.ToString());
		UE_SCOPED_ENGINE_ACTIVITY(TEXT("Adding component to world %s (%s)"), *World->GetDebugDisplayName(), *Entry.ComponentClass.ToString());
		TSubclassOf<UActorComponent> ComponentClass = Entry.ComponentClass.LoadSynchronous();
		if (ComponentClass)
		{
			return Manager->AddComponentRequest(Entry.ActorClass, ComponentClass, static_cast<EGameFrameworkAddComponentFlags>(Entry.AdditionFlags));
		}
		else if (!Entry.ComponentClass.IsNull())
		{
			UE_LOGF(LogGameFeatures, Error, "[GameFeatureData %ls]: Failed to load component class %ls. Not applying component.", *GetPathNameSafe(this), *Entry.ComponentClass.ToString());
		}
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

