// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AddWPContent.h"

#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleClient.h"
#include "GameFeatureData.h"
#include "GameFeaturesSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AddWPContent)

UGameFeatureAction_AddWPContent::UGameFeatureAction_AddWPContent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ContentBundleDescriptor = ObjectInitializer.CreateDefaultSubobject<UContentBundleDescriptor>(this, TEXT("ContentBundleDescriptor"));

#if WITH_EDITOR
	if (UGameFeatureData* GameFeatureData = GetTypedOuter<UGameFeatureData>())
	{
		ContentBundleDescriptor->InitializeObject(GameFeatureData->GetName());
	}
#endif
}

void UGameFeatureAction_AddWPContent::OnGameFeatureRegistering()
{
	Super::OnGameFeatureRegistering();

	ContentBundleClient = FContentBundleClient::CreateClient(ContentBundleDescriptor, GetTypedOuter<UGameFeatureData>()->GetName());
	
#if WITH_EDITOR
	if (IsRunningCommandlet() && ContentBundleClient != nullptr)
	{
		ContentBundleClient->RequestContentInjection();
	}
#endif 
}

void UGameFeatureAction_AddWPContent::OnGameFeatureUnregistering()
{
	if (ContentBundleClient != nullptr)
	{
		ContentBundleClient->RequestUnregister();
		ContentBundleClient = nullptr;
	}

	Super::OnGameFeatureUnregistering();
}

void UGameFeatureAction_AddWPContent::OnGameFeatureActivating()
{
	Super::OnGameFeatureActivating();
	
	if (ContentBundleClient)
	{
		ContentBundleClient->RequestContentInjection();
	}
}

void UGameFeatureAction_AddWPContent::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	if (ContentBundleClient)
	{
		ContentBundleClient->RequestRemoveContent();
	}

	Super::OnGameFeatureDeactivating(Context);
}
