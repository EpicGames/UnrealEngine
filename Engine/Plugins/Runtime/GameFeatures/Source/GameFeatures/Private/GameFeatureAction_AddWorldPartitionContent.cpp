// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AddWorldPartitionContent.h"
#include "WorldPartition/DataLayer/ExternalDataLayerEngineSubsystem.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "GameFeaturesSubsystem.h"
#include "GameFeatureData.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AddWorldPartitionContent)

UGameFeatureAction_AddWorldPartitionContent::UGameFeatureAction_AddWorldPartitionContent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UGameFeatureAction_AddWorldPartitionContent::OnGameFeatureRegistering()
{
	Super::OnGameFeatureRegistering();
	if (ExternalDataLayerAsset)
	{
		check(IsValid(ExternalDataLayerAsset));
		UExternalDataLayerEngineSubsystem::Get().RegisterExternalDataLayerAsset(ExternalDataLayerAsset, this);
	}
}

void UGameFeatureAction_AddWorldPartitionContent::OnGameFeatureUnregistering()
{
	if (ExternalDataLayerAsset)
	{
		check(IsValid(ExternalDataLayerAsset));
		UExternalDataLayerEngineSubsystem::Get().UnregisterExternalDataLayerAsset(ExternalDataLayerAsset, this);
	}
	Super::OnGameFeatureUnregistering();
}

void UGameFeatureAction_AddWorldPartitionContent::OnGameFeatureActivating(FGameFeatureActivatingContext& Context)
{
	Super::OnGameFeatureActivating(Context);

	if (ExternalDataLayerAsset && ShouldActivateExternalDataLayer())
	{
		check(IsValid(ExternalDataLayerAsset));
		UExternalDataLayerEngineSubsystem::Get().ActivateExternalDataLayerAsset(ExternalDataLayerAsset, this);
	}
}

void UGameFeatureAction_AddWorldPartitionContent::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	if (ExternalDataLayerAsset)
	{
		check(IsValid(ExternalDataLayerAsset));
		UExternalDataLayerEngineSubsystem::Get().DeactivateExternalDataLayerAsset(ExternalDataLayerAsset, this);
	}
	Super::OnGameFeatureDeactivating(Context);
}

bool UGameFeatureAction_AddWorldPartitionContent::ShouldActivateExternalDataLayer() const
{
	if (RequiredCVar.IsNone())
	{
		return true;
	}

	const IConsoleVariable* RequiredCVarPtr = IConsoleManager::Get().FindConsoleVariable(*RequiredCVar.ToString());
	if (!RequiredCVarPtr)
	{
		UE_LOG(LogGameFeatures, Display, TEXT("[%ls] RequiredCVarName '%ls' could not be found; treating ExternalDataLayer as inactive."),
			*GetFullName(), *RequiredCVar.ToString());
		return false;
	}

	return RequiredCVarPtr->GetBool();
}

#if WITH_EDITOR

void UGameFeatureAction_AddWorldPartitionContent::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	PreEditChangeExternalDataLayerAsset.Reset();
	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(UGameFeatureAction_AddWorldPartitionContent, ExternalDataLayerAsset))
	{
		PreEditChangeExternalDataLayerAsset = ExternalDataLayerAsset;
	}
}

void UGameFeatureAction_AddWorldPartitionContent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UGameFeatureAction_AddWorldPartitionContent, ExternalDataLayerAsset))
	{
		if (PreEditChangeExternalDataLayerAsset != ExternalDataLayerAsset)
		{
			OnExternalDataLayerAssetChanged(PreEditChangeExternalDataLayerAsset.Get(), ExternalDataLayerAsset);
		}
	}
	PreEditChangeExternalDataLayerAsset.Reset();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UGameFeatureAction_AddWorldPartitionContent::PreEditUndo()
{
	Super::PreEditUndo();

	PreEditUndoExternalDataLayerAsset = ExternalDataLayerAsset;
}

void UGameFeatureAction_AddWorldPartitionContent::PostEditUndo()
{
	if (PreEditUndoExternalDataLayerAsset != ExternalDataLayerAsset)
	{
		OnExternalDataLayerAssetChanged(PreEditUndoExternalDataLayerAsset.Get(), ExternalDataLayerAsset);
	}
	PreEditUndoExternalDataLayerAsset.Reset();

	Super::PostEditUndo();
}

void UGameFeatureAction_AddWorldPartitionContent::OnExternalDataLayerAssetChanged(const UExternalDataLayerAsset* OldAsset, const UExternalDataLayerAsset* NewAsset)
{
	// Detect if there's data associated to this EDL
	// Decide whether the data will be deleted or simply left there (unused)
	UExternalDataLayerEngineSubsystem& ExternalDataLayerEngineSubsystem = UExternalDataLayerEngineSubsystem::Get();
	if (OldAsset)
	{
		check(IsValid(OldAsset));
		if (ExternalDataLayerEngineSubsystem.IsExternalDataLayerAssetRegistered(OldAsset, this))
		{
			ExternalDataLayerEngineSubsystem.UnregisterExternalDataLayerAsset(OldAsset, this);
		}
	}

	if (NewAsset)
	{
		check(IsValid(NewAsset));
		if (IsGameFeaturePluginRegistered() && !ExternalDataLayerEngineSubsystem.IsExternalDataLayerAssetRegistered(NewAsset, this))
		{
			ExternalDataLayerEngineSubsystem.RegisterExternalDataLayerAsset(NewAsset, this);
		}
		if (IsGameFeaturePluginActive() && !ExternalDataLayerEngineSubsystem.IsExternalDataLayerAssetActive(NewAsset, this))
		{
			ExternalDataLayerEngineSubsystem.ActivateExternalDataLayerAsset(NewAsset, this);
		}
	}
}

#endif