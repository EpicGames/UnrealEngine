// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_AudioActionBase.h"
#include "AudioDeviceManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_AudioActionBase)

#define LOCTEXT_NAMESPACE "GameFeatures"

void UGameFeatureAction_AudioActionBase::OnGameFeatureActivating(FGameFeatureActivatingContext& Context)
{
	DeviceCreatedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddUObject(this, &UGameFeatureAction_AudioActionBase::OnDeviceCreated);
	DeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddUObject(this, &UGameFeatureAction_AudioActionBase::OnDeviceDestroyed);

	// Add to any existing devices
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		AudioDeviceManager->IterateOverAllDevices([this](Audio::FDeviceId DeviceId, FAudioDevice* InDevice)
		{
			AddToDevice(FAudioDeviceManager::Get()->GetAudioDevice(DeviceId));
		});
	}
}

void UGameFeatureAction_AudioActionBase::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	// Remove from any existing devices
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		AudioDeviceManager->IterateOverAllDevices([this](Audio::FDeviceId DeviceId, FAudioDevice* InDevice)
		{
			RemoveFromDevice(FAudioDeviceManager::Get()->GetAudioDevice(DeviceId));
		});
	}

	FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(DeviceCreatedHandle);
	FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);
}

void UGameFeatureAction_AudioActionBase::OnDeviceCreated(Audio::FDeviceId InDeviceId)
{
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		AddToDevice(AudioDeviceManager->GetAudioDevice(InDeviceId));
	}	
}

void UGameFeatureAction_AudioActionBase::OnDeviceDestroyed(Audio::FDeviceId InDeviceId)
{
	if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
	{
		FAudioDeviceHandle AudioDeviceHandle = AudioDeviceManager->GetAudioDevice(InDeviceId);
		if (AudioDeviceHandle.IsValid())
		{
			RemoveFromDevice(AudioDeviceHandle);
		}
	}
}

#undef LOCTEXT_NAMESPACE
