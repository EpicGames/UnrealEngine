// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_DataRegistry.h"
#include "AssetRegistry/AssetBundleData.h"
#include "GameFeaturesSubsystemSettings.h"
#include "GameFeaturesSubsystem.h"
#include "DataRegistrySubsystem.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_DataRegistry)

#define LOCTEXT_NAMESPACE "GameFeatures"

void UGameFeatureAction_DataRegistry::OnGameFeatureRegistering()
{
	Super::OnGameFeatureRegistering();

	if (ShouldPreloadAtRegistration())
	{
		// TODO: Right now this loads the source for both editor and runtime usage, in the future the preload could be changed to only allow resolves and not full data gets

		UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get();
		if (ensure(DataRegistrySubsystem))
		{
			for (const TSoftObjectPtr<UDataRegistry>& RegistryToAdd : RegistriesToAdd)
			{
				if (!RegistryToAdd.IsNull())
				{
					const FSoftObjectPath RegistryPath = RegistryToAdd.ToSoftObjectPath();

					UE_LOGF(LogGameFeatures, Log, "OnGameFeatureRegistering %ls: Preloading DataRegistry %ls for editor preview", *GetPathName(), *RegistryPath.ToString())
					DataRegistrySubsystem->LoadRegistryPath(RegistryPath);
				}
			}
		}
	}
}

void UGameFeatureAction_DataRegistry::OnGameFeatureActivating()
{
	Super::OnGameFeatureActivating();

	if (ShouldPreloadAtRegistration())
	{
		// This already happened at registration
		return;
	}

	UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get();
	if (ensure(DataRegistrySubsystem))
	{
		for (const TSoftObjectPtr<UDataRegistry>& RegistryToAdd : RegistriesToAdd)
		{
			if (!RegistryToAdd.IsNull())
			{
				const FSoftObjectPath RegistryPath = RegistryToAdd.ToSoftObjectPath();

#if !UE_BUILD_SHIPPING
				// If we're after data registry startup, then this asset should already exist in memory from either the bundle preload or game-specific logic
				if (DataRegistrySubsystem->AreRegistriesInitialized())
				{
					UDataRegistry* LoadedRegistry = RegistryToAdd.Get();
					if (!LoadedRegistry)
					{
						UE_LOGF(LogGameFeatures, Log, "OnGameFeatureActivating %ls: DataRegistry %ls was not loaded before activation, this may cause a long hitch", *GetPathName(), *RegistryPath.ToString())
					}
					else
					{
						// Need to verify this isn't already registered
						TArray<UDataRegistry*> RegistryList;
						DataRegistrySubsystem->GetAllRegistries(RegistryList);

						if (RegistryList.Contains(LoadedRegistry))
						{
							UE_LOGF(LogGameFeatures, Log, "OnGameFeatureActivating %ls: DataRegistry %ls is already enabled from another source! This can cause problems on deactivation", *GetPathName(), *RegistryPath.ToString())
						}
					}
				}
#endif

				DataRegistrySubsystem->LoadRegistryPath(RegistryPath);
			}
		}
	}
}

void UGameFeatureAction_DataRegistry::OnGameFeatureUnregistering()
{
	Super::OnGameFeatureUnregistering();

	if (ShouldPreloadAtRegistration())
	{
		UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get();
		if (ensure(DataRegistrySubsystem))
		{
			for (const TSoftObjectPtr<UDataRegistry>& RegistryToAdd : RegistriesToAdd)
			{
				if (!RegistryToAdd.IsNull())
				{
					const FSoftObjectPath RegistryPath = RegistryToAdd.ToSoftObjectPath();

					// This should only happen when the user is manually changing phase via the feature editor UI
					UE_LOGF(LogGameFeatures, Log, "OnGameFeatureUnregistering %ls: Temporarily disabling preloaded DataRegistry %ls", *GetPathName(), *RegistryPath.ToString())

					DataRegistrySubsystem->IgnoreRegistryPath(RegistryPath);
				}
			}
		}
	}
}

void UGameFeatureAction_DataRegistry::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	Super::OnGameFeatureDeactivating(Context);

	if (ShouldPreloadAtRegistration())
	{
		// This will only happen at unregistration
		return;
	}

	UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get();
	if (ensure(DataRegistrySubsystem))
	{
		for (const TSoftObjectPtr<UDataRegistry>& RegistryToAdd : RegistriesToAdd)
		{
			if (!RegistryToAdd.IsNull())
			{
				// This does not do any reference counting, warning above will hit if this is registered from two separate sources and then the first to deactivate will remove it
				const FSoftObjectPath RegistryPath = RegistryToAdd.ToSoftObjectPath();
				DataRegistrySubsystem->IgnoreRegistryPath(RegistryPath);
			}
		}
	}
}

bool UGameFeatureAction_DataRegistry::ShouldPreloadAtRegistration()
{
	if (IsRunningCommandlet())
	{
		return bPreloadInCommandlets;
	}
	else
	{
		return GIsEditor && bPreloadInEditor;
	}
}

#if WITH_EDITORONLY_DATA
void UGameFeatureAction_DataRegistry::AddAdditionalAssetBundleData(FAssetBundleData& AssetBundleData)
{
	Super::AddAdditionalAssetBundleData(AssetBundleData);
	for (const TSoftObjectPtr<UDataRegistry>& RegistryToAdd : RegistriesToAdd)
	{
		if(!RegistryToAdd.IsNull())
		{
			const FTopLevelAssetPath RegistryPath = RegistryToAdd.ToSoftObjectPath().GetAssetPath();

			// Add for both clients and servers, this will not work properly for games that do not set those bundle states
			// @TODO: If another way to preload specific assets is added, switch to that so it works regardless of bundles
			AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateClient, RegistryPath);
			AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateServer, RegistryPath);
		}
	}
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
EDataValidationResult UGameFeatureAction_DataRegistry::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	int32 EntryIndex = 0;
	for (const TSoftObjectPtr<UDataRegistry>& RegistryToAdd : RegistriesToAdd)
	{
		if (RegistryToAdd.IsNull())
		{
			Context.AddError(FText::Format(LOCTEXT("DataRegistryMissingSource", "No valid data registry specified at index {0} in RegistriesToAdd"), FText::AsNumber(EntryIndex)));
			Result = EDataValidationResult::Invalid;
		}

		++EntryIndex;
	}

	return Result;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

