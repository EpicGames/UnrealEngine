// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_DataRegistrySource.h"
#include "AssetRegistry/AssetBundleData.h"
#include "GameFeaturesSubsystemSettings.h"
#include "Engine/CurveTable.h"
#include "GameFeaturesProjectPolicies.h"
#include "DataRegistrySubsystem.h"
#include "Engine/DataTable.h"

#if WITH_EDITOR
#include "Misc/DataValidation.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_DataRegistrySource)

#define LOCTEXT_NAMESPACE "GameFeatures"

void UGameFeatureAction_DataRegistrySource::OnGameFeatureRegistering()
{
	Super::OnGameFeatureRegistering();

	if (ShouldPreloadAtRegistration())
	{
		// TODO: Right now this loads the source for both editor and runtime usage, in the future the preload could be changed to only allow resolves and not full data gets

		UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get();
		if (ensure(DataRegistrySubsystem))
		{
			for (const FDataRegistrySourceToAdd& RegistrySource : SourcesToAdd)
			{
				// Don't check the client/server flags as they won't work properly
				TMap<FDataRegistryType, TArray<FSoftObjectPath>> AssetMap;
				TArray<FSoftObjectPath>& AssetList = AssetMap.Add(RegistrySource.RegistryToAddTo);

				if (!RegistrySource.DataTableToAdd.IsNull())
				{
					UE_LOGF(LogGameFeatures, Log, "OnGameFeatureRegistering %ls: Preloading DataRegistrySource %ls for editor preview", *GetPathName(), *RegistrySource.DataTableToAdd.ToString())
						AssetList.Add(RegistrySource.DataTableToAdd.ToSoftObjectPath());
				}

				if (!RegistrySource.CurveTableToAdd.IsNull())
				{
					UE_LOGF(LogGameFeatures, Log, "OnGameFeatureRegistering %ls: Preloading DataRegistrySource %ls for editor preview", *GetPathName(), *RegistrySource.CurveTableToAdd.ToString())
						AssetList.Add(RegistrySource.CurveTableToAdd.ToSoftObjectPath());
				}

				DataRegistrySubsystem->PreregisterSpecificAssets(AssetMap, RegistrySource.AssetPriority);
			}
		}
	}
}

void UGameFeatureAction_DataRegistrySource::RegisterRuntimeSources()
{
	if (ShouldPreloadAtRegistration())
	{
		// This already happened at registration
		return;
	}

	UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get();
	if (ensure(DataRegistrySubsystem))
	{
		UGameFeaturesProjectPolicies& Policy = UGameFeaturesSubsystem::Get().GetPolicy<UGameFeaturesProjectPolicies>();
		bool bIsClient, bIsServer;

		Policy.GetGameFeatureLoadingMode(bIsClient, bIsServer);

		for (const FDataRegistrySourceToAdd& RegistrySource : SourcesToAdd)
		{
			const bool bShouldAdd = (bIsServer && RegistrySource.bServerSource) || (bIsClient && RegistrySource.bClientSource);
			if (bShouldAdd)
			{
				TMap<FDataRegistryType, TArray<FSoftObjectPath>> AssetMap;
				TArray<FSoftObjectPath>& AssetList = AssetMap.Add(RegistrySource.RegistryToAddTo);

				if (!RegistrySource.DataTableToAdd.IsNull())
				{
					AssetList.Add(RegistrySource.DataTableToAdd.ToSoftObjectPath());
				}

				if (!RegistrySource.CurveTableToAdd.IsNull())
				{
					AssetList.Add(RegistrySource.CurveTableToAdd.ToSoftObjectPath());
				}

#if !UE_BUILD_SHIPPING
				// If we're after data registry startup, then this asset should already exist in memory from either the bundle preload or game-specific logic
				if (DataRegistrySubsystem->AreRegistriesInitialized())
				{
					if (!RegistrySource.DataTableToAdd.IsNull() && !RegistrySource.DataTableToAdd.IsValid())
					{
						UE_LOGF(LogGameFeatures, Log, "RegisterRuntimeSources %ls: DataRegistry source asset %ls was not loaded before activation, this may cause a long hitch", *GetPathName(), *RegistrySource.DataTableToAdd.ToString())
					}

					if (!RegistrySource.CurveTableToAdd.IsNull() && !RegistrySource.CurveTableToAdd.IsValid())
					{
						UE_LOGF(LogGameFeatures, Log, "RegisterRuntimeSources %ls: DataRegistry source asset %ls was not loaded before activation, this may cause a long hitch", *GetPathName(), *RegistrySource.DataTableToAdd.ToString())
					}
				}
#endif

				// This will either load the sources immediately, or schedule them for load when registries are initialized
				DataRegistrySubsystem->PreregisterSpecificAssets(AssetMap, RegistrySource.AssetPriority);
			}
		}
	}
}

void UGameFeatureAction_DataRegistrySource::OnGameFeatureActivating(FGameFeatureActivatingContext& Context)
{
	Super::OnGameFeatureActivating(Context);

	RegisterRuntimeSources();
}
	
void UGameFeatureAction_DataRegistrySource::UnregisterSources()
{
	UDataRegistrySubsystem* DataRegistrySubsystem = UDataRegistrySubsystem::Get();
	if (ensure(DataRegistrySubsystem))
	{
		for (const FDataRegistrySourceToAdd& RegistrySource : SourcesToAdd)
		{
			if (!RegistrySource.DataTableToAdd.IsNull())
			{
				if (!DataRegistrySubsystem->UnregisterSpecificAsset(RegistrySource.RegistryToAddTo, RegistrySource.DataTableToAdd.ToSoftObjectPath()))
				{
					UE_LOGF(LogGameFeatures, Log, "UnregisterSources %ls: DataRegistry data table %ls failed to unregister", *GetPathName(), *RegistrySource.DataTableToAdd.ToString())
				}
				else
				{
					UE_LOGF(LogGameFeatures, Log, "UnregisterSources %ls: Temporarily disabling data table %ls", *GetPathName(), *RegistrySource.DataTableToAdd.ToString())
				}
			}

			if (!RegistrySource.CurveTableToAdd.IsNull())
			{
				if (!DataRegistrySubsystem->UnregisterSpecificAsset(RegistrySource.RegistryToAddTo, RegistrySource.CurveTableToAdd.ToSoftObjectPath()))
				{
					UE_LOGF(LogGameFeatures, Log, "UnregisterSources %ls: DataRegistry curve table %ls failed to unregister", *GetPathName(), *RegistrySource.CurveTableToAdd.ToString())
				}
				else
				{
					UE_LOGF(LogGameFeatures, Log, "UnregisterSources %ls: Temporarily disabling curve table %ls", *GetPathName(), *RegistrySource.CurveTableToAdd.ToString())
				}
			}
		}
	}
}

void UGameFeatureAction_DataRegistrySource::OnGameFeatureUnregistering()
{
	Super::OnGameFeatureUnregistering();

	if (ShouldPreloadAtRegistration())
	{
		// This should only happen when the user is manually changing phase via the feature editor UI
		UnregisterSources();
	}
}

void UGameFeatureAction_DataRegistrySource::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	Super::OnGameFeatureDeactivating(Context);

	if (ShouldPreloadAtRegistration())
	{
		// This will only happen at unregistration
		return;
	}

	UnregisterSources();
}

bool UGameFeatureAction_DataRegistrySource::ShouldPreloadAtRegistration()
{
	// We want to preload in interactive editor sessions only
	return (GIsEditor && !IsRunningCommandlet() && bPreloadInEditor);
}

#if WITH_EDITORONLY_DATA
void UGameFeatureAction_DataRegistrySource::AddAdditionalAssetBundleData(FAssetBundleData& AssetBundleData)
{
	Super::AddAdditionalAssetBundleData(AssetBundleData);
	for (const FDataRegistrySourceToAdd& RegistrySource : SourcesToAdd)
	{
		// Register table assets for preloading, this will only work if the game uses client/server bundle states
		// @TODO: If another way of preloading data is added, client+server sources should use that instead

		if (!RegistrySource.DataTableToAdd.IsNull())
		{
			const FTopLevelAssetPath DataTableSourcePath = RegistrySource.DataTableToAdd.ToSoftObjectPath().GetAssetPath();
			if (RegistrySource.bClientSource)
			{
				AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateClient, DataTableSourcePath);
			}
			if (RegistrySource.bServerSource)
			{
				AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateServer, DataTableSourcePath);
			}
		}

		if (!RegistrySource.CurveTableToAdd.IsNull())
		{
			const FTopLevelAssetPath CurveTableSourcePath = RegistrySource.CurveTableToAdd.ToSoftObjectPath().GetAssetPath();
			if (RegistrySource.bClientSource)
			{
				AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateClient, CurveTableSourcePath);
			}
			if (RegistrySource.bServerSource)
			{
				AssetBundleData.AddBundleAsset(UGameFeaturesSubsystemSettings::LoadStateServer, CurveTableSourcePath);
			}
		}
	}
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
EDataValidationResult UGameFeatureAction_DataRegistrySource::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Result = CombineDataValidationResults(Super::IsDataValid(Context), EDataValidationResult::Valid);

	int32 EntryIndex = 0;
	for (const FDataRegistrySourceToAdd& Entry : SourcesToAdd)
	{
		if (Entry.CurveTableToAdd.IsNull() && Entry.DataTableToAdd.IsNull())
		{
			Context.AddError(FText::Format(LOCTEXT("DataRegistrySourceMissingSource", "No valid data table or curve table specified at index {0} in SourcesToAdd"), FText::AsNumber(EntryIndex)));
			Result = EDataValidationResult::Invalid;
		}

		if (!Entry.CurveTableToAdd.IsNull())
		{
			FAssetData TableAssetData = IAssetRegistry::Get()->GetAssetByObjectPath(Entry.CurveTableToAdd.ToSoftObjectPath());

			// This will catch normal curve tables, composites, and any reasonably named subclass without doing a slow load
			if (!TableAssetData.IsValid() || !TableAssetData.AssetClassPath.GetAssetName().ToString().Contains(TEXT("CurveTable")))
			{
				Context.AddError(FText::Format(LOCTEXT("DataRegistrySourceMissingCurveTable", "Path {0} does not point to valid curvetable at index {1} in SourcesToAdd"), FText::FromString(Entry.CurveTableToAdd.ToString()), FText::AsNumber(EntryIndex)));
				Result = EDataValidationResult::Invalid;
			}
		}

		if (!Entry.DataTableToAdd.IsNull())
		{
			FAssetData TableAssetData = IAssetRegistry::Get()->GetAssetByObjectPath(Entry.DataTableToAdd.ToSoftObjectPath());

			// This will catch normal data tables, composites, and any reasonably named subclass without doing a slow load
			if (!TableAssetData.IsValid() || !TableAssetData.AssetClassPath.GetAssetName().ToString().Contains(TEXT("DataTable")))
			{
				Context.AddError(FText::Format(LOCTEXT("DataRegistrySourceMissingDataTable", "Path {0} does not point to valid datatable at index {1} in SourcesToAdd"), FText::FromString(Entry.DataTableToAdd.ToString()), FText::AsNumber(EntryIndex)));
				Result = EDataValidationResult::Invalid;
			}
		}

		if (Entry.bServerSource == false && Entry.bClientSource == false)
		{
			Context.AddError(FText::Format(LOCTEXT("DataRegistrySourceNeverUsed", "Source not specified to load on either client or server, it will be unused at index {0} in SourcesToAdd"), FText::AsNumber(EntryIndex)));
			Result = EDataValidationResult::Invalid;
		}

		if (Entry.RegistryToAddTo.IsNone())
		{
			Context.AddError(FText::Format(LOCTEXT("DataRegistrySourceInvalidRegistry", "Source specified an invalid name (NONE) as the target registry at index {0} in SourcesToAdd"), FText::AsNumber(EntryIndex)));
			Result = EDataValidationResult::Invalid;
		}

		++EntryIndex;
	}

	return Result;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UGameFeatureAction_DataRegistrySource::AddSource(const FDataRegistrySourceToAdd& NewSource)
{
	SourcesToAdd.Add(NewSource);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

