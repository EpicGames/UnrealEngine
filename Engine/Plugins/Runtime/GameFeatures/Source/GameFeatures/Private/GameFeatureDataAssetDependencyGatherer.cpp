// Copyright Epic Games, Inc. All Rights Reserved.
#include "GameFeatureDataAssetDependencyGatherer.h"
#include "Misc/PackageName.h"

#if WITH_EDITOR

#include "AssetRegistry/ARFilter.h"
#include "GameFeatureData.h"
#include "AssetRegistry/AssetRegistryState.h"

// Register FGameFeatureDataAssetDependencyGatherer for UGameFeatureData class
REGISTER_ASSETDEPENDENCY_GATHERER(FGameFeatureDataAssetDependencyGatherer, UGameFeatureData);

void FGameFeatureDataAssetDependencyGatherer::GatherDependencies(FGatherDependenciesContext& Context) const
{
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = true;

	TArray<FString> DependencyDirectories;
	UGameFeatureData::GetDependencyDirectoriesFromAssetData(Context.GetAssetData(), DependencyDirectories);
	for (const FString& DependencyDirectory : DependencyDirectories)
	{
		Context.GetOutDependencyDirectories().Add(DependencyDirectory);
		Filter.PackagePaths.Add(*DependencyDirectory);
	}

	if (Filter.PackagePaths.Num() > 0)
	{
		TArray<FAssetData> FilteredAssets;
		Context.GetAssetRegistryState().GetAssets(Context.CompileFilter(Filter), {}, FilteredAssets, true);

		for (const FAssetData& FilteredAsset : FilteredAssets)
		{
			Context.GetOutDependencies().Emplace(IAssetDependencyGatherer::FGathereredDependency{ FilteredAsset.PackageName,
				UE::AssetRegistry::EDependencyProperty::Game | UE::AssetRegistry::EDependencyProperty::Build });
		}
	}
}

#endif
