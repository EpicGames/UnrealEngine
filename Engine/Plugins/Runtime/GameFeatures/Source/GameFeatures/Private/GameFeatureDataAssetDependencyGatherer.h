// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "AssetRegistry/AssetDependencyGatherer.h"

class FGameFeatureDataAssetDependencyGatherer : public IAssetDependencyGatherer
{
public:
	
	FGameFeatureDataAssetDependencyGatherer() = default;
	virtual ~FGameFeatureDataAssetDependencyGatherer() = default;

	virtual void GatherDependencies(FGatherDependenciesContext& Context) const override;
};

#endif
