// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureOptionalContentInstaller.h"

#include "Algo/AllOf.h"
#include "GameFeaturePluginOperationResult.h"
#include "GameFeaturesSubsystem.h"
#include "GameFeatureTypes.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureOptionalContentInstaller)

namespace GameFeatureOptionalContentInstaller
{
	static const ELogVerbosity::Type InstallBundleManagerVerbosityOverride = ELogVerbosity::Verbose;

	static const FStringView ErrorNamespace = TEXTVIEW("GameFeaturePlugin.OptionalDownload.");

	static TAutoConsoleVariable<bool> CVarEnableOptionalContentInstaller(TEXT("GameFeatureOptionalContentInstaller.Enable"), 
		true,
		TEXT("Enable optional content installer"));

	static bool bSkipRequestsForUnknownBundles = true;
	static FAutoConsoleVariableRef CVarSkipRequestsForUnknownBundles(TEXT("GameFeatureOptionalContentInstaller.SkipUnkownBundles"),
		bSkipRequestsForUnknownBundles,
		TEXT("When true optional bundles that do not exist are skipped and the request continues."));
}

const FName UGameFeatureOptionalContentInstaller::GFOContentRequestName = TEXT("GFOContentRequest");

TMulticastDelegate<void(const FString& PluginName, const UE::GameFeatures::FResult&)> UGameFeatureOptionalContentInstaller::OnOptionalContentInstalled;
TMulticastDelegate<void()> UGameFeatureOptionalContentInstaller::OnOptionalContentInstallStarted;
TMulticastDelegate<void(const bool bInstallSuccessful)> UGameFeatureOptionalContentInstaller::OnOptionalContentInstallFinished;

void UGameFeatureOptionalContentInstaller::BeginDestroy()
{
	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(CVarSinkHandle);
	Super::BeginDestroy();
}

void UGameFeatureOptionalContentInstaller::Init(
	TUniqueFunction<TArray<FName>(FString)> InGetOptionalBundlePredicate,
	TUniqueFunction<TArray<FName>(FString)> InGetOptionalKeepBundlePredicate)
{
	GetOptionalBundlePredicate = MoveTemp(InGetOptionalBundlePredicate);
	GetOptionalKeepBundlePredicate = MoveTemp(InGetOptionalKeepBundlePredicate);
	BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();

	// Create the cvar sink
	CVarSinkHandle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(
		FConsoleCommandDelegate::CreateUObject(this, &UGameFeatureOptionalContentInstaller::OnCVarsChanged));
	bEnabledCVar = GameFeatureOptionalContentInstaller::CVarEnableOptionalContentInstaller.GetValueOnGameThread();
}

void UGameFeatureOptionalContentInstaller::Enable(bool bInEnable)
{
	bool bOldEnabled = IsEnabled();
	bEnabled = bInEnable;
	bEnabledCVar = GameFeatureOptionalContentInstaller::CVarEnableOptionalContentInstaller.GetValueOnGameThread();
	bool bNewEnabled = IsEnabled();

	if (bOldEnabled != bNewEnabled)
	{
		if (bNewEnabled)
		{
			OnEnabled();
		}
		else
		{
			OnDisabled();
		}
	}
}

void UGameFeatureOptionalContentInstaller::UninstallContent()
{
	for (const FString& GFP : RelevantGFPs)
	{
		UE_LOGF(LogGameFeatures, Log, "Uninstalling Optional bundles for %ls", *GFP);
		ReleaseContent(GFP, EInstallBundleReleaseRequestFlags::RemoveFilesIfPossible);
	}
	RelevantGFPs.Empty();
}

void UGameFeatureOptionalContentInstaller::EnableCellularDownloading(bool bEnable)
{
	if (bAllowCellDownload == bEnable)
	{
		return;
	}

	bAllowCellDownload = bEnable;
	BundleManager->SetCellularPreference(bAllowCellDownload ? 1 : 0);

	// Update flags on active requests
	for ( TPair<FString, FGFPInstall>& Pair : ActiveGFPInstalls)
	{
		BundleManager->UpdateContentRequestFlags(Pair.Value.BundlesEnqueued,
			bEnable ? EInstallBundleRequestFlags::None : EInstallBundleRequestFlags::CheckForCellularDataUsage,
			bEnable ? EInstallBundleRequestFlags::CheckForCellularDataUsage : EInstallBundleRequestFlags::None);
	}
}

bool UGameFeatureOptionalContentInstaller::HasOngoingInstalls() const
{
	return ActiveGFPInstalls.Num() > 0;
}

float UGameFeatureOptionalContentInstaller::GetAllInstallsProgress()
{
	if (TotalProgressTracker.IsSet())
	{
		TotalProgressTracker->ForceTick();
		return TotalProgressTracker->GetCurrentCombinedProgress().ProgressPercent;
	}

	if (ActiveGFPInstalls.Num() > 0 && RelevantGFPs.Num() > 0)
	{
		// Start the tracker for next calls to this function
		StartTotalProgressTracker();
	}

	// Return 1 if some optional bundles are installed, 0 if none are installed or active installs are present 
	return ActiveGFPInstalls.Num() == 0 && RelevantGFPs.Num() > 0 ? 1.f : 0.f;
}

bool UGameFeatureOptionalContentInstaller::UpdateContent(const FString& PluginName, bool bIsPredownload)
{
	TArray<FName> Bundles = GetOptionalBundlePredicate(PluginName);

	bool bIsAvailable = false;
	if (!Bundles.IsEmpty())
	{
		TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> MaybeInstallState = BundleManager->GetInstallStateSynchronous(Bundles, true);
		if (MaybeInstallState.HasValue())
		{
			const FInstallBundleCombinedInstallState& InstallState = MaybeInstallState.GetValue();
			bIsAvailable = ensureMsgf(InstallState.IndividualBundleStates.Num() <= Bundles.Num(),

				TEXT("UGameFeatureOptionalContentInstaller does not support dependencies tracking. Check optional install bundle dependencies for plugin %s."),
				*PluginName);
			if (GameFeatureOptionalContentInstaller::bSkipRequestsForUnknownBundles && bIsAvailable)
			{
				Bundles.Reset();
				InstallState.IndividualBundleStates.GetKeys(Bundles);
			}
			else
			{
				bIsAvailable &= Algo::AllOf(Bundles, [&InstallState](FName BundleName) { return InstallState.IndividualBundleStates.Contains(BundleName); });
			}
		}
	}

	if (!bIsAvailable)
	{
		return false;
	}

	for (const FName& Bundle : Bundles)
	{
		UE_LOGF(LogGameFeatures, Verbose, "Requesting update for %ls", *Bundle.ToString());
	}
	EInstallBundleRequestFlags InstallFlags = EInstallBundleRequestFlags::AsyncMount | EInstallBundleRequestFlags::ExplicitUpdateList;
	if (bIsPredownload)
	{
		InstallFlags |= EInstallBundleRequestFlags::SkipMount;
	}
	if (!bAllowCellDownload)
	{
		InstallFlags |= EInstallBundleRequestFlags::CheckForCellularDataUsage;
	}

	TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> MaybeRequest = BundleManager->RequestUpdateContent(
		Bundles, 
		InstallFlags, 
		GameFeatureOptionalContentInstaller::InstallBundleManagerVerbosityOverride);

	if (MaybeRequest.HasError())
	{
		UE_LOGFMT(LogGameFeatures, Error, "Failed to request optional content for GFP {GFP}, Error: {Error}", 
			("GFP", PluginName),
			("Error", LexToString(MaybeRequest.GetError())));

		UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
			GameFeatureOptionalContentInstaller::ErrorNamespace.Len(), GameFeatureOptionalContentInstaller::ErrorNamespace.GetData(),
			LexToString(MaybeRequest.GetError())));
		OnOptionalContentInstalled.Broadcast(PluginName, ErrorResult);

		return false;
	}

	FInstallBundleRequestInfo& Request =  MaybeRequest.GetValue();
	if (!Request.BundlesEnqueued.IsEmpty())
	{
		const bool bIsOptionalContentInstallStart = ActiveGFPInstalls.Num() == 0;
	
		FGFPInstall& Pending = ActiveGFPInstalls.FindOrAdd(PluginName);
		
		if (bIsOptionalContentInstallStart)
		{
			// We call the delegate after adding the entry to 'ActiveGFPInstalls'. 
			// If we called it before then the code triggered by this delegate could request information from the current script
			// and since 'ActiveGFPInstalls' would be empty it would behave as if no installs were happening.
			OnOptionalContentInstallStarted.Broadcast();
		}

		if (!Pending.CallbackHandle.IsValid())
		{
			Pending.CallbackHandle = IInstallBundleManager::InstallBundleCompleteDelegate.AddUObject(this,
				&UGameFeatureOptionalContentInstaller::OnContentInstalled, PluginName);
		}

		// This should overwrite any previous pending request info
		Pending.BundlesEnqueued = MoveTemp(Request.BundlesEnqueued);
		Pending.bIsPredownload = bIsPredownload;
	}
	const FName PluginFName(PluginName);
	for (const FName& Bundle : Bundles)
	{
		TSet<FName>& PluginsUsingBundle = BundleToPlugins.FindOrAdd(Bundle);
		PluginsUsingBundle.Add(PluginFName);
	}

	return true;
}

void UGameFeatureOptionalContentInstaller::OnContentInstalled(FInstallBundleRequestResultInfo InResult, FString PluginName)
{
	FGFPInstall* MaybeInstall = ActiveGFPInstalls.Find(PluginName);
	if (!MaybeInstall)
	{
		return;
	}

	FGFPInstall& GFPInstall = *MaybeInstall;
	if (!GFPInstall.BundlesEnqueued.Contains(InResult.BundleName))
	{
		return;
	}

	GFPInstall.BundlesEnqueued.Remove(InResult.BundleName);

	UE_LOGF(LogGameFeatures, Verbose, "Finished install for %ls", *InResult.BundleName.ToString());
	if (InResult.Result != EInstallBundleResult::OK)
	{
		if (InResult.OptionalErrorCode.IsEmpty())
		{
			UE_LOGFMT(LogGameFeatures, Error, "Failed to install optional bundle {Bundle} for GFP {GFP}, Error: {Error}",
				("Bundle", InResult.BundleName),
				("GFP", PluginName),
				("Error", LexToString(InResult.Result)));
		}
		else
		{
			UE_LOGFMT(LogGameFeatures, Error, "Failed to install optional bundle {Bundle} for GFP {GFP}, Error: {Error}",
				("Bundle", InResult.BundleName),
				("GFP", PluginName),
				("Error", InResult.OptionalErrorCode));
		}

		//Use OptionalErrorCode and/or OptionalErrorText if available
		const FString ErrorCodeEnding = (InResult.OptionalErrorCode.IsEmpty()) ? LexToString(InResult.Result) : InResult.OptionalErrorCode;
		FText ErrorText = InResult.OptionalErrorCode.IsEmpty() ? UE::GameFeatures::CommonErrorCodes::GetErrorTextForBundleResult(InResult.Result) : InResult.OptionalErrorText;
		UE::GameFeatures::FResult ErrorResult = UE::GameFeatures::FResult(
			MakeError(FString::Printf(TEXT("%.*s%s"), GameFeatureOptionalContentInstaller::ErrorNamespace.Len(), GameFeatureOptionalContentInstaller::ErrorNamespace.GetData(), *ErrorCodeEnding)),
			MoveTemp(ErrorText)
		);
		OnOptionalContentInstalled.Broadcast(PluginName, ErrorResult);

		// Cancel any remaining downloads
		BundleManager->CancelUpdateContent(GFPInstall.BundlesEnqueued);
	}

	if (GFPInstall.BundlesEnqueued.IsEmpty())
	{
		if (GFPInstall.bIsPredownload)
		{
			// Predownload shouldn't pin any cached bundles so release them now

			// Delay call to ReleaseBundlesIfPossible. We don't want to release them from within the complete callback.
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, 
			[this, PluginName, bInstalled = InResult.bContentWasInstalled](float)
			{
				// A machine is active, don't release
				if (!RelevantGFPs.Contains(PluginName))
				{
					ReleaseContent(PluginName);
				}

				if (bInstalled)
				{
					OnOptionalContentInstalled.Broadcast(PluginName, MakeValue());
				}

				return false;
			}));
		}
		else if (InResult.bContentWasInstalled)
		{
			OnOptionalContentInstalled.Broadcast(PluginName, MakeValue());
		}

		// book keeping
		IInstallBundleManager::InstallBundleCompleteDelegate.Remove(GFPInstall.CallbackHandle);
		ActiveGFPInstalls.Remove(PluginName);

		if (ActiveGFPInstalls.Num() == 0)
		{
			const bool bInstallSuccessful = RelevantGFPs.Num() > 0;
			OnOptionalContentInstallFinished.Broadcast(bInstallSuccessful);
			TotalProgressTracker.Reset();
		}
	}
}

void UGameFeatureOptionalContentInstaller::ReleaseContent(const FString& PluginName, EInstallBundleReleaseRequestFlags Flags)
{
	TArray<FName> Bundles = GetOptionalBundlePredicate(PluginName);
	if (Bundles.IsEmpty())
	{
		return;
	}
	TArray<FName> KeepBundles = GetOptionalKeepBundlePredicate(PluginName);

	TArray<FName> BundlesWithNoInUsePlugin;
	const FName PluginFName(PluginName);
	for (const FName& Bundle : Bundles)
	{
		TSet<FName>& PluginsUsingBundle = BundleToPlugins.FindOrAdd(Bundle);
		PluginsUsingBundle.Remove(PluginFName);
		if (PluginsUsingBundle.Num() == 0)
		{
			BundlesWithNoInUsePlugin.Add(Bundle);
			BundleToPlugins.Remove(Bundle);
		}
	}
	if (BundlesWithNoInUsePlugin.IsEmpty())
	{
		return;
	}

	for (const FName& Bundle : BundlesWithNoInUsePlugin)
	{
		UE_LOGF(LogGameFeatures, Log, "Requesting Release for %ls. No plugin is actively using it.", *Bundle.ToString());
	}
	Flags |= EInstallBundleReleaseRequestFlags::ExplicitRemoveList;

	BundleManager->RequestReleaseContent(
		BundlesWithNoInUsePlugin,
		Flags,
		KeepBundles, 
		GameFeatureOptionalContentInstaller::InstallBundleManagerVerbosityOverride);
}

void UGameFeatureOptionalContentInstaller::OnEnabled()
{
	ensure(RelevantGFPs.IsEmpty());
	RelevantGFPs.Empty();

	// Capture a list of all existing bundles that are currently mounted outside of our system. We do not want to unmount something another system mounted.
	{
		BundleToPlugins.Reset();
		const FName CoreInUseBundle(TEXT("#@#Core_Already_Mounted#@#"));

		TArray<FName> AllBundles;
		EInstallBundleResult InstallStateResult = BundleManager->GetAllMountedBundles(AllBundles);
		if (InstallStateResult == EInstallBundleResult::OK)
		{
			for (const FName& MountedBundle : AllBundles)
			{
				TSet<FName>& PluginsUsingBundle = BundleToPlugins.FindOrAdd(MountedBundle);
				PluginsUsingBundle.Add(CoreInUseBundle);
			}
		}
		else
		{
			UE_LOGF(LogGameFeatures, Error, "Failed to get all mounted bundles. Optional content installer enabled before budnle manager was initialized.");
		}
	}

	UGameFeaturesSubsystem::Get().ForEachGameFeature([this](FGameFeatureInfo&& Info) -> void
	{
		if (Info.CurrentState >= EGameFeaturePluginState::Downloading)
		{
			if (UpdateContent(Info.Name, false))
			{
				RelevantGFPs.Add(Info.Name);
			}
		}
	});
}

void UGameFeatureOptionalContentInstaller::OnDisabled()
{
	for (const FString& GFP : RelevantGFPs)
	{
		ReleaseContent(GFP);
	}

	RelevantGFPs.Empty();
	TotalProgressTracker.Reset();
	BundleToPlugins.Reset();
}

bool UGameFeatureOptionalContentInstaller::IsEnabled() const
{
	return bEnabled && bEnabledCVar;
}

void UGameFeatureOptionalContentInstaller::OnCVarsChanged()
{
	Enable(bEnabled); // Check if CVar changed IsEnabled() and if so, call callbacks
}

void UGameFeatureOptionalContentInstaller::StartTotalProgressTracker()
{
	TotalProgressTracker.Reset();
	BundleManager->CancelAllGetContentStateRequestsForTag(GFOContentRequestName);
	
	TArray<FName> AllActiveBundleInstalls;
	for (const TTuple<FString, FGFPInstall>& ActiveInstall : ActiveGFPInstalls)
	{
		for (const FName& BundleEnqueued : ActiveInstall.Value.BundlesEnqueued)
		{
			AllActiveBundleInstalls.AddUnique(BundleEnqueued);
		}
	}

	if (AllActiveBundleInstalls.Num() > 0 && RelevantGFPs.Num() > 0)
	{
		// Start a new progress tracker for the currently active bundle installs. Auto tick is disabled.
		TotalProgressTracker.Emplace(false);
		
		TWeakObjectPtr This_WeakPtr = this;	
		BundleManager->GetContentState(AllActiveBundleInstalls, EInstallBundleGetContentStateFlags::None, false,
			FInstallBundleGetContentStateDelegate::CreateLambda([This_WeakPtr](FInstallBundleCombinedContentState BundleContentState)
			{
				if (This_WeakPtr.IsValid())
				{
					const TStrongObjectPtr<UGameFeatureOptionalContentInstaller> This_StrongPtr = This_WeakPtr.Pin();
					if (This_StrongPtr->TotalProgressTracker.IsSet())
					{
						TArray<FName> RequiredBundlesForTracking;
						for (const TPair<FName, FInstallBundleContentState>& BundleState : BundleContentState.IndividualBundleStates)
						{
							RequiredBundlesForTracking.Add(BundleState.Key);
						}
						This_StrongPtr->TotalProgressTracker->SetBundlesToTrackFromContentState(BundleContentState, MoveTemp(RequiredBundlesForTracking));
					}
				}
			}), GFOContentRequestName);
	}
}

void UGameFeatureOptionalContentInstaller::OnGameFeaturePredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	if (!IsEnabled())
	{
		return;
	}

	UpdateContent(PluginName, true);
	// Predownloads are not 'relevant', they don't have an active state machine
}

void UGameFeatureOptionalContentInstaller::OnGameFeatureDownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	if (!IsEnabled())
	{
		return;
	}

	if (UpdateContent(PluginName, false))
	{
		RelevantGFPs.Add(PluginName);
	}
}

void UGameFeatureOptionalContentInstaller::OnGameFeatureRegistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FString& PluginURL)
{
    // Used for already downloaded cached plugins that do not download at startup but register.
    if (!IsEnabled() || RelevantGFPs.Contains(PluginName))
    {
        return;
    }

    if (UpdateContent(PluginName, false))
    {
        RelevantGFPs.Add(PluginName);
    }
}

void UGameFeatureOptionalContentInstaller::OnGameFeatureReleasing(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	if (!IsEnabled())
	{
		return;
	}

	ReleaseContent(PluginName);

	RelevantGFPs.Remove(PluginName);
}