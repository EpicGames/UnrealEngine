// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturePluginOperationResult.h"
#include "GameFeaturesSubsystem.h" // needed for log category
#include "InstallBundleTypes.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogMacros.h"

namespace UE::GameFeatures
{
	FResult::FResult(FErrorCodeType ErrorCodeIn)
		: ErrorCode(MoveTemp(ErrorCodeIn))
		, OptionalErrorText()
	{
	}

	FResult::FResult(FErrorCodeType ErrorCodeIn, FText ErrorTextIn)
		: ErrorCode(MoveTemp(ErrorCodeIn))
		, OptionalErrorText(MoveTemp(ErrorTextIn))
	{
	}

	FString ToString(const FResult& Result)
	{
		TStringBuilder<512> Out;
		if (Result.HasValue())
		{
			Out << TEXT("Success");
		}
		else
		{
			Out << TEXT("ErrorCode=") << Result.GetError();
			if (!Result.OptionalErrorText.IsEmpty())
			{
				Out << TEXT(", ErrorText=") << Result.OptionalErrorText.ToString();
			}
		}
		return Out.ToString();
	}

	namespace CommonErrorCodes
	{
		const FText Generic_FatalError = NSLOCTEXT("GameFeatures", "ErrorCodes.GenericFatalError", "A fatal error has occurred installing the game feature. An update to the application may be needed. Please check for updates and restart the application.");
		const FText Generic_ConnectionError = NSLOCTEXT("GameFeatures", "ErrorCodes.ConnectionGenericError", "An internet connection error has occurred. Please try again later.");
		const FText Generic_MountError = NSLOCTEXT("GameFeatures", "ErrorCodes.MountGenericError", "An error has occurred loading data for this game feature. Please try again later.");

		const FText BundleResult_NeedsUpdate = NSLOCTEXT("GameFeatures", "ErrorCodes.BundleResult.NeedsUpdate", "An application update is required to install this game feature. Please restart the application after downloading any required updates.");
		const FText BundleResult_NeedsCacheSpace = NSLOCTEXT("GameFeatures", "ErrorCodes.BundleResult.NeedsCacheSpace", "Unable to allocate enough space in the cache to install this game feature.");
		const FText BundleResult_NeedsDiskSpace = NSLOCTEXT("GameFeatures", "ErrorCodes.BundleResult.NeedsDiskSpace", "You do not have enough disk space to install this game feature. Please try again after clearing up disk space.");
		const FText BundleResult_DownloadCancelled = NSLOCTEXT("GameFeatures", "ErrorCodes.BundleResult.DownloadCancelled", "This game feature download was canceled.");

		const FText ReleaseResult_Generic = NSLOCTEXT("GameFeatures", "ErrorCodes.ReleaseResult.Generic", "There was an error uninstalling the content for this game feature. Please restart the application and try again.");
		const FText ReleaseResult_Cancelled = NSLOCTEXT("GameFeatures", "ErrorCodes.ReleaseResult.Cancelled", "This game feature uninstall was canceled.");

		const FText& GetErrorTextForBundleResult(EInstallBundleResult ErrorResult)
		{
			switch (ErrorResult)
			{
				//These errors mean an app update is available that we either don't have or failed to get.
				case EInstallBundleResult::FailedPrereqRequiresLatestClient:
				case EInstallBundleResult::FailedPrereqRequiresLatestContent:
				{
					return BundleResult_NeedsUpdate;
				}


				//These are generally unrecoverable and mean something is seriously wrong with the data for this build
				case EInstallBundleResult::InitializationError:
				case EInstallBundleResult::MetadataError:
				{
					return Generic_FatalError;
				}

				//Not enough space in cache to install the files
				case EInstallBundleResult::FailedCacheReserve:
				{
					return BundleResult_NeedsCacheSpace;
				}

				//All of these are indicative of not having enough disk space to install the required files
				case EInstallBundleResult::InstallerOutOfDiskSpaceError:
				case EInstallBundleResult::ManifestArchiveError:
				{
					return BundleResult_NeedsDiskSpace;
				}

				case EInstallBundleResult::UserCancelledError:
				{
					return BundleResult_DownloadCancelled;
				}

				//Intentionally just show generic error for all these cases
				case EInstallBundleResult::InstallError:
				case EInstallBundleResult::ConnectivityError:
				case EInstallBundleResult::InitializationPending:
				{
					return Generic_ConnectionError;
				}

				//Show generic error for anything missing but log an error
				default:
				{
					UE_LOGF(LogGameFeatures, Error, "Missing error text for EInstallBundleResult %ls", LexToString(ErrorResult));
					return Generic_ConnectionError;
				}
			}
		}

		const FText& GetErrorTextForReleaseResult(EInstallBundleReleaseResult ErrorResult)
		{
			switch (ErrorResult)
			{
				case EInstallBundleReleaseResult::UserCancelledError:
				{
					return ReleaseResult_Cancelled;
				}
			
				case EInstallBundleReleaseResult::ManifestArchiveError:
				case EInstallBundleReleaseResult::MetadataError:
				{
					return ReleaseResult_Generic;
				}

				default:
				{
					//Show generic error for anything missing but log an error
					UE_LOGF(LogGameFeatures, Error, "Missing error text for EInstallBundleReleaseResult %ls", LexToString(ErrorResult));
					return ReleaseResult_Generic;
				}
			}
		}
		const FText& GetGenericFatalError()
		{
			return Generic_FatalError;
		}
		const FText& GetGenericConnectionError()
		{
			return Generic_ConnectionError;
		}
		const FText& GetGenericMountError()
		{
			return Generic_MountError;
		}
		const FText& GetGenericReleaseResult()
		{
			return ReleaseResult_Generic;
		}
	}

}	// namespace UE::GameFeatures