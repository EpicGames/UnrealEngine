// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturePluginStateMachine.h"

#include "Algo/RandomShuffle.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Engine/StreamableManager.h"
#include "GameFeatureData.h"
#include "GameFeaturesSubsystem.h"
#include "GameplayTagsManager.h"
#include "Engine/AssetManager.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "IPlatformFilePak.h"
#include "InstallBundleUtils.h"
#include "BundlePrereqCombinedStatusHelper.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "Internationalization/StringTable.h"
#include "IO/IoStoreOnDemand.h"
#include "Logging/LogVerbosity.h"
#include "Logging/StructuredLog.h"
#include "Materials/MaterialInterface.h"
#include "Misc/AsciiSet.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/EnumRange.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/WildcardString.h"
#include "Algo/Accumulate.h"
#include "Algo/AllOf.h"
#include "Algo/Find.h"
#include "Misc/TVariantMeta.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RenderDeferredCleanup.h"
#include "Serialization/MemoryReader.h"
#include "String/ParseTokens.h"
#include "String/LexFromString.h"
#include "Tasks/Pipe.h"
#include "GameFeaturesProjectPolicies.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectRename.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectIterator.h"
#include "UObject/NoExportTypes.h"
#include "Misc/PathViews.h"
#include "Containers/Queue.h"
#include "ShaderCodeLibrary.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Trace/Trace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeaturePluginStateMachine)

#define LOCTEXT_NAMESPACE "GameFeatureDataStateMachine"

#if WITH_EDITOR
#include "PluginUtils.h"
#include "Misc/App.h"
#endif //if WITH_EDITOR

LLM_DEFINE_TAG(GFP, TEXT("GameFeatures"), TEXT("UObject"));

#define GFP_STATE_LOG(VERBOSITY, MSG, ...) UE_LOGFMT(LogGameFeatures, VERBOSITY, \
	"{Plugin}[{State}] " MSG, StateProperties.PluginIdentifier.GetPluginName(), State __VA_OPT__(,) __VA_ARGS__);

namespace UE::GameFeatures
{
	static const FString StateMachineErrorNamespace(TEXT("GameFeaturePlugin.StateMachine."));

	static UE::GameFeatures::FResult CanceledResult = MakeError(StateMachineErrorNamespace + TEXT("Canceled"));

	bool bUseNewExecutor = true;
	static FAutoConsoleVariableRef CVarUseNewExecutor(TEXT("GameFeaturePlugin.UseNewExecutor"),
		bUseNewExecutor,
		TEXT("Should control of state machine updates be handled by the GameFeaturesSubsystem (new behavior) to all more batch & parallel processing."),
	ECVF_ReadOnly | ECVF_SaveForNextBoot);

	// Options for how to defer execution of state updates and how to order those updates
	struct FExecutionOptions
	{
		// Group by current state before executing
		bool bGrouped = true;
		// Execute the update requests (potentially within each group) in a random order
		bool bShuffle = false;
		// Reverse the order of update requests so that deferred request are executed in LIFO order
		bool bReverse = false;
	} ExecutionOptions;

	static FAutoConsoleVariableRef CVarExecutionGrouped(TEXT("GameFeaturePlugin.Execution.Grouped"),
		ExecutionOptions.bGrouped,
		TEXT("Group deferred updates by current state before executing. This is the intended future default path."),
		ECVF_Default);
	static FAutoConsoleVariableRef CVarExecutionShuffle(TEXT("GameFeaturePlugin.Execution.Shuffle"),
		ExecutionOptions.bShuffle,
		TEXT("Shuffle deferred updates before execution (within a group if .Grouped is set). For testing implicit dependencies."),
		ECVF_Default);
	static FAutoConsoleVariableRef CVarExecutionReverse(TEXT("GameFeaturePlugin.Execution.Reverse"),
		ExecutionOptions.bReverse,
		TEXT("Reverse deferred updates before execution to execute in LIFO order. For testing deferred dependencies."),
		ECVF_Default);

	bool bLoadBuiltin_DeferExecution = true;
	static FAutoConsoleVariableRef CVarLoadBuiltin_DeferExecution (TEXT("GameFeaturePlugin.LoadBuiltins.DeferExecution"),
		bLoadBuiltin_DeferExecution ,
		TEXT("When GameFeaturePlugin.UseNewExecutor is true, should execution of state machine updates in LoadBuiltInGameFeaturePlugins be deferred until all state transitions have been queued?"),
	ECVF_ReadOnly);

	static int32 ShouldLogMountedFiles = 0;
	static FAutoConsoleVariableRef CVarShouldLogMountedFiles(TEXT("GameFeaturePlugin.ShouldLogMountedFiles"),
		ShouldLogMountedFiles,
		TEXT("Should the newly mounted files be logged."));

	static FString GVerifyPluginSkipList;
	static FAutoConsoleVariableRef CVarVerifyPluginSkipList(TEXT("PluginManager.VerifyUnload.SkipList"),
		GVerifyPluginSkipList,
		TEXT("Comma-separated list of names of plugins for which to skip verification."),
		ECVF_Default);

	static bool bDeferLocalizationDataLoad = true;
	static FAutoConsoleVariableRef CVarDeferLocalizationLoading(TEXT("GameFeaturePlugin.DeferLocalizationDataLoad"),
		bDeferLocalizationDataLoad,
		TEXT("True if we should defer loading the localization data until 'loading' (new behavior), or false to load it on 'mounting' (old behavior)."));

	static TAutoConsoleVariable<bool> CVarAsyncLoad(TEXT("GameFeaturePlugin.AsyncLoad"),
		true,
		TEXT("Enable to use async loading as well async downloading and registering"));

	static TAutoConsoleVariable<bool> CVarForceSyncRegisterStartupPlugins(TEXT("GameFeaturePlugin.ForceSyncRegisterStartupPlugins"),
		true,
		TEXT("If true, all plugins loaded during startup will be synchronously registered to ensure things are initialized in time, this only applies if AsyncLoad is enabled"));

	static TAutoConsoleVariable<bool> CVarForceSyncLoadShaderLibrary(TEXT("GameFeaturePlugin.ForceSyncLoadShaderLibrary"),
		true,
		TEXT("Enable to force shaderlibs to be opened on the game thread"));

	static TAutoConsoleVariable<bool> CVarForceSyncAssetRegistryAppend(TEXT("GameFeaturePlugin.ForceSyncAssetRegistryAppend"),
		false,
		TEXT("Enable to force calls to IAssetRegistry::AppendState to happen on the game thread"));

	static TAutoConsoleVariable<bool> CVarWaitForDependencyDeactivation(TEXT("GameFeaturePlugin.WaitForDependencyDeactivation"),
		false,
		TEXT("Enable to make block deactivation until all dependencies are deactivated. Warning - this can lead to failure to unload"));

	static TAutoConsoleVariable<bool> CVarEnableAssetStreaming(TEXT("GameFeaturePlugin.EnableAssetStreaming"),
		true,
		TEXT("Enable experimental asset streaming"));

	static TAutoConsoleVariable<bool> CVarEnableBatchProcessing(TEXT("GameFeaturePlugin.EnableBatchProcessing"),
		false,
		TEXT("Enable batch processing when processing multiple plugins and specified in protocol options."));

	/*extern*/ TAutoConsoleVariable<bool> CVarAllowMissingOnDemandDependencies(TEXT("GameFeaturePlugin.AllowMissingOnDemandDependencies"),
		false,
		TEXT("Don't fail on missing on demand (IAD) asset dependencies"));


	static TAutoConsoleVariable<bool> CVarForceGFPIADDownload(TEXT("GameFeaturePlugin.ForceGFPIADDownload"),
		false,
		TEXT("Enable batch processing when processing multiple plugins and specified in protocol options."));

#if !UE_BUILD_SHIPPING
	FString DebugGFPNameMatch = "";
	static FAutoConsoleVariableRef CVarDebugGFPNameMatch(TEXT("GameFeaturePlugin.DebugGFPName"), DebugGFPNameMatch, TEXT("Set cvar and add breakpoint to \"ShouldDebugGFPNameMatch()\" To debug"));

	bool ShouldDebugGFP(const FString& GFPName)
	{
		static FString CachedDebugName;
		static TUniquePtr<FRegexPattern> Pattern;
		if (CachedDebugName != DebugGFPNameMatch)
		{
			CachedDebugName = DebugGFPNameMatch;
			if (DebugGFPNameMatch.IsEmpty())
			{
				Pattern = nullptr;
			}
			else
			{
				Pattern.Reset(new FRegexPattern(DebugGFPNameMatch));
			}
		}

		if (Pattern.IsValid())
		{
			FRegexMatcher Matcher(*Pattern, GFPName);
			if (Matcher.FindNext())
			{
				return true;
			}
		}
		return false;
	}
#else
	bool ShouldDebugGFP(const FString& GFPName)
	{
		return false;
	}
#endif
	bool ShouldDeferLocalizationDataLoad()
	{
		// Note: We don't defer localization data loading in the editor, as the editor only needs to mount plugins to use them
		return !GIsEditor && bDeferLocalizationDataLoad;
	}

	void MountLocalizationData(UGameFeaturePluginStateMachine* CurrentMachine, FGameFeaturePluginStateMachineProperties& StateProperties)
	{
		check(IsInGameThread());
		check(CurrentMachine && &CurrentMachine->StateProperties == &StateProperties);

		StateProperties.bIsLoadingLocalizationData = true;
		IPluginManager::Get().MountExplicitlyLoadedPluginLocalizationData(StateProperties.PluginName, [WeakMachine = TWeakObjectPtr<UGameFeaturePluginStateMachine>(CurrentMachine), &StateProperties, bAllowAsyncLoading = StateProperties.AllowAsyncLoading()](bool bLoadedLocalization, const FString& PluginName)
		{
			if (UGameFeaturePluginStateMachine* StateMachine = WeakMachine.Get();
				StateMachine && ensureAlways(&StateMachine->StateProperties == &StateProperties))
			{
				if (IsInGameThread())
				{
					StateProperties.bIsLoadingLocalizationData = false;
				}
				else if (bAllowAsyncLoading)
				{
					ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakMachine, &StateProperties]()
					{
						if (UGameFeaturePluginStateMachine* StateMachine = WeakMachine.Get();
							StateMachine && ensureAlways(&StateMachine->StateProperties == &StateProperties))
						{
							StateProperties.bIsLoadingLocalizationData = false;
							StateProperties.UpdateStateMachineImmediate();
						}
					});
				}
			}
		});
	}

	bool ShouldSkipVerify(const FString& PluginName)
	{
		static const FAsciiSet Wildcards("*?");
		bool bSkip = false;
		UE::String::ParseTokens(MakeStringView(GVerifyPluginSkipList), TEXTVIEW(","), [&PluginName, &bSkip](FStringView Item) {
				if (bSkip) { return; }
				if (Item.Equals(PluginName, ESearchCase::IgnoreCase))
				{
					bSkip = true;
				}
				else if (FAsciiSet::HasAny(Item, Wildcards))
				{
					FString Pattern = FString(Item); // Need to copy to null terminate
					if (FWildcardString::IsMatchSubstring(*Pattern, *PluginName, *PluginName + PluginName.Len(), ESearchCase::IgnoreCase))
					{
						bSkip = true;
					}
				}
			});
		return bSkip;
	}

	// Return a higher number for packages which it's more important to include in leak reporting, when the number of leaks we want to report is limited. 
	int32 GetPackageLeakReportingPriority(UPackage* Package)
	{	
		int32 Priority = 0;
		ForEachObjectWithPackage(Package, [&Priority](UObject* Object)
		{
			if (UWorld* World = Cast<UWorld>(Object))
			{
				Priority = 100;
				return true;
			}
			else if (Cast<UMaterialInterface>(Object))
			{
				Priority = FMath::Max(Priority, 50);
				// keep iterating in case we find a world 
			}
			return true;
		}, EGetObjectsFlags::None);
		return Priority;
	}

	bool bRealtimeMode = false;
	constexpr double RealtimeMaxFrameTime = 0.033; // 30fps
	constexpr double RealtimeAllottedTime = RealtimeMaxFrameTime / 2;

	class FRealtimeMode : public TSharedFromThis<FRealtimeMode>
	{
	public:
		~FRealtimeMode()
		{
			if (TickHandle.IsValid())
			{
				FTSTicker::RemoveTicker(MoveTemp(TickHandle));
			}

			int64 UpdateRequest = 0;
			UGameFeaturesSubsystem& Subsystem = UGameFeaturesSubsystem::Get();
			while (UpdateRequests.Dequeue(UpdateRequest))
			{
				Subsystem.TryUpdateStateMachineImmediate(UpdateRequest);
			}
		}

		void AddUpdateRequest(FGameFeaturePluginStateMachineProperties StateProperties)
		{
			UpdateRequests.Enqueue(StateProperties.Id);
			EnableTick();
		}

	private:
		void EnableTick()
		{
			if (!TickHandle.IsValid())
			{
				TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FRealtimeMode::Tick));
			}
		}

		bool Tick(float DeltaTime)
		{
			// Self-reference so we don't get destroyed during Tick
			TSharedRef<FRealtimeMode> SelfRef = AsShared();

			{
				const double StartTime = FPlatformTime::Seconds();

				UGameFeaturesSubsystem& Subsystem = UGameFeaturesSubsystem::Get();
				int64 UpdateRequest = 0;
				while (UpdateRequests.Dequeue(UpdateRequest))
				{
					Subsystem.TryUpdateStateMachineImmediate(UpdateRequest);

					const double ElapsedTime = FPlatformTime::Seconds() - StartTime;
					if ((ElapsedTime > RealtimeAllottedTime) || ((DeltaTime + ElapsedTime) > RealtimeMaxFrameTime))
					{
						break;
					}
				}
			}

			if (UpdateRequests.IsEmpty())
			{
				TickHandle.Reset();
				return false;
			}
			else
			{
				return true;
			}
		}

	private:
		TQueue<int64> UpdateRequests;
		FTSTicker::FDelegateHandle TickHandle;
	};

	static TSharedPtr<FRealtimeMode> RealtimeMode;

	static FAutoConsoleVariableRef CVarRealtimeMode(TEXT("GameFeaturePlugin.RealtimeMode"),
		bRealtimeMode,
		TEXT("Sets whether GFS realtime mode is enabled; which distributes plugin state updates over several frames"),
		FConsoleVariableDelegate::CreateLambda(
			[](IConsoleVariable* Var)
			{
				if (bUseNewExecutor)
				{
					// new executor reads bRealtimeMode directly and enforces timings
					return;
				}
				if (Var->GetBool())
				{
					if (!RealtimeMode)
					{
						RealtimeMode = MakeShared<FRealtimeMode>();
					}
				}
				else
				{
					TSharedPtr<FRealtimeMode> Rm = MoveTemp(RealtimeMode);
					Rm.Reset();
				}
			}),
			ECVF_ReadOnly);

#if WITH_EDITOR
	TMap<FString, int64> PluginsToUnloadAssets;
	FTSTicker::FDelegateHandle UnloadPluginAssetsHandle;

	bool TickUnloadPluginAssets(float /*DeltaTime*/)
	{
		UnloadPluginAssetsHandle.Reset();
	
		TArray<FString> PluginNames;
		TArray<int64> UpdateStateMachineIds;
		{
			PluginNames.Reserve(PluginsToUnloadAssets.Num());
			UpdateStateMachineIds.Reserve(PluginsToUnloadAssets.Num());
			for (TPair<FString, int64>& PluginsToUnloadAsset : PluginsToUnloadAssets)
			{
				PluginNames.Add(PluginsToUnloadAsset.Key);
				UpdateStateMachineIds.Add(PluginsToUnloadAsset.Value);
			}
			PluginsToUnloadAssets.Empty();
		}

		verify(FPluginUtils::UnloadPluginsAssets(PluginNames));

		UGameFeaturesSubsystem& Subsystem = UGameFeaturesSubsystem::Get();
		for (int64 Id : UpdateStateMachineIds)
		{
			Subsystem.TryUpdateStateMachineImmediate(Id);
		}

		return false;
	}

	void ScheduleUnloadPluginAssets(const FString& PluginName, const FGameFeaturePluginStateMachineProperties& StateProperties)
	{
		check(IsInGameThread());
		ensure(!PluginsToUnloadAssets.Contains(PluginName));
		PluginsToUnloadAssets.Add(PluginName, StateProperties.Id);
		if (!UnloadPluginAssetsHandle.IsValid())
		{
			UnloadPluginAssetsHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&TickUnloadPluginAssets));
		}
	}
#endif //if WITH_EDITOR

	enum class EGFPInstallLevel : uint8
	{
		Download = 0,
		Mount,
		AssetStream
	};

	struct FPluginInstallBundleReferencers
	{
		// GFPs using an install bundle and the relevant state of that GFP
		TMap<FString, EGFPInstallLevel> GFPs;
	};

	struct FPluginIoStoreOnDemandHandles
	{
		// IoStoreOnDemand assets required for initial download
		UE::IoStore::FOnDemandContentHandle DownloadHandle;
		// IoStoreOnDemand assets from AssetDependencyStreaming
		UE::IoStore::FOnDemandContentHandle StreamInHandle;
	};

	// class to manage GFPs sharing installation data
	class FGFPSharedInstallTracker
	{
	public:
		// The caller should pass all resolved bundle depenencies 
		void AddBundleRefs(const FStringView PluginName, const EGFPInstallLevel Level, const TConstArrayView<FName> Bundles)
		{
			for (const FName BundleName : Bundles)
			{
				FPluginInstallBundleReferencers& PluginRefs = InstallBundleToGFPRefs.FindOrAdd(BundleName);
				PluginRefs.GFPs.Emplace(PluginName, Level);
			}
		}

		// The caller should pass all resolved bundle depenencies 
		UE::IoStore::FOnDemandContentHandle AddOnDemandContentHandle(const FName Bundle, const EGFPInstallLevel Level)
		{
			switch(Level)
			{
			case EGFPInstallLevel::Download:
			{
				FPluginIoStoreOnDemandHandles& Handles = InstallBundleToOnDemandHandles.FindOrAdd(Bundle);
				if (!Handles.DownloadHandle.IsValid())
				{
					Handles.DownloadHandle = UE::IoStore::FOnDemandContentHandle::Create(Bundle.ToString());
				}
				return Handles.DownloadHandle;
			}
			case EGFPInstallLevel::AssetStream:
			{
				FPluginIoStoreOnDemandHandles& Handles = InstallBundleToOnDemandHandles.FindOrAdd(Bundle);
				if (!Handles.StreamInHandle.IsValid())
				{
					Handles.StreamInHandle = UE::IoStore::FOnDemandContentHandle::Create(Bundle.ToString() + TEXTVIEW("/deps"));
				}
				return Handles.StreamInHandle;
			}
			};

			return {};
		}

		// The caller should pass all resolved bundle depenencies 
		TArray<FName> Release(
			const FStringView PluginName, const EGFPInstallLevel InLevel, const TConstArrayView<FName> Bundles)
		{
			uint32 PluginNameHash = GetTypeHash(PluginName);
			TArray<FName> BundlesToRelease;
			for (const FName Bundle : Bundles)
			{
				bool bRelease = true;

				FPluginInstallBundleReferencers* PluginRefs = InstallBundleToGFPRefs.Find(Bundle);
				if (PluginRefs)
				{
					EGFPInstallLevel* Level = PluginRefs->GFPs.FindByHash(PluginNameHash, PluginName);
					if (Level && *Level >= InLevel)
					{
						switch(*Level)
						{
						case EGFPInstallLevel::Download:
							PluginRefs->GFPs.RemoveByHash(PluginNameHash, PluginName);
							break;
						case EGFPInstallLevel::Mount:
							(*Level) = EGFPInstallLevel::Download;
							break;
						case EGFPInstallLevel::AssetStream:
							(*Level) = EGFPInstallLevel::Mount;
							break;
						}
					}

					for (TPair<FString, EGFPInstallLevel>& GFPRef : PluginRefs->GFPs)
					{
						if (GFPRef.Value >= InLevel)
						{
							bRelease = false;
							break;
						}
					}

					if (PluginRefs->GFPs.IsEmpty())
					{
						InstallBundleToGFPRefs.Remove(Bundle);
					}
				}

				if (bRelease)
				{
					BundlesToRelease.Add(Bundle);
				}
			}

			switch(InLevel)
			{
			case EGFPInstallLevel::Download:
				for (const FName Bundle : BundlesToRelease)
				{
					InstallBundleToOnDemandHandles.Remove(Bundle);
				}
				break;
			case EGFPInstallLevel::AssetStream:
				for (const FName& Bundle : BundlesToRelease)
				{
					FPluginIoStoreOnDemandHandles* Handles = InstallBundleToOnDemandHandles.Find(Bundle);
					if (Handles)
					{
						Handles->StreamInHandle.Reset();
					}
				}
				break;
			}

			return BundlesToRelease;
		}

	private:
		TMap<FName, FPluginInstallBundleReferencers> InstallBundleToGFPRefs;
		TMap<FName, FPluginIoStoreOnDemandHandles> InstallBundleToOnDemandHandles;
	};

	static FGFPSharedInstallTracker GFPSharedInstallTracker;

	// Callback delegates are moved to the stack before broadcasting.
	// This class tracks callback delegates on the stack to handle removing callbacks from them
	// for state machines that are also on the stack.
	template<typename TCallbackMultiDelegate> 
		requires 
			std::is_same_v<TCallbackMultiDelegate, FDestinationGameFeaturePluginState::FOnDestinationStateReached> ||
			std::is_same_v<TCallbackMultiDelegate, FGameFeaturePluginStateMachineProperties::FOnTransitionCanceled>
	class TBroadcastingCallback
	{
	public:
		TCallbackMultiDelegate CallbackDelegate;

	private:
		TBroadcastingCallback<TCallbackMultiDelegate>* Next = nullptr;
		inline static TBroadcastingCallback<TCallbackMultiDelegate>* Head = nullptr;

	public:
		TBroadcastingCallback(TCallbackMultiDelegate&& InCallbackDelegate)
			: CallbackDelegate(MoveTemp(InCallbackDelegate))
			, Next(Head)
		{
			Head = this;
		}

		~TBroadcastingCallback()
		{
			Head = Next;
		}

		static void RemovePendingCallback(FDelegateHandle InHandle)
		{
			TBroadcastingCallback<TCallbackMultiDelegate>* Current = Head;
			while (Current)
			{
				Current->CallbackDelegate.Remove(InHandle);
				Current = Current->Next;
			}
		}

		static void RemovePendingCallback(FDelegateUserObject InObject)
		{
			TBroadcastingCallback<TCallbackMultiDelegate>* Current = Head;
			while (Current)
			{
				Current->CallbackDelegate.RemoveAll(InObject);
				Current = Current->Next;
			}
		}
	};

	using FBroadcastingOnDestinationStateReached = TBroadcastingCallback<FDestinationGameFeaturePluginState::FOnDestinationStateReached>;
	using FBroadcastingOnTransitionCanceled = TBroadcastingCallback<FGameFeaturePluginStateMachineProperties::FOnTransitionCanceled>;

	static FName GetStateName(EGameFeaturePluginState State)
	{
		switch (State)
		{
			#define X(inEnum, inText) \
				case EGameFeaturePluginState::inEnum: \
				{ \
					static const FName Name_##inEnum = FName(TEXT("GFP_") #inEnum); \
					return Name_##inEnum; \
				}

			GAME_FEATURE_PLUGIN_STATE_LIST(X)

			#undef X
		}

		return FName(TEXT("Unknown"));
	};

	// Default next state to transition to when transitioning down towards Terminal
	static constexpr EGameFeaturePluginState NextStateTowardsTerminal[EGameFeaturePluginState::MAX] = 
	{
		/* Uninitialized */ 				EGameFeaturePluginState::Uninitialized,
		/* Terminal */ 						EGameFeaturePluginState::Terminal,
		/* UnknownStatus */ 				EGameFeaturePluginState::Terminal,
		/* Uninstalled */ 					EGameFeaturePluginState::Terminal,
		/* Uninstalling */					EGameFeaturePluginState::Uninstalled,
		/* ErrorUninstalling */ 			EGameFeaturePluginState::Uninstalling, // NOTE: May not need to uninstall
		/* CheckingStatus */ 				EGameFeaturePluginState::StatusKnown, // NOTE: No down transition
		/* ErrorCheckingStatus */ 			EGameFeaturePluginState::Terminal,
		/* ErrorUnavailable */ 				EGameFeaturePluginState::Terminal,
		/* StatusKnown */ 					EGameFeaturePluginState::Uninstalling, // NOTE: May not need to uninstall
		/* Releasing */ 					EGameFeaturePluginState::StatusKnown,
		/* ErrorManagingData */ 			EGameFeaturePluginState::Releasing, // TODO: Confirm
		/* Downloading */ 					EGameFeaturePluginState::Installed, // NOTE: No down transition
		/* Installed */ 					EGameFeaturePluginState::Releasing,
		/* ErrorMounting */ 				EGameFeaturePluginState::Unmounting,
		/* ErrorWaitingForDependencies */	EGameFeaturePluginState::Unmounting,
		/* ErrorRegistering */		 		EGameFeaturePluginState::Unregistering,
		/* WaitingForDependencies */ 		EGameFeaturePluginState::AssetDependencyStreaming, // NOTE: No down transition
		/* AssetDependencyStreamOut */	 	EGameFeaturePluginState::Unmounting,
		/* ErrorAssetDependencyStreaming */ EGameFeaturePluginState::AssetDependencyStreamOut,
		/* AssetDependencyStreaming */ 		EGameFeaturePluginState::Registering, // NOTE: No down transition
		/* Unmounting */ 					EGameFeaturePluginState::Installed,
		/* Mounting */ 						EGameFeaturePluginState::WaitingForDependencies, // NOTE: No down transition
		/* Unregistering */ 				EGameFeaturePluginState::AssetDependencyStreamOut,
		/* Registering */ 					EGameFeaturePluginState::Registered, // NOTE: No down transition
		/* Registered */ 					EGameFeaturePluginState::Unregistering,
		/* ErrorLoading */					EGameFeaturePluginState::Unloading,
		/* Unloading */ 					EGameFeaturePluginState::Registered,
		/* Loading */ 						EGameFeaturePluginState::Loaded, // NOTE: No down transitoin
		/* Loaded */ 						EGameFeaturePluginState::Unloading,
		/* ErrorActivatingDependencies */	EGameFeaturePluginState::Unloading,
		/* ActivatingDependencies */ 		EGameFeaturePluginState::Activating, // NOTE: No down transition
		/* Deactivating */ 					EGameFeaturePluginState::Loaded,
		/* ErrorDeactivatingDependencies */	EGameFeaturePluginState::DeactivatingDependencies,
		/* DeactivatingDependencies */ 		EGameFeaturePluginState::Deactivating,
		/* Activating */ 					EGameFeaturePluginState::Active, // NOTE: No down transition
		/* Active */ 						EGameFeaturePluginState::DeactivatingDependencies,
	};
	// Default next state to transition to when transitioning up towards Active
	static constexpr EGameFeaturePluginState NextStateTowardsActive[EGameFeaturePluginState::MAX] = 
	{
		/* Uninitialized */ 				EGameFeaturePluginState::UnknownStatus,
		/* Terminal */ 						EGameFeaturePluginState::Terminal, // No up transition, must be recreated
		/* UnknownStatus */ 				EGameFeaturePluginState::CheckingStatus,
		/* Uninstalled */ 					EGameFeaturePluginState::CheckingStatus,
		/* Uninstalling */					EGameFeaturePluginState::Uninstalled, // NOTE: No up transition
		/* ErrorUninstalling */ 			EGameFeaturePluginState::StatusKnown,
		/* CheckingStatus */ 				EGameFeaturePluginState::StatusKnown,
		/* ErrorCheckingStatus */ 			EGameFeaturePluginState::CheckingStatus,
		/* ErrorUnavailable */ 				EGameFeaturePluginState::CheckingStatus,
		/* StatusKnown */ 					EGameFeaturePluginState::Downloading, // NOTE: May go straight to Installed
		/* Releasing */ 					EGameFeaturePluginState::StatusKnown, // NOTE: No up transition
		/* ErrorManagingData */ 			EGameFeaturePluginState::Downloading,
		/* Downloading */ 					EGameFeaturePluginState::Installed,
		/* Installed */ 					EGameFeaturePluginState::Mounting,
		/* ErrorMounting */ 				EGameFeaturePluginState::Mounting,
		/* ErrorWaitingForDependencies */	EGameFeaturePluginState::WaitingForDependencies,
		/* ErrorRegistering */		 		EGameFeaturePluginState::Registering,
		/* WaitingForDependencies */ 		EGameFeaturePluginState::AssetDependencyStreaming,
		/* AssetDependencyStreamOut */	 	EGameFeaturePluginState::Unmounting, // NOTE: No up transition
		/* ErrorAssetDependencyStreaming */ EGameFeaturePluginState::AssetDependencyStreaming,
		/* AssetDependencyStreaming */ 		EGameFeaturePluginState::Registering,
		/* Unmounting */ 					EGameFeaturePluginState::Installed, // NOTE: No up transition
		/* Mounting */ 						EGameFeaturePluginState::WaitingForDependencies,
		/* Unregistering */ 				EGameFeaturePluginState::AssetDependencyStreamOut, // NOTE: No up transition
		/* Registering */ 					EGameFeaturePluginState::Registered,
		/* Registered */ 					EGameFeaturePluginState::Loading,
		/* ErrorLoading */					EGameFeaturePluginState::Loading,
		/* Unloading */ 					EGameFeaturePluginState::Registered, // NOTE: No up transition
		/* Loading */ 						EGameFeaturePluginState::Loaded,
		/* Loaded */ 						EGameFeaturePluginState::ActivatingDependencies,
		/* ErrorActivatingDependencies */	EGameFeaturePluginState::ActivatingDependencies,
		/* ActivatingDependencies */ 		EGameFeaturePluginState::Activating,
		/* Deactivating */ 					EGameFeaturePluginState::Loaded, // NOTE: No up transition
		/* ErrorDeactivatingDependencies */	EGameFeaturePluginState::DeactivatingDependencies,
		/* DeactivatingDependencies */ 		EGameFeaturePluginState::Deactivating, // NOTE: No up transition
		/* Activating */ 					EGameFeaturePluginState::Active,
		/* Active */ 						EGameFeaturePluginState::Active
	};
	// Default next state to transition to when encountering an error
	static constexpr EGameFeaturePluginState NextStateOnError[EGameFeaturePluginState::MAX] = 
	{
		/* Uninitialized */ 				EGameFeaturePluginState::Uninitialized,// No errors handled
		/* Terminal */ 						EGameFeaturePluginState::Terminal, // No errors handled
		/* UnknownStatus */ 				EGameFeaturePluginState::UnknownStatus, // No errors handled
		/* Uninstalled */ 					EGameFeaturePluginState::Uninstalled, // No errors handled
		/* Uninstalling */					EGameFeaturePluginState::ErrorUninstalling, 
		/* ErrorUninstalling */ 			EGameFeaturePluginState::ErrorUninstalling, // Already an error state
		/* CheckingStatus */ 				EGameFeaturePluginState::ErrorCheckingStatus,
		/* ErrorCheckingStatus */ 			EGameFeaturePluginState::ErrorCheckingStatus, // Already an error state
		/* ErrorUnavailable */ 				EGameFeaturePluginState::ErrorUnavailable, // Already an error state
		/* StatusKnown */ 					EGameFeaturePluginState::StatusKnown, // No errors handled
		/* Releasing */ 					EGameFeaturePluginState::ErrorManagingData, 
		/* ErrorManagingData */ 			EGameFeaturePluginState::ErrorManagingData, // Already an error state
		/* Downloading */ 					EGameFeaturePluginState::ErrorManagingData,
		/* Installed */ 					EGameFeaturePluginState::Installed, // No errors handled
		/* ErrorMounting */ 				EGameFeaturePluginState::ErrorMounting, // Already an error state
		/* ErrorWaitingForDependencies */	EGameFeaturePluginState::ErrorWaitingForDependencies, // Already an error state
		/* ErrorRegistering */		 		EGameFeaturePluginState::ErrorRegistering, // Already an error state
		/* WaitingForDependencies */ 		EGameFeaturePluginState::ErrorWaitingForDependencies,
		/* AssetDependencyStreamOut */	 	EGameFeaturePluginState::ErrorAssetDependencyStreaming, 
		/* ErrorAssetDependencyStreaming */ EGameFeaturePluginState::ErrorAssetDependencyStreaming, // Already an error state
		/* AssetDependencyStreaming */ 		EGameFeaturePluginState::ErrorAssetDependencyStreaming,
		/* Unmounting */ 					EGameFeaturePluginState::ErrorMounting,
		/* Mounting */ 						EGameFeaturePluginState::ErrorMounting,
		/* Unregistering */ 				EGameFeaturePluginState::AssetDependencyStreamOut, // NOTE: No up transition
		/* Registering */ 					EGameFeaturePluginState::ErrorRegistering,
		/* Registered */ 					EGameFeaturePluginState::Registered, // No errors handled
		/* ErrorLoading */					EGameFeaturePluginState::Loading,
		/* Unloading */ 					EGameFeaturePluginState::ErrorLoading, 
		/* Loading */ 						EGameFeaturePluginState::ErrorLoading,
		/* Loaded */ 						EGameFeaturePluginState::Loaded, // No errors handled
		/* ErrorActivatingDependencies */	EGameFeaturePluginState::ErrorActivatingDependencies, // Already an error state
		/* ActivatingDependencies */ 		EGameFeaturePluginState::ErrorActivatingDependencies,
		/* Deactivating */ 					EGameFeaturePluginState::Deactivating, // No errors handled
		/* ErrorDeactivatingDependencies */	EGameFeaturePluginState::ErrorActivatingDependencies, // Already an error state
		/* DeactivatingDependencies */ 		EGameFeaturePluginState::ErrorDeactivatingDependencies, 
		/* Activating */ 					EGameFeaturePluginState::Activating, // No errors handled
		/* Active */ 						EGameFeaturePluginState::Active // No errors handled
	};
	static constexpr EGameFeaturePluginStateType StateType[EGameFeaturePluginState::MAX] = 
	{
		/* Uninitialized */ 				EGameFeaturePluginStateType::Transition,
		/* Terminal */ 						EGameFeaturePluginStateType::Destination,
		/* UnknownStatus */ 				EGameFeaturePluginStateType::Destination,
		/* Uninstalled */ 					EGameFeaturePluginStateType::Destination,
		/* Uninstalling */					EGameFeaturePluginStateType::Transition,
		/* ErrorUninstalling */ 			EGameFeaturePluginStateType::Error,
		/* CheckingStatus */ 				EGameFeaturePluginStateType::Transition,
		/* ErrorCheckingStatus */ 			EGameFeaturePluginStateType::Error,
		/* ErrorUnavailable */ 				EGameFeaturePluginStateType::Error,
		/* StatusKnown */ 					EGameFeaturePluginStateType::Destination,
		/* Releasing */ 					EGameFeaturePluginStateType::Transition,
		/* ErrorManagingData */ 			EGameFeaturePluginStateType::Error,
		/* Downloading */ 					EGameFeaturePluginStateType::Transition,
		/* Installed */ 					EGameFeaturePluginStateType::Destination,
		/* ErrorMounting */ 				EGameFeaturePluginStateType::Error,
		/* ErrorWaitingForDependencies */	EGameFeaturePluginStateType::Error,
		/* ErrorRegistering */		 		EGameFeaturePluginStateType::Error,
		/* WaitingForDependencies */ 		EGameFeaturePluginStateType::Transition,
		/* AssetDependencyStreamOut */	 	EGameFeaturePluginStateType::Transition,
		/* ErrorAssetDependencyStreaming */ EGameFeaturePluginStateType::Error,
		/* AssetDependencyStreaming */ 		EGameFeaturePluginStateType::Transition,
		/* Unmounting */ 					EGameFeaturePluginStateType::Transition,
		/* Mounting */ 						EGameFeaturePluginStateType::Transition,
		/* Unregistering */ 				EGameFeaturePluginStateType::Transition,
		/* Registering */ 					EGameFeaturePluginStateType::Transition,
		/* Registered */ 					EGameFeaturePluginStateType::Destination,
		/* ErrorLoading */					EGameFeaturePluginStateType::Error,
		/* Unloading */ 					EGameFeaturePluginStateType::Transition,
		/* Loading */ 						EGameFeaturePluginStateType::Transition,
		/* Loaded */ 						EGameFeaturePluginStateType::Destination,
		/* ErrorActivatingDependencies */	EGameFeaturePluginStateType::Error,
		/* ActivatingDependencies */ 		EGameFeaturePluginStateType::Transition,
		/* Deactivating */ 					EGameFeaturePluginStateType::Transition,
		/* ErrorDeactivatingDependencies */	EGameFeaturePluginStateType::Error,
		/* DeactivatingDependencies */ 		EGameFeaturePluginStateType::Transition,
		/* Activating */ 					EGameFeaturePluginStateType::Transition,
		/* Active */ 						EGameFeaturePluginStateType::Destination,
	};
}

void SerializeForLog(FCbWriter& Writer, FGameFeaturePluginStateRange InRange)
{
	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("GameFeaturePluginStateRange"));
	Writer.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("({MinState}, {MaxState})"));
	Writer << ANSITEXTVIEW("MinState") << InRange.MinState;
	Writer << ANSITEXTVIEW("MaxState") <<  InRange.MaxState;
	Writer.EndObject();
}

void FGameFeaturePluginStateStatus::SetTransition(EGameFeaturePluginState InTransitionToState)
{
	TransitionToState = InTransitionToState;
	TransitionResult.ErrorCode = MakeValue();
	TransitionResult.OptionalErrorText = FText();
}

void FGameFeaturePluginStateStatus::SetTransitionError(EGameFeaturePluginState TransitionToErrorState, UE::GameFeatures::FResult TransitionResultIn, bool bInSuppressErrorLog /*= false*/)
{
	if (ensureAlwaysMsgf(TransitionResultIn.HasError(), TEXT("Invalid call to SetTransitionError with an FResult that isn't an error! TransitionToErrorState: %s"), *UE::GameFeatures::ToString(TransitionToErrorState)))
	{
		TransitionResult = MoveTemp(TransitionResultIn);
	}
	else
	{
		//Logic error using a non-error FResult, so just generate a general error to keep the SetTransitionError intent
		TransitionResult = MakeError(TEXT("Invalid_Transition_Error"));
	}

	TransitionToState = TransitionToErrorState;
	bSuppressErrorLog = bInSuppressErrorLog;
}

UE::GameFeatures::FResult FGameFeaturePluginState::GetErrorResult(const FString& ErrorCode, const FText OptionalErrorText/*= FText()*/) const
{
	return GetErrorResult(TEXT(""), ErrorCode, OptionalErrorText);
}

UE::GameFeatures::FResult FGameFeaturePluginState::GetErrorResult(const FString& ErrorNamespaceAddition, const FString& ErrorCode, const FText OptionalErrorText/*= FText()*/) const
{
	const FString StateName = UE::GameFeatures::ToString(UGameFeaturesSubsystem::Get().GetPluginState(StateProperties.PluginIdentifier));
	const FString ErrorCodeEnding = ErrorNamespaceAddition.IsEmpty() ? ErrorCode : ErrorNamespaceAddition + ErrorCode;
	const FString CompleteErrorCode = FString::Printf(TEXT("%s%s.%s"), *UE::GameFeatures::StateMachineErrorNamespace, *StateName, *ErrorCodeEnding);
	return UE::GameFeatures::FResult(MakeError(CompleteErrorCode), OptionalErrorText);
}

UE::GameFeatures::FResult FGameFeaturePluginState::GetErrorResult(const FString& ErrorNamespaceAddition, const EInstallBundleResult ErrorResult) const
{
	UE::GameFeatures::FResult BaseResult = GetErrorResult(ErrorNamespaceAddition, LexToString(ErrorResult));
	BaseResult.OptionalErrorText = UE::GameFeatures::CommonErrorCodes::GetErrorTextForBundleResult(ErrorResult);
	return MoveTemp(BaseResult);
}

UE::GameFeatures::FResult FGameFeaturePluginState::GetErrorResult(const FString& ErrorNamespaceAddition, const EInstallBundleReleaseResult ErrorResult) const
{
	UE::GameFeatures::FResult BaseResult = GetErrorResult(ErrorNamespaceAddition, LexToString(ErrorResult));
	BaseResult.OptionalErrorText = UE::GameFeatures::CommonErrorCodes::GetErrorTextForReleaseResult(ErrorResult);
	return MoveTemp(BaseResult);
}

void SerializeForLog(FCbWriter& Writer, EGameFeaturePluginState State)
{
#define STATE_TO_STRING(NAME, LOCNAME) case EGameFeaturePluginState::NAME: Writer.AddString(TEXT(#NAME)); break;
	switch (State)
	{
GAME_FEATURE_PLUGIN_STATE_LIST(STATE_TO_STRING)
	}
#undef STATE_TO_STRING
	
}

FGameFeaturePluginState::~FGameFeaturePluginState()
{
	CleanupDeferredUpdateCallbacks();
}

UE::GameFeatures::FResult FGameFeaturePluginState::TryUpdateProtocolOptions(const FGameFeatureProtocolOptions& NewOptions)
{
	UE::GameFeatures::FResult Result = StateProperties.ValidateProtocolOptionsUpdate(NewOptions);

	if (!Result.HasError())
	{
		StateProperties.ProtocolOptions = NewOptions;
	}
	return Result;
}

FDestinationGameFeaturePluginState* FGameFeaturePluginState::AsDestinationState()
{
	FDestinationGameFeaturePluginState* Ret = nullptr;

	EGameFeaturePluginStateType Type = GetStateType();
	if (Type == EGameFeaturePluginStateType::Destination || Type == EGameFeaturePluginStateType::Error)
	{
		Ret = static_cast<FDestinationGameFeaturePluginState*>(this);
	}

	return Ret;
}

FErrorGameFeaturePluginState* FGameFeaturePluginState::AsErrorState()
{
	FErrorGameFeaturePluginState* Ret = nullptr;

	if (GetStateType() == EGameFeaturePluginStateType::Error)
	{
		Ret = static_cast<FErrorGameFeaturePluginState*>(this);
	}

	return Ret;
}

void FGameFeaturePluginState::UpdateStateMachineDeferred(float Delay /*= 0.0f*/) const
{
	// If the shared executor is enabled, don't create an individual callback for this state
	if (StateProperties.UpdateStateMachineDeferred(Delay))
	{
		return;
	}
	CleanupDeferredUpdateCallbacks();

	GFP_STATE_LOG(VeryVerbose, "queueing deferred update");
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float dts) mutable
	{
		// @note Release FGameFeaturePluginState::TickHandle first in case the termination callback triggers a GC and destroys the state machine
		TickHandle.Reset();
		StateProperties.UpdateStateMachineImmediate();
		return false;
	}), Delay);
}

void FGameFeaturePluginState::UpdateStateMachineImmediate() const
{
	StateProperties.UpdateStateMachineImmediate();
}

void FGameFeaturePluginState::UpdateProgress(float Progress) const
{
	StateProperties.StateProgress = Progress; 
}

bool FGameFeaturePluginState::CanBatchProcess() const
{
	// Batch processing is tick driven so is technically "async". 
	// Hence if we are sync loading, avoid using batch processing since it could impact order of operations.
	return UseAsyncLoading();
}

bool FGameFeaturePluginState::IsWaitingForBatchProcessing() const
{
	return StateProperties.IsWaitingForBatchProcessing();
}

bool FGameFeaturePluginState::WasBatchProcessed() const
{
	return StateProperties.WasBatchProcessed();
}

void FGameFeaturePluginState::CleanupDeferredUpdateCallbacks() const
{
	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
}

bool FGameFeaturePluginState::ShouldVisitUninstallStateBeforeTerminal() const
{
	switch (StateProperties.GetPluginProtocol())
	{
		case (EGameFeaturePluginProtocol::InstallBundle):
		{
			//InstallBundleProtocol's have a MetaData that controlls if they uninstall currently
			return StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>().bUninstallBeforeTerminate;
		}

		//Default behavior is to just Terminate
		default:
		{
			return false;
		}
	}
}

bool FGameFeaturePluginState::AllowIniLoading() const
{
	switch (StateProperties.GetPluginProtocol())
	{
	case (EGameFeaturePluginProtocol::InstallBundle):
	{
		//InstallBundleProtocol's have a MetaData that controls if INI loading is allowed
		//The protocol default is not to allow INI loading since the source is likely untrusted
		return StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>().bAllowIniLoading;
	}

	//Default behavior is to allow INI loading
	default:
	{
		return true;
	}
	}
}

bool FGameFeaturePluginState::AllowAsyncLoading() const
{
	return StateProperties.AllowAsyncLoading();
}

bool FGameFeaturePluginState::UseAsyncLoading() const
{
	return AllowAsyncLoading() && UE::GameFeatures::CVarAsyncLoad.GetValueOnGameThread();
}

void FDestinationGameFeaturePluginState::UpdateState(FGameFeaturePluginStateStatus& StateStatus)
{
	if (StateProperties.Destination < StateProperties.CurrentState)
	{
		StateStatus.SetTransition(UE::GameFeatures::NextStateTowardsTerminal[StateProperties.CurrentState]);
	}
	else if (StateProperties.Destination > StateProperties.CurrentState)
	{
		StateStatus.SetTransition(UE::GameFeatures::NextStateTowardsActive[StateProperties.CurrentState]);
	}
}

void FDestinationGameFeaturePluginState::TryCancelState()
{
}

void FDestinationGameFeaturePluginState::EndState()
{
}

/*
=========================================================
  States
=========================================================
*/

template<typename TransitionPolicy>
struct FTransitionDependenciesGameFeaturePluginState : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	virtual ~FTransitionDependenciesGameFeaturePluginState()
	{
		ClearDependencies();
	}

	virtual void BeginState() override
	{
		ClearDependencies();
		bCheckedRealtimeMode = false;
	}

	virtual void EndState() override
	{
		ClearDependencies();
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (!bCheckedRealtimeMode)
		{
			bCheckedRealtimeMode = true;
			if (UE::GameFeatures::RealtimeMode)
			{
				UE::GameFeatures::RealtimeMode->AddUpdateRequest(StateProperties);
				return;
			}
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_TransitionDependencies);
		checkf(!StateProperties.PluginInstalledFilename.IsEmpty(), TEXT("PluginInstalledFilename must be set by the loading dependencies phase. PluginURL: %s"), *StateProperties.PluginIdentifier.GetFullPluginURL());
		checkf(FPaths::GetExtension(StateProperties.PluginInstalledFilename) == TEXT("uplugin"), TEXT("PluginInstalledFilename must have a uplugin extension. PluginURL: %s"), *StateProperties.PluginIdentifier.GetFullPluginURL());

		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();
		if (!bRequestedDependencies)
		{
			TArray<UGameFeaturePluginStateMachine*> Dependencies;
			if (!TransitionPolicy::GetPluginDependencyStateMachines(StateProperties, Dependencies))
			{
				// Failed to query dependencies
				StateStatus.SetTransitionError(TransitionPolicy::GetErrorState(), GetErrorResult(TEXT("Failed_Dependency_Query")));
				return;
			}

			bRequestedDependencies = true;

			UE_CLOGFMT(Dependencies.Num() > 0, LogGameFeatures, Verbose, "Found {Count} dependencies for {PluginName} : {Dependencies}", 
				Dependencies.Num(), StateProperties.PluginName,
				[&](){
					TArray<FString> DependencyNames;
					for (UGameFeaturePluginStateMachine* SM : Dependencies)
					{
						DependencyNames.Add(SM->StateProperties.PluginName);
					}
					return DependencyNames;
				}());

			const bool bAllowAsyncLoading = AllowAsyncLoading();
			RemainingDependencies.Reserve(Dependencies.Num());
			for (UGameFeaturePluginStateMachine* Dependency : Dependencies)
			{
				ensureMsgf(bAllowAsyncLoading || !GameFeaturesSubsystem.AllowAsyncLoading(Dependency), 
					TEXT("FGameFeaturePluginState::AllowAsyncLoading is false for %s but true for dependency being waited on %s"), 
					*StateProperties.PluginName, *GameFeaturesSubsystem.GetPluginURL(Dependency));

				if (TransitionPolicy::ShouldWaitForDependencies() && 
					GameFeaturesSubsystem.IsInErrorState(Dependency) &&
					GameFeaturesSubsystem.IsErrorStateUnrecoverable(Dependency))
				{
					const EGameFeaturePluginState DependencyState = GameFeaturesSubsystem.GetCurrentState(Dependency);
					const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
					const FStringView DepShortUrl = GameFeaturesSubsystem.GetPluginIdentifier(Dependency).GetIdentifyingString();
					UE_LOGF(LogGameFeatures, Error, "Plugin (%.*ls) dependency (%.*ls) failed and is in state %ls",
						ShortUrl.Len(), ShortUrl.GetData(),
						DepShortUrl.Len(), DepShortUrl.GetData(),
						*UE::GameFeatures::ToString(DependencyState));
					StateStatus.SetTransitionError(TransitionPolicy::GetErrorState(), GetErrorResult(TEXT("Failed_Dependency_Transition")));
					if (UGameFeaturePluginStateMachine* CurrentMachine = GameFeaturesSubsystem.FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier))
					{
						UE_LOGF(LogGameFeatures, Error, "Setting plugin (%.*ls) to be in unrecoverable error because dependency (%.*ls) is in unrecoverable error", 
							ShortUrl.Len(), ShortUrl.GetData(),
							DepShortUrl.Len(), DepShortUrl.GetData());
						GameFeaturesSubsystem.SetUnrecoverableError(CurrentMachine);
					}
					return;
				}
				else
				{
					RemainingDependencies.Emplace(Dependency, MakeValue());
					TransitionDependency(Dependency);
				}
			}
		}

		for (FDepResultPair& Pair : RemainingDependencies)
		{
			UGameFeaturePluginStateMachine* RemainingDependency = Pair.Key.Get();
			if (!RemainingDependency)
			{
				// One of the dependency state machines was destroyed before finishing
				StateStatus.SetTransitionError(TransitionPolicy::GetErrorState(), GetErrorResult(TEXT("Dependency_Destroyed_Before_Finish")));
				return;
			}

			if (Pair.Value.HasError())
			{
				const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
				const FStringView DepShortUrl = GameFeaturesSubsystem.GetPluginIdentifier(RemainingDependency).GetIdentifyingString();
				UE_LOGF(LogGameFeatures, Error, "Plugin (%.*ls) dependency (%.*ls) failed to transition with error %ls",
					ShortUrl.Len(), ShortUrl.GetData(),
					DepShortUrl.Len(), DepShortUrl.GetData(),
					*Pair.Value.GetError());
				StateStatus.SetTransitionError(TransitionPolicy::GetErrorState(), GetErrorResult(TEXT("Failed_Dependency_Transition")));
				if (GameFeaturesSubsystem.IsErrorStateUnrecoverable(RemainingDependency))
				{
					if (UGameFeaturePluginStateMachine* CurrentMachine = GameFeaturesSubsystem.FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier))
					{
						UE_LOGF(LogGameFeatures, Error, "Setting plugin (%.*ls) to be in unrecoverable error because dependency (%.*ls) is in unrecoverable error",
							ShortUrl.Len(), ShortUrl.GetData(),
							DepShortUrl.Len(), DepShortUrl.GetData());
						GameFeaturesSubsystem.SetUnrecoverableError(CurrentMachine);
					}
				}
				return;
			}
		}

		if (RemainingDependencies.Num() == 0)
		{
			StateStatus.SetTransition(TransitionPolicy::GetTransitionState());
		}
	}

	void TransitionDependency(UGameFeaturePluginStateMachine* Dependency)
	{
		bool bSetDestination = false;

		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();
		if (TransitionPolicy::ExcludeDepedenciesFromBatchProcessing())
		{
			GameFeaturesSubsystem.ExcludeFromBatchProcessing(Dependency);
		}
		
		if (TransitionPolicy::ShouldWaitForDependencies())
		{
			bSetDestination = GameFeaturesSubsystem.SetDestination(Dependency, TransitionPolicy::GetDependencyStateRange(),
				FGameFeatureStateTransitionComplete::CreateRaw(this, &FTransitionDependenciesGameFeaturePluginState::OnDependencyTransitionComplete));
		}
		else
		{
			bSetDestination = GameFeaturesSubsystem.SetDestination(Dependency, TransitionPolicy::GetDependencyStateRange(), 
				FGameFeatureStateTransitionComplete::CreateStatic(&FTransitionDependenciesGameFeaturePluginState::OnDependencyTransitionCompleteNoWait));
			if (bSetDestination)
			{
				OnDependencyTransitionComplete(Dependency, MakeValue());
			}
		}

		if (!bSetDestination)
		{
			const bool bCancelPending = GameFeaturesSubsystem.TryCancel(Dependency,
				FGameFeatureStateTransitionCanceled::CreateRaw(this, &FTransitionDependenciesGameFeaturePluginState::OnDependencyTransitionCanceled));
			if (!ensure(bCancelPending))
			{
				OnDependencyTransitionComplete(Dependency, GetErrorResult(TEXT("Failed_Dependency_Transition")));
			}
		}
	}

	void OnDependencyTransitionCanceled(UGameFeaturePluginStateMachine* Dependency)
	{
		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();
		// Special case for terminal state since it cannot be exited, we need to make a new machine
		if (GameFeaturesSubsystem.GetCurrentState(Dependency) == EGameFeaturePluginState::Terminal)
		{
			// Inherit dep protocol options if possible
			FGameFeatureProtocolOptions DepProtocolOptions;
			EGameFeaturePluginProtocol DepProtocol = GameFeaturesSubsystem.GetPluginIdentifier(Dependency).GetPluginProtocol();
			if (DepProtocol == EGameFeaturePluginProtocol::InstallBundle && StateProperties.ProtocolOptions.HasSubtype<FInstallBundlePluginProtocolOptions>())
			{
				DepProtocolOptions = StateProperties.RecycleProtocolOptions();
			}

			UGameFeaturePluginStateMachine* NewMachine = UGameFeaturesSubsystem::Get().FindOrCreateGameFeaturePluginStateMachine(GameFeaturesSubsystem.GetPluginURL(Dependency), DepProtocolOptions);
			checkf(NewMachine != Dependency, TEXT("Game Feature Plugin %s should have already been removed from subsystem!"), *GameFeaturesSubsystem.GetPluginURL(Dependency));

			const int32 Index = RemainingDependencies.IndexOfByPredicate([Dependency](const FDepResultPair& Pair)
				{
					return Pair.Key == Dependency;
				});

			check(Index != INDEX_NONE);
			FDepResultPair& FoundDep = RemainingDependencies[Index];
			FoundDep.Key = NewMachine;

			GameFeaturesSubsystem.RemovePendingTransitionCallback(Dependency, this);
			GameFeaturesSubsystem.RemovePendingCancelCallback(Dependency, this);

			Dependency = NewMachine;
		}

		// Now that the transition has been canceled, retry reaching the desired destination
		bool bSetDestination = false;
		if (TransitionPolicy::ShouldWaitForDependencies())
		{
			bSetDestination = GameFeaturesSubsystem.SetDestination(Dependency, TransitionPolicy::GetDependencyStateRange(),
				FGameFeatureStateTransitionComplete::CreateRaw(this, &FTransitionDependenciesGameFeaturePluginState::OnDependencyTransitionComplete));
		}
		else
		{
			bSetDestination = GameFeaturesSubsystem.SetDestination(Dependency, TransitionPolicy::GetDependencyStateRange(), 
				FGameFeatureStateTransitionComplete::CreateStatic(&FTransitionDependenciesGameFeaturePluginState::OnDependencyTransitionCompleteNoWait));
			if (bSetDestination)
			{
				OnDependencyTransitionComplete(Dependency, MakeValue());
			}
		}

		if (!ensure(bSetDestination))
		{
			OnDependencyTransitionComplete(Dependency, GetErrorResult(TEXT("Failed_Dependency_Transition")));
		}
	}

	void OnDependencyTransitionComplete(UGameFeaturePluginStateMachine* Dependency, const UE::GameFeatures::FResult& Result)
	{
		const int32 Index = RemainingDependencies.IndexOfByPredicate([Dependency](const FDepResultPair& Pair)
			{
				return Pair.Key == Dependency;
			});

		if (ensure(Index != INDEX_NONE))
		{
			FDepResultPair& FoundDep = RemainingDependencies[Index];

			if (Result.HasError())
			{
				FoundDep.Value = Result;
			}
			else
			{
				RemainingDependencies.RemoveAtSwap(Index, EAllowShrinking::No);
			}

			UpdateStateMachineImmediate();
		}
	}

	static void OnDependencyTransitionCompleteNoWait(UGameFeaturePluginStateMachine* Dependency, const UE::GameFeatures::FResult& Result)
	{
		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();
		if (Result.HasError())
		{
			if (Result.GetError() == UE::GameFeatures::CanceledResult.GetError())
			{
				UE_LOGFMT(LogGameFeatures, Warning, "Dependency {Dep} failed to transition because it was cancelled by another request {Error}",
					("Dep", GameFeaturesSubsystem.GetPluginIdentifier(Dependency).GetIdentifyingString()),
					("Error", Result.GetError()));
			}
			else
			{

				UE_LOGFMT(LogGameFeatures, Error, "Dependency {Dep} failed to transition with error {Error}",
					("Dep", GameFeaturesSubsystem.GetPluginIdentifier(Dependency).GetIdentifyingString()),
					("Error", Result.GetError()));
			}
		}
	}

	void ClearDependencies()
	{
		if (RemainingDependencies.Num() > 0)
		{
			UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();
			for (FDepResultPair& Pair : RemainingDependencies)
			{
				UGameFeaturePluginStateMachine* RemainingDependency = Pair.Key.Get();
				if (RemainingDependency)
				{
					GameFeaturesSubsystem.RemovePendingTransitionCallback(RemainingDependency, this);
					GameFeaturesSubsystem.RemovePendingCancelCallback(RemainingDependency, this);
				}
			}

			// Also need to cleanup callbacks from any delegates currently on the stack
			UE::GameFeatures::FBroadcastingOnDestinationStateReached::RemovePendingCallback(this);
			UE::GameFeatures::FBroadcastingOnTransitionCanceled::RemovePendingCallback(this);

			RemainingDependencies.Empty();
		}

		bRequestedDependencies = false;
	}

	using FDepResultPair = TPair<TWeakObjectPtr<UGameFeaturePluginStateMachine>, UE::GameFeatures::FResult>;
	TArray<FDepResultPair> RemainingDependencies;
	bool bRequestedDependencies = false;
	bool bCheckedRealtimeMode = false;
};

struct FGameFeaturePluginState_Uninitialized : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		checkf(false, TEXT("UpdateState can not be called while uninitialized"));
	}
};

struct FGameFeaturePluginState_Terminal : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;
	bool bEnteredTerminalState = false;

	virtual UE::GameFeatures::FResult TryUpdateProtocolOptions(const FGameFeatureProtocolOptions& NewOptions) override
	{
		//Should never update our options during Terminal
		return GetErrorResult(TEXT("ProtocolOptions."), TEXT("Terminal"));
	}

	virtual void BeginState() override
	{
		checkf(!bEnteredTerminalState, TEXT("Plugin entered terminal state more than once! %s"), *StateProperties.PluginIdentifier.GetFullPluginURL());
		bEnteredTerminalState = true;

		UGameFeaturesSubsystem::Get().OnGameFeatureTerminating(StateProperties.PluginName, StateProperties.PluginIdentifier);
	}
};

struct FGameFeaturePluginState_UnknownStatus : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::UnknownStatus] == EGameFeaturePluginState::Terminal);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::UnknownStatus] == EGameFeaturePluginState::CheckingStatus);
};

struct FGameFeaturePluginState_CheckingStatus : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	bool bParsedURL = false;
	bool bIsAvailable = false;

	virtual void BeginState() override
	{
		UGameFeaturesSubsystem::Get().OnGameFeatureCheckingStatus(StateProperties.PluginIdentifier);

		bParsedURL = false;
		bIsAvailable = false;
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (!bParsedURL)
		{
			TValueOrError<void, FString> ParseUrlResult = StateProperties.ParseURL();
			bParsedURL = !ParseUrlResult.HasError();
			if (!bParsedURL)
			{
				StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorCheckingStatus, GetErrorResult(ParseUrlResult.GetError()));
				return;
			}
		}

		if (StateProperties.GetPluginProtocol() == EGameFeaturePluginProtocol::File)
		{
			bIsAvailable = FPaths::FileExists(StateProperties.PluginInstalledFilename);
		}
		else if (StateProperties.GetPluginProtocol() == EGameFeaturePluginProtocol::InstallBundle)
		{
			TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
			if (BundleManager == nullptr)
			{
				StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorCheckingStatus, GetErrorResult(TEXT("BundleManager_Null")));
				return;
			}

			if (BundleManager->GetInitState() == EInstallBundleManagerInitState::Failed)
			{
				StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorCheckingStatus, GetErrorResult(TEXT("BundleManager_Failed_Init")));
				return;
			}

			if (BundleManager->GetInitState() == EInstallBundleManagerInitState::NotInitialized)
			{
				// Just wait for any pending init
				UpdateStateMachineDeferred(1.0f);
				return;
			}

			FInstallBundlePluginProtocolMetaData& ProtocolMetadata = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();

			const bool bAddDependencies = true;
			TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> MaybeInstallState = BundleManager->GetInstallStateSynchronous(
				ProtocolMetadata.InstallBundles, bAddDependencies);
			if (MaybeInstallState.HasError())
			{
				StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorCheckingStatus, GetErrorResult(TEXT("BundleManager_Failed_GetInstallState")));
				return;
			}

			const FInstallBundleCombinedInstallState& InstallState = MaybeInstallState.GetValue();
			bIsAvailable = Algo::AllOf(ProtocolMetadata.InstallBundles, 
				[&InstallState](FName BundleName) { return InstallState.IndividualBundleStates.Contains(BundleName); });

			if (bIsAvailable)
			{
				// Update metadata with fully expanded dependency list. This can only be done after all bundles are known to be available,
				// otherwise unavailable bundles in the URL could be stripped from the list.
				ProtocolMetadata.InstallBundles.Empty(InstallState.IndividualBundleStates.Num());
				InstallState.IndividualBundleStates.GetKeys(ProtocolMetadata.InstallBundles);
				ProtocolMetadata.InstallBundlesWithAssetDependencies = InstallState.BundlesWithIoStoreOnDemand.Array();
			}
		}
		else
		{
			StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorCheckingStatus, GetErrorResult(TEXT("Unknown_Protocol")));
			return;
		}

		if (!bIsAvailable)
		{
			StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorUnavailable, GetErrorResult(TEXT("Plugin_Unavailable")));
			return;
		}

		UGameFeaturesSubsystem::Get().OnGameFeatureStatusKnown(StateProperties.PluginName, StateProperties.PluginIdentifier);
		StateStatus.SetTransition(EGameFeaturePluginState::StatusKnown);
	}
};

struct FGameFeaturePluginState_ErrorCheckingStatus : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorCheckingStatus] == EGameFeaturePluginState::Terminal);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorCheckingStatus] == EGameFeaturePluginState::CheckingStatus);
};

struct FGameFeaturePluginState_ErrorUnavailable : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorUnavailable] == EGameFeaturePluginState::Terminal);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorUnavailable] == EGameFeaturePluginState::CheckingStatus);
};

struct FGameFeaturePluginState_StatusKnown : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::StatusKnown] == EGameFeaturePluginState::Uninstalling);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::StatusKnown] == EGameFeaturePluginState::Downloading);
};

struct FGameFeaturePluginState_ErrorManagingData : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;
	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorManagingData] == EGameFeaturePluginState::Releasing);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorManagingData] == EGameFeaturePluginState::Downloading);
};

struct FGameFeaturePluginState_ErrorUninstalling : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorUninstalling] == EGameFeaturePluginState::Uninstalling);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorUninstalling] == EGameFeaturePluginState::StatusKnown);
};

//Base class for our states that want to request to release data from InstallBundleManager
struct FBaseDataReleaseGameFeaturePluginState : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	virtual ~FBaseDataReleaseGameFeaturePluginState()
	{}

	UE::GameFeatures::FResult Result = MakeValue();
	bool bWasDeleted = false;
	TArray<FName> PendingBundles;

	void CleanUp()
	{
		PendingBundles.Empty();
		IInstallBundleManager::ReleasedDelegate.RemoveAll(this);
	}

	void OnContentRemoved(FInstallBundleReleaseRequestResultInfo BundleResult)
	{
		if (!PendingBundles.Contains(BundleResult.BundleName))
		{
			return;
		}

		PendingBundles.Remove(BundleResult.BundleName);

		if (!Result.HasError() && BundleResult.Result != EInstallBundleReleaseResult::OK)
		{
			Result = GetErrorResult(TEXT("BundleManager.OnRemove_Failed."), BundleResult.Result);
		}

		if (PendingBundles.Num() > 0)
		{
			return;
		}

		if (Result.HasValue())
		{
			bWasDeleted = true;
		}

		UpdateStateMachineImmediate();
	}

	void BeginRemoveRequest()
	{
		CleanUp();

		Result = MakeValue();
		bWasDeleted = false;

		if (!ShouldReleaseContent())
		{
			bWasDeleted = true;
			return;
		}

		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
		check(BundleManager.IsValid());

		TArray<FName> InstallBundlesToRelease = UE::GameFeatures::GFPSharedInstallTracker.Release(
			StateProperties.PluginName, UE::GameFeatures::EGFPInstallLevel::Download, GetInstallBundles());

		// Always set ExplicitRemoveList, GFPSharedInstallTracker has filtered out shared dependencies
		EInstallBundleReleaseRequestFlags ReleaseFlags = GetReleaseRequestFlags() | EInstallBundleReleaseRequestFlags::ExplicitRemoveList;
		TValueOrError<FInstallBundleReleaseRequestInfo, EInstallBundleResult> MaybeRequestInfo = BundleManager->RequestReleaseContent(
			InstallBundlesToRelease, ReleaseFlags, TArrayView<const FName>(), ELogVerbosity::Verbose);

		if (MaybeRequestInfo.HasError())
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue uninstall for the PluginURL(%.*s) because %s"), ShortUrl.Len(), ShortUrl.GetData(), LexToString(MaybeRequestInfo.GetError()));
			Result = GetErrorResult(TEXT("BundleManager.Begin."), MaybeRequestInfo.GetError());

			return;
		}

		FInstallBundleReleaseRequestInfo RequestInfo = MaybeRequestInfo.StealValue();

		if (EnumHasAnyFlags(RequestInfo.InfoFlags, EInstallBundleRequestInfoFlags::SkippedUnknownBundles))
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue uninstall for the PluginURL(%.*s) because failed to resolve install bundles!"), ShortUrl.Len(), ShortUrl.GetData());
			Result = GetErrorResult(TEXT("BundleManager.Begin."), TEXT("Resolve_Failed"), UE::GameFeatures::CommonErrorCodes::GetGenericReleaseResult());

			return;
		}

		if (RequestInfo.BundlesEnqueued.Num() == 0)
		{
			bWasDeleted = true;
		}
		else
		{
			PendingBundles = MoveTemp(RequestInfo.BundlesEnqueued);
			IInstallBundleManager::ReleasedDelegate.AddRaw(this, &FBaseDataReleaseGameFeaturePluginState::OnContentRemoved);
		}
	}

	virtual void BeginState() override
	{
		BeginRemoveRequest();
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (!Result.HasValue())
		{
			StateStatus.SetTransitionError(GetFailureTransitionState(), Result);
			return;
		}

		if (!bWasDeleted)
		{
			return;
		}

		StateStatus.SetTransition(GetSuccessTransitionState());
	}

	virtual void EndState() override
	{
		CleanUp();
	}

	/** Controls what check is done to determine if this state should run or not */
	bool ShouldReleaseContent() const
	{
		switch (StateProperties.GetPluginProtocol())
		{
			case (EGameFeaturePluginProtocol::InstallBundle):
			{
				return true;
			}

			default:
			{
				return false;
			}
		}
	}

	virtual TConstArrayView<FName> GetInstallBundles()
	{
		TConstArrayView<FName> Ret;
		if (ShouldReleaseContent())
		{
			Ret = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>().InstallBundles;
		}
		return Ret;
	}

	/** Determine what kind of release request flags we submit */
	virtual EInstallBundleReleaseRequestFlags GetReleaseRequestFlags() const
	{
		// Always set ExplicitRemoveList, GFPSharedInstallTracker has filtered out shared dependencies
		EInstallBundleReleaseRequestFlags ReleaseFlags = EInstallBundleReleaseRequestFlags::ExplicitRemoveList;
		return ReleaseFlags;
	}

	/** Determines what state you transition to in the event of a success or failure to release content */
	virtual EGameFeaturePluginState GetSuccessTransitionState() const = 0;
	virtual EGameFeaturePluginState GetFailureTransitionState() const = 0;
};

struct FGameFeaturePluginState_Uninstalled : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Uninstalled] == EGameFeaturePluginState::Terminal);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Uninstalled] == EGameFeaturePluginState::CheckingStatus);
};

struct FGameFeaturePluginState_Uninstalling : public FBaseDataReleaseGameFeaturePluginState
{
	using FBaseDataReleaseGameFeaturePluginState::FBaseDataReleaseGameFeaturePluginState;

	virtual void BeginState() override
	{
		if (ShouldVisitUninstallStateBeforeTerminal())
		{
			FBaseDataReleaseGameFeaturePluginState::BeginState();
		}
		else
		{
			Result = MakeValue();
			bWasDeleted = true;
		}
	}

	virtual EGameFeaturePluginState GetSuccessTransitionState() const override
	{
		return EGameFeaturePluginState::Uninstalled;
	}
	
	virtual EGameFeaturePluginState GetFailureTransitionState() const override
	{
		return EGameFeaturePluginState::ErrorUninstalling;
	}

	//Must be overridden to determine what kind of release request we submit
	virtual EInstallBundleReleaseRequestFlags GetReleaseRequestFlags() const override
	{
		const EInstallBundleReleaseRequestFlags BaseFlags = FBaseDataReleaseGameFeaturePluginState::GetReleaseRequestFlags();
		return (BaseFlags | EInstallBundleReleaseRequestFlags::RemoveFilesIfPossible);
	}

	virtual UE::GameFeatures::FResult TryUpdateProtocolOptions(const FGameFeatureProtocolOptions& NewOptions) override
	{
		//Use base functionality to update our metadata
		UE::GameFeatures::FResult LocalResult = FGameFeaturePluginState::TryUpdateProtocolOptions(NewOptions);

		if (LocalResult.HasError())
		{
			return LocalResult;
		}

		//If we are no longer uninstalling before terminate, just exit now as a success immediately
		if (!ShouldVisitUninstallStateBeforeTerminal())
		{
			FBaseDataReleaseGameFeaturePluginState::CleanUp();

			Result = MakeValue();
			bWasDeleted = true;
			UpdateStateMachineImmediate();

			return LocalResult;
		}		
		
		//Restart our remove request to handle other changes
		BeginRemoveRequest();

		return LocalResult;
	}
};

struct FGameFeaturePluginState_Releasing : public FBaseDataReleaseGameFeaturePluginState
{
	using FBaseDataReleaseGameFeaturePluginState::FBaseDataReleaseGameFeaturePluginState;

	virtual void BeginState() override
	{
		if (ShouldReleaseContent())
		{
			UGameFeaturesSubsystem::Get().OnGameFeatureReleasing(StateProperties.PluginName, StateProperties.PluginIdentifier);
		}

		FBaseDataReleaseGameFeaturePluginState::BeginState();
	}

	virtual EGameFeaturePluginState GetSuccessTransitionState() const override
	{
		return EGameFeaturePluginState::StatusKnown;
	}

	virtual EGameFeaturePluginState GetFailureTransitionState() const override
	{
		return EGameFeaturePluginState::ErrorManagingData;
	}
};

struct FGameFeaturePluginState_Downloading : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	virtual ~FGameFeaturePluginState_Downloading()
	{
		Cleanup();
	}

	UE::GameFeatures::FResult Result = MakeValue();
	bool bSuppressResultErrorLog = false;
	bool bPluginDownloaded = false;

	TArray<FName> PendingBundleDownloads;
	TUniquePtr<FInstallBundleCombinedProgressTracker> ProgressTracker;
	FTSTicker::FDelegateHandle ProgressUpdateHandle;
	FDelegateHandle GotContentStateHandle;

	// Required for callback lifetime safety
	struct FIoStoreOnDemandContext
	{
		TArray<UE::IoStore::FOnDemandInstallRequest> InstallRequests;
		int32 PendingInstalls = 0;
		bool bStateValid = true;

		void Cancel()
		{
			for (UE::IoStore::FOnDemandInstallRequest& Request : InstallRequests)
			{
				Request.Cancel();
			}
		}
	};
	TSharedPtr<FIoStoreOnDemandContext> IoStoreOnDemandContext;

	void EnsureAllowAsyncLoading() const
	{
		ensureMsgf(AllowAsyncLoading(), TEXT("FGameFeaturePluginState::AllowAsyncLoading is false while attempting to download GFP data for %s"), *StateProperties.PluginName);
	}

	void Cleanup()
	{
		if (ProgressUpdateHandle.IsValid())
		{
			FTSTicker::RemoveTicker(ProgressUpdateHandle);
			ProgressUpdateHandle.Reset();
		}

		if (GotContentStateHandle.IsValid())
		{
			TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
			if (BundleManager)
			{
				BundleManager->CancelAllGetContentStateRequests(GotContentStateHandle);
			}
			GotContentStateHandle.Reset();
		}

		IInstallBundleManager::InstallBundleCompleteDelegate.RemoveAll(this);
		IInstallBundleManager::PausedBundleDelegate.RemoveAll(this);

		Result = MakeValue();
		bSuppressResultErrorLog = false;
		bPluginDownloaded = false;
		PendingBundleDownloads.Empty();
		ProgressTracker = nullptr;

		if (IoStoreOnDemandContext)
		{
			IoStoreOnDemandContext->Cancel();
			IoStoreOnDemandContext->bStateValid = false;
			IoStoreOnDemandContext = nullptr;
		}
	}

	void OnGotContentState(FInstallBundleCombinedContentState BundleContentState)
	{
		GotContentStateHandle.Reset();

		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();

		if (StateProperties.bTryCancel)
		{
			Result = UE::GameFeatures::CanceledResult;
			UpdateStateMachineImmediate();
			return;
		}

		const FInstallBundlePluginProtocolMetaData& MetaData = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();

		UE::GameFeatures::GFPSharedInstallTracker.AddBundleRefs(StateProperties.PluginName, UE::GameFeatures::EGFPInstallLevel::Download, MetaData.InstallBundles);

		const TConstArrayView<FName> InstallBundles = GetInstallBundles();
		const EInstallBundleRequestFlags InstallFlags = GetRequestFlags();
		TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> MaybeRequestInfo = BundleManager->RequestUpdateContent(InstallBundles, InstallFlags);

		if (MaybeRequestInfo.HasError())
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue download for the PluginURL(%.*s) because %s"), ShortUrl.Len(), ShortUrl.GetData(), LexToString(MaybeRequestInfo.GetError()));
			Result = GetErrorResult(TEXT("BundleManager.GotState."), LexToString(MaybeRequestInfo.GetError()));
			UpdateStateMachineImmediate();
			return;
		}

		FInstallBundleRequestInfo RequestInfo = MaybeRequestInfo.StealValue();

		if (EnumHasAnyFlags(RequestInfo.InfoFlags, EInstallBundleRequestInfoFlags::SkippedUnknownBundles))
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue download for the PluginURL(%.*s) because failed to resolve install bundles!"), ShortUrl.Len(), ShortUrl.GetData());
			Result = GetErrorResult(TEXT("BundleManager.GotState."), TEXT("Resolve_Failed"), UE::GameFeatures::CommonErrorCodes::GetGenericConnectionError());

			UpdateStateMachineImmediate();
			return;
		}

		if (RequestInfo.BundlesEnqueued.Num() == 0)
		{
			UpdateProgress(1.0f);
			InstallIoStoreOnDemandContent();
		}
		else
		{
			EnsureAllowAsyncLoading();
			PendingBundleDownloads = MoveTemp(RequestInfo.BundlesEnqueued);
			IInstallBundleManager::InstallBundleCompleteDelegate.AddRaw(this, &FGameFeaturePluginState_Downloading::OnInstallBundleCompleted);
			IInstallBundleManager::PausedBundleDelegate.AddRaw(this, &FGameFeaturePluginState_Downloading::OnInstallBundlePaused);

			ProgressTracker = MakeUnique<FInstallBundleCombinedProgressTracker>(false);
			ProgressTracker->SetBundlesToTrackFromContentState(BundleContentState, PendingBundleDownloads);

			ProgressUpdateHandle = FTSTicker::GetCoreTicker().AddTicker(
				FTickerDelegate::CreateRaw(this, &FGameFeaturePluginState_Downloading::OnUpdateProgress)/*, 0.1f*/);

			//If this setting is flipped then we should immediately request to pause downloads.
			//We still generate the downloads so that we have an accurate PendingBundleDownloads list
			const FInstallBundlePluginProtocolOptions& Options = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();
			if (Options.bUserPauseDownload)
			{
				ChangePauseState(true);
			}
		}
	}

	void OnInstallBundleCompleted(FInstallBundleRequestResultInfo BundleResult)
	{
		if (!PendingBundleDownloads.Contains(BundleResult.BundleName))
		{
			return;
		}

		PendingBundleDownloads.Remove(BundleResult.BundleName);

		if (!Result.HasError() && BundleResult.Result != EInstallBundleResult::OK)
		{
			//Use OptionalErrorCode and/or OptionalErrorText if available
			const FString ErrorCodeEnding = (BundleResult.OptionalErrorCode.IsEmpty()) ? LexToString(BundleResult.Result) : BundleResult.OptionalErrorCode;
			const FText ErrorText = BundleResult.OptionalErrorCode.IsEmpty() ? UE::GameFeatures::CommonErrorCodes::GetErrorTextForBundleResult(BundleResult.Result) : BundleResult.OptionalErrorText;

			Result = GetErrorResult(TEXT("BundleManager.OnComplete."), ErrorCodeEnding, ErrorText);

			if (BundleResult.Result != EInstallBundleResult::UserCancelledError)
			{
				TryCancelState();
			}
		}

		if (PendingBundleDownloads.Num() > 0)
		{
			return;
		}

		OnUpdateProgress(0.0f);

		if (Result.HasError())
		{
			UpdateStateMachineImmediate();
			return;
		}

		InstallIoStoreOnDemandContent();
	}

	void InstallIoStoreOnDemandContent()
	{
		const FInstallBundlePluginProtocolMetaData& MetaData = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();
		if (MetaData.InstallBundlesWithAssetDependencies.IsEmpty())
		{
			bPluginDownloaded = Result.HasValue();
			UpdateStateMachineImmediate();
			return;
		}

		UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
		if (!IoStore && !Result.HasError())
		{
			Result = GetErrorResult(TEXT("IoStoreOnDemand.ModuleNotFound"));
			UpdateStateMachineImmediate();
			return;
		}

		const FInstallBundlePluginProtocolOptions& Options = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();

		IoStoreOnDemandContext = MakeShared<FIoStoreOnDemandContext>();
		IoStoreOnDemandContext->PendingInstalls = MetaData.InstallBundlesWithAssetDependencies.Num();

#if !UE_BUILD_SHIPPING
		static bool bBreakOnInstallBundlesWithAssetDependencies = true;
		if (bBreakOnInstallBundlesWithAssetDependencies && MetaData.InstallBundlesWithAssetDependencies.IsEmpty())
		{
			static int i = 0;
			++i;
		}
#endif
		for (const FName InstallBundle : MetaData.InstallBundlesWithAssetDependencies)
		{
			UE::IoStore::FOnDemandInstallArgs InstallArgs;
			InstallArgs.MountId = InstallBundle.ToString();
			InstallArgs.TagSets.Emplace(TEXTVIEW("mount")); // May not be a real tagset, but this will install all the mandatory untagged chunks
			InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::InstallSoftReferences;
			InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::CallbackOnGameThread;
			if (Options.bDoNotDownload && !UE::GameFeatures::CVarForceGFPIADDownload.GetValueOnGameThread())
			{
				InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::DoNotDownload;
			}
			
			InstallArgs.ContentHandle = UE::GameFeatures::GFPSharedInstallTracker.AddOnDemandContentHandle(
				InstallBundle, UE::GameFeatures::EGFPInstallLevel::Download);
			InstallArgs.DebugName = StateProperties.PluginName;
			check(InstallArgs.ContentHandle.IsValid());

			// This should be pretty small, so not going to worry about progress here.
			IoStoreOnDemandContext->InstallRequests.Add(
				IoStore->Install(MoveTemp(InstallArgs),
				[this, LambdaOnDemandContext = IoStoreOnDemandContext](UE::IoStore::FOnDemandInstallResult&& OnDemandInstallResult)
				{
					if (!LambdaOnDemandContext->bStateValid)
					{
						// Owning state got cleaned up, bail
						return;
					}

					if (!OnDemandInstallResult.IsOk() && !Result.HasError())
					{
						const FText ErrorMessage	= OnDemandInstallResult.Error.GetValue().GetErrorMessage();
						FString ErrorCode			= FString(OnDemandInstallResult.Error.GetValue().GetModuleIdAndErrorCodeString());

						ErrorCode.ReplaceCharInline(TEXT(' '), TEXT('_'), ESearchCase::CaseSensitive);
						Result = GetErrorResult(TEXT("IoStoreOnDemand.OnComplete."), ErrorCode, FText::AsCultureInvariant(ErrorMessage));
						TryCancelState();
					}

					--IoStoreOnDemandContext->PendingInstalls;
					if (IoStoreOnDemandContext->PendingInstalls == 0)
					{
						bPluginDownloaded = Result.HasValue();
						UpdateStateMachineImmediate();
					}
				}));
		}
	}

	bool OnUpdateProgress(float dts)
	{
		if (ProgressTracker)
		{
			ProgressTracker->ForceTick();

			float Progress = ProgressTracker->GetCurrentCombinedProgress().ProgressPercent;
			UpdateProgress(Progress);

			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			UE_LOGF(LogGameFeatures, VeryVerbose, "Download Progress: %f for PluginURL(%.*ls)", Progress, ShortUrl.Len(), ShortUrl.GetData());
		}

		return true;
	}

	void ChangePauseState(bool bPause)
	{
		if (PendingBundleDownloads.Num() > 0)
		{
			TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();

			if (bPause)
			{
				BundleManager->PauseUpdateContent(PendingBundleDownloads);
			}
			else
			{
				BundleManager->ResumeUpdateContent(PendingBundleDownloads);
			}
			BundleManager->RequestPausedBundleCallback();

			//Use same text we use for InstallBundleManager's UserPaused as this was also a user pause
			const TCHAR* PauseReason = InstallBundleUtil::GetInstallBundlePauseReason(EInstallBundlePauseFlags::UserPaused);

			NotifyPauseChange(bPause, PauseReason);
		}
	}

	void OnInstallBundlePaused(FInstallBundlePauseInfo InPauseBundleInfo)
	{
		if (PendingBundleDownloads.Contains(InPauseBundleInfo.BundleName))
		{
			const bool bIsPaused = (InPauseBundleInfo.PauseFlags != EInstallBundlePauseFlags::None);
			const TCHAR* PauseReason = InstallBundleUtil::GetInstallBundlePauseReason(InPauseBundleInfo.PauseFlags);

			NotifyPauseChange(bIsPaused, PauseReason);
		}
	}

	void NotifyPauseChange(bool bIsPaused, FString PauseReason)
	{
		FGameFeaturePauseStateChangeContext Context(UE::GameFeatures::ToString(EGameFeaturePluginState::Downloading), PauseReason, bIsPaused);
		UGameFeaturesSubsystem::Get().OnGameFeaturePauseChange(StateProperties.PluginIdentifier, StateProperties.PluginName, Context);
	}

	virtual void BeginState() override
	{
		Cleanup();

		if ((StateProperties.GetPluginProtocol() == EGameFeaturePluginProtocol::File) 
		|| !ensure(ShouldDownloadContent()))
		{
			bPluginDownloaded = true;
			UpdateProgress(1.0f);
			return;
		}

		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
		const TConstArrayView<FName> InstallBundles = GetInstallBundles();

		const FInstallBundlePluginProtocolOptions& Options = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();

		const bool bAddDependencies = false; // We already got all dependencies in the CheckingStatus state

		TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> MaybeInstallState =
			BundleManager->GetInstallStateSynchronous(InstallBundles, bAddDependencies);
		check(MaybeInstallState.HasValue());
		FInstallBundleCombinedInstallState& InstallState = MaybeInstallState.GetValue();

		const bool bAllUpToDate = InstallState.GetAllBundlesHaveState(EInstallBundleInstallState::UpToDate);

		// Handle bDoNotDownload flag before doing any async ops
		// if not up to date, check to see if we allow downloading
		if ((Options.bDoNotDownload && !UE::GameFeatures::CVarForceGFPIADDownload.GetValueOnGameThread()) && !bAllUpToDate)
		{
			Result = GetErrorResult(TEXT("GFPStateMachine.DownloadNotAllowed"));
			bSuppressResultErrorLog = true; // Don't log an error if the user disallowed the download
			UpdateStateMachineImmediate();
			return;
		}

		UGameFeaturesSubsystem::Get().OnGameFeatureDownloading(StateProperties.PluginName, StateProperties.PluginIdentifier);

		if (!bAllUpToDate && InstallBundles.Num() > 1)
		{
			EnsureAllowAsyncLoading();
			GotContentStateHandle = BundleManager->GetContentState(InstallBundles, EInstallBundleGetContentStateFlags::None, bAddDependencies,
				FInstallBundleGetContentStateDelegate::CreateRaw(this, &FGameFeaturePluginState_Downloading::OnGotContentState));
		}
		else
		{
			// Only if all bundles up to date or only one bundle:
			// We only care about relative weighting here, so we don't need any of the other content state metadata.
			// We can just assume the weight is 1.0 and not have to wait for the full async call to get the rest of the metadata.
			FInstallBundleCombinedContentState HackContentState;
			HackContentState.IndividualBundleStates.Reserve(InstallState.IndividualBundleStates.Num());
			for (const TPair<FName, EInstallBundleInstallState>& Pair : InstallState.IndividualBundleStates)
			{
				FInstallBundleContentState& BundleContentState = HackContentState.IndividualBundleStates.Emplace(Pair.Key);
				BundleContentState.State = Pair.Value;
				BundleContentState.Weight = 1.0f;
			}
			HackContentState.BundlesWithIoStoreOnDemand = MoveTemp(InstallState.BundlesWithIoStoreOnDemand);
			OnGotContentState(MoveTemp(HackContentState));
		}
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (!Result.HasValue())
		{
			const EGameFeaturePluginState FailState = GetFailureTransitionState();
			StateStatus.SetTransitionError(FailState, Result, bSuppressResultErrorLog);
			return;
		}

		if (!bPluginDownloaded)
		{
			return;
		}

		const EGameFeaturePluginState SuccessState = GetSuccessTransitionState();
		StateStatus.SetTransition(SuccessState);
	}

	virtual void TryCancelState() override
	{
		if (PendingBundleDownloads.Num() > 0)
		{
			TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
			BundleManager->CancelUpdateContent(PendingBundleDownloads);
			if (IoStoreOnDemandContext)
			{
				IoStoreOnDemandContext->Cancel();
			}
		}
	}

	virtual UE::GameFeatures::FResult TryUpdateProtocolOptions(const FGameFeatureProtocolOptions& NewOptions) override
	{
		//Need to update our BundleFlags for any bundles we are downloading
		EInstallBundleRequestFlags OldRequestFlags;
		bool OldUserPausedFlag;
		{
			const FInstallBundlePluginProtocolOptions& OldOptions = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();
			OldRequestFlags = OldOptions.InstallBundleFlags;
			OldUserPausedFlag = OldOptions.bUserPauseDownload;
		}

		UE::GameFeatures::FResult OptionsResult = FGameFeaturePluginState::TryUpdateProtocolOptions(NewOptions);
		if (OptionsResult.HasError())
		{
			return OptionsResult;
		}

		//if we don't have any in-progress downloads the default behavior is all we need
		if (PendingBundleDownloads.Num() == 0)
		{
			return OptionsResult;
		}

		const FInstallBundlePluginProtocolOptions& Options = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();

		//Update our InstallBundleRequestFlags
		{
			EInstallBundleRequestFlags UpdatedRequestFlags = Options.InstallBundleFlags;

			EInstallBundleRequestFlags AddFlags = (UpdatedRequestFlags & (~OldRequestFlags));
			EInstallBundleRequestFlags RemoveFlags = ((~UpdatedRequestFlags) & OldRequestFlags);

			if ((AddFlags != EInstallBundleRequestFlags::None) || (RemoveFlags != EInstallBundleRequestFlags::None))
			{
				TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
				BundleManager->UpdateContentRequestFlags(PendingBundleDownloads, AddFlags, RemoveFlags);
			}
		}

		//Handle pausing or resuming the download if the bUserPauseDownload flag has changed
		{
			if (Options.bUserPauseDownload != OldUserPausedFlag)
			{
				ChangePauseState(Options.bUserPauseDownload);
			}
		}

		return OptionsResult;
	}

	virtual void EndState() override
	{
		Cleanup();
	}

	/** Controls what check is done to determine if this state should run or not */
	bool ShouldDownloadContent() const
	{
		switch (StateProperties.GetPluginProtocol())
		{
		case (EGameFeaturePluginProtocol::InstallBundle):
			return true;
		default:
			return false;
		}
	}

	virtual TConstArrayView<FName> GetInstallBundles()
	{
		TConstArrayView<FName> Ret;
		if (ShouldDownloadContent())
		{
			Ret = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>().InstallBundles;
		}
		return Ret;
	}

	/** Determine what kind of request flags we submit */
	EInstallBundleRequestFlags GetRequestFlags() const
	{
		//Pull our InstallFlags from the Options, but also make sure SkipMount is set as there is a separate mounting step that will re-request this
		//without SkipMount and then mount the data, this allows us to pre-download data without mounting it
		EInstallBundleRequestFlags InstallFlags = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>().InstallBundleFlags;
		InstallFlags |= EInstallBundleRequestFlags::SkipMount;
		InstallFlags |= EInstallBundleRequestFlags::ExplicitUpdateList; // avoid chasing depends for the install bundles
		return InstallFlags;
	}

	/** Determines what state you transition to in the event of a success or failure to release content */
	EGameFeaturePluginState GetSuccessTransitionState() const
	{
		return EGameFeaturePluginState::Installed;
	}

	EGameFeaturePluginState GetFailureTransitionState() const
	{
		return EGameFeaturePluginState::ErrorManagingData;
	}
};

struct FGameFeaturePluginState_Installed : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Installed] == EGameFeaturePluginState::Mounting);
	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Installed] == EGameFeaturePluginState::Releasing);
};

struct FGameFeaturePluginState_ErrorMounting : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorMounting] == EGameFeaturePluginState::Unmounting);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorMounting] == EGameFeaturePluginState::Mounting);
};

struct FGameFeaturePluginState_ErrorWaitingForDependencies : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorWaitingForDependencies] == EGameFeaturePluginState::Unmounting);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorWaitingForDependencies] == EGameFeaturePluginState::WaitingForDependencies);

};

struct FGameFeaturePluginState_ErrorRegistering : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorRegistering] == EGameFeaturePluginState::Unregistering);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorRegistering] == EGameFeaturePluginState::Registering);
};

struct FGameFeaturePluginState_Unmounting : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	UE::GameFeatures::FResult Result = MakeValue();
	TArray<FName> PendingBundles;
	bool bUnmounting = false;
	bool bUnmounted = false;
	bool bCheckedRealtimeMode = false;

	void Unmount()
	{
		if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(StateProperties.PluginName);
			Plugin && Plugin->GetDescriptor().bExplicitlyLoaded)
		{
			if (!UE::GameFeatures::ShouldDeferLocalizationDataLoad())
			{
				IPluginManager::Get().UnmountExplicitlyLoadedPluginLocalizationData(StateProperties.PluginName);
			}

#if UE_MERGED_MODULES
			// We normally do not allow unloading code because it's difficult to make that safe.
			// For example, destructors can be called much later than we'd expect because of garbage collection or async work.
			// We allow this behavior for merged modular builds because the have a smaller scope (intended for console clients) and
			// unloading code to save memory is the main goal for that feature.
			static constexpr bool bAllowUnloadCode = true;
#else
			static constexpr bool bAllowUnloadCode = false;
#endif // UE_MERGED_MODULES

			// The asset registry listens to FPackageName::OnContentPathDismounted() and 
			// will automatically cleanup the asset registry state we added for this plugin.
			// This will also cause any assets we added to the asset manager to be removed.
			// Scan paths added to the asset manager should have already been cleaned up.
			FText FailureReason;
			if (!IPluginManager::Get().UnmountExplicitlyLoadedPlugin(StateProperties.PluginName, &FailureReason, bAllowUnloadCode))
			{
				const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
				ensureMsgf(false, TEXT("Failed to explicitly unmount the PluginURL(%.*s) because %s"), ShortUrl.Len(), ShortUrl.GetData(), *FailureReason.ToString());
				Result = GetErrorResult(TEXT("Plugin_Cannot_Explicitly_Unmount"));
				return;
			}
		}

		if (StateProperties.bAddedPluginToManager)
		{
			verify(IPluginManager::Get().RemoveFromPluginsList(StateProperties.PluginInstalledFilename));
			StateProperties.bAddedPluginToManager = false;
		}

		if (StateProperties.GetPluginProtocol() != EGameFeaturePluginProtocol::InstallBundle)
		{
			bUnmounted = true;
			return;
		}

		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
		check(BundleManager.IsValid());

		const TArray<FName>& InstallBundles = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>().InstallBundles;

		TArray<FName> InstallBundlesToRelease = UE::GameFeatures::GFPSharedInstallTracker.Release(
			StateProperties.PluginName, UE::GameFeatures::EGFPInstallLevel::Mount, InstallBundles);

		EInstallBundleReleaseRequestFlags ReleaseFlags = 
			EInstallBundleReleaseRequestFlags::SkipReleaseUnmountOnly |
			EInstallBundleReleaseRequestFlags::ExplicitRemoveList; // Always set ExplicitRemoveList, GFPSharedInstallTracker has filtered out shared dependencies
		TValueOrError<FInstallBundleReleaseRequestInfo, EInstallBundleResult> MaybeRequestInfo = BundleManager->RequestReleaseContent(
			InstallBundlesToRelease, ReleaseFlags, TArrayView<const FName>(), ELogVerbosity::Verbose);

		if (MaybeRequestInfo.HasError())
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue unmount for the PluginURL(%.*s) because %s"), ShortUrl.Len(), ShortUrl.GetData(), LexToString(MaybeRequestInfo.GetError()));
			Result = GetErrorResult(TEXT("BundleManager.Begin."), MaybeRequestInfo.GetError());
			return;
		}

		FInstallBundleReleaseRequestInfo RequestInfo = MaybeRequestInfo.StealValue();

		if (EnumHasAnyFlags(RequestInfo.InfoFlags, EInstallBundleRequestInfoFlags::SkippedUnknownBundles))
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue unmount for the PluginURL(%.*s) because failed to resolve install bundles!"), ShortUrl.Len(), ShortUrl.GetData());
			Result = GetErrorResult(TEXT("BundleManager.Begin."), TEXT("Cannot_Resolve"), UE::GameFeatures::CommonErrorCodes::GetGenericConnectionError());
			return;
		}

		if (RequestInfo.BundlesEnqueued.Num() == 0)
		{
			bUnmounted = true;
		}
		else
		{
			PendingBundles = MoveTemp(RequestInfo.BundlesEnqueued);
			IInstallBundleManager::ReleasedDelegate.AddRaw(this, &FGameFeaturePluginState_Unmounting::OnContentReleased);
		}
	}

	void OnContentReleased(FInstallBundleReleaseRequestResultInfo BundleResult)
	{
		if (!PendingBundles.Contains(BundleResult.BundleName))
		{
			return;
		}

		PendingBundles.Remove(BundleResult.BundleName);

		if (!Result.HasError() && BundleResult.Result != EInstallBundleReleaseResult::OK)
		{
			Result = GetErrorResult(TEXT("BundleManager.OnReleased."), BundleResult.Result);
		}

		if (PendingBundles.Num() > 0)
		{
			return;
		}

		if (Result.HasValue())
		{
			bUnmounted = true;
		}

		UpdateStateMachineImmediate();
	}

	virtual void BeginState() override
	{
		Result = MakeValue();
		PendingBundles.Empty();
		bUnmounting = false;
		bUnmounted = false;
		bCheckedRealtimeMode = false;
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (!bCheckedRealtimeMode)
		{
			bCheckedRealtimeMode = true;
			if (UE::GameFeatures::RealtimeMode)
			{
				UE::GameFeatures::RealtimeMode->AddUpdateRequest(StateProperties);
				return;
			}
		}

		if (!bUnmounting)
		{
			bUnmounting = true;
			Unmount();
		}

		if (!Result.HasValue())
		{
			StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorMounting, Result);
			return;
		}

		if (!bUnmounted)
		{
			return;
		}

		static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Unmounting] == EGameFeaturePluginState::Installed);
		static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Unmounting] == EGameFeaturePluginState::Installed);
		StateStatus.SetTransition(EGameFeaturePluginState::Installed);
	}

	virtual void EndState() override
	{
		IInstallBundleManager::ReleasedDelegate.RemoveAll(this);
	}
};

struct FGameFeaturePluginState_Mounting : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	enum class ESubState : uint8
	{
		None = 0,
		MountPlugin,
		LoadAssetRegistry
	};
	FRIEND_ENUM_CLASS_FLAGS(ESubState);

	int32 NumObservedPostMountPausers = 0;
	int32 NumExpectedPostMountPausers = 0;
	TArray<FName> PendingBundles;
	FDelegateHandle PakFileMountedDelegateHandle;
	UE::GameFeatures::FResult Result = MakeValue();
	ESubState StartedSubStates = ESubState::None;
	ESubState CompletedSubStates = ESubState::None;
	bool bCheckedRealtimeMode = false;

	static UE::Tasks::FPipe ShaderlibPipe;

	void OnInstallBundleCompleted(FInstallBundleRequestResultInfo BundleResult)
	{
		if (!PendingBundles.Contains(BundleResult.BundleName))
		{
			return;
		}

		PendingBundles.Remove(BundleResult.BundleName);

		if (!Result.HasError() && BundleResult.Result != EInstallBundleResult::OK)
		{
			if (BundleResult.OptionalErrorCode.IsEmpty())
			{
				Result = GetErrorResult(TEXT("BundleManager.OnComplete."), BundleResult.Result);
			}
			else
			{
				Result = GetErrorResult(TEXT("BundleManager.OnComplete."), BundleResult.OptionalErrorCode, BundleResult.OptionalErrorText);
			}
		}

		if (PendingBundles.IsEmpty())
		{
			IInstallBundleManager::InstallBundleCompleteDelegate.RemoveAll(this);
			if (PakFileMountedDelegateHandle.IsValid())
			{
				FCoreDelegates::GetOnPakFileMounted2().Remove(PakFileMountedDelegateHandle);
				PakFileMountedDelegateHandle.Reset();
			}
			
			UpdateStateMachineImmediate();
		}
	}

	void OnPakFileMounted(const IPakFile& PakFile)
	{
		if (FPakFile* Pak = (FPakFile*)(&PakFile))
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			UE_LOGF(LogGameFeatures, Display, "Mounted Pak File for (%.*ls) with following files:", ShortUrl.Len(), ShortUrl.GetData());
			TArray<FString> OutFileList;
			Pak->GetPrunedFilenames(OutFileList);
			for (const FString& FileName : OutFileList)
			{
				UE_LOGF(LogGameFeatures, Display, "(%ls)", *FileName);
			}
		}
	}

	void OnPostMountPauserCompleted(FStringView InPauserTag)
	{
		check(IsInGameThread());
		ensure(NumExpectedPostMountPausers != INDEX_NONE);
		++NumObservedPostMountPausers;

		UE_LOGF(LogGameFeatures, Display, "Post-mount of %ls resumed by %.*ls", *StateProperties.PluginName, InPauserTag.Len(), InPauserTag.GetData());

		if (NumObservedPostMountPausers == NumExpectedPostMountPausers)
		{
			UpdateStateMachineImmediate();
		}
	}

	virtual bool UseAsyncLoading() const override
	{
		if (UE::GameFeatures::CVarForceSyncRegisterStartupPlugins.GetValueOnGameThread())
		{
			if (UGameFeaturesSubsystem::Get().GetPolicy().IsLoadingStartupPlugins())
			{
				return false;
			}
		}

		return FGameFeaturePluginState::UseAsyncLoading();
	}

	virtual void BeginState() override
	{
		NumObservedPostMountPausers = 0;
		NumExpectedPostMountPausers = 0;
		PendingBundles.Empty();
		PakFileMountedDelegateHandle.Reset();
		Result = MakeValue();
		StartedSubStates = ESubState::None;
		CompletedSubStates = ESubState::None;
		bCheckedRealtimeMode = false;

		if (StateProperties.GetPluginProtocol() != EGameFeaturePluginProtocol::InstallBundle)
		{
			return;
		}

		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();

		const FInstallBundlePluginProtocolMetaData& MetaData = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();
		const TArray<FName>& InstallBundles = MetaData.InstallBundles;

		UE::GameFeatures::GFPSharedInstallTracker.AddBundleRefs(StateProperties.PluginName, UE::GameFeatures::EGFPInstallLevel::Mount, InstallBundles);

		const FInstallBundlePluginProtocolOptions& Options = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();
		EInstallBundleRequestFlags InstallFlags = UseAsyncLoading() ?
			(Options.InstallBundleFlags | EInstallBundleRequestFlags::AsyncMount) : Options.InstallBundleFlags;
		InstallFlags |= EInstallBundleRequestFlags::ExplicitUpdateList; // skip over collecting all the depends here, as we do this in CheckingStatus state

		// Make bundle manager use verbose log level for most logs.
		// We are already done with downloading, so we don't care about logging too much here unless mounting fails.
		const ELogVerbosity::Type InstallBundleManagerVerbosityOverride = ELogVerbosity::Verbose;
		TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> MaybeRequestInfo = BundleManager->RequestUpdateContent(InstallBundles, InstallFlags, InstallBundleManagerVerbosityOverride);

		if (MaybeRequestInfo.HasError())
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue mount for the PluginURL(%.*s) because %s"), ShortUrl.Len(), ShortUrl.GetData(), LexToString(MaybeRequestInfo.GetError()));
			Result = GetErrorResult(TEXT("BundleManager.Begin.CannotStart."), MaybeRequestInfo.GetError());
			return;
		}

		FInstallBundleRequestInfo RequestInfo = MaybeRequestInfo.StealValue();

		if (EnumHasAnyFlags(RequestInfo.InfoFlags, EInstallBundleRequestInfoFlags::SkippedUnknownBundles))
		{
			const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
			ensureMsgf(false, TEXT("Unable to enqueue mount for the PluginURL(%.*s) because failed to resolve install bundles!"), ShortUrl.Len(), ShortUrl.GetData());
			Result = GetErrorResult(TEXT("BundleManager.Begin."), TEXT("Resolve_Failed"));
			return;
		}

		if (RequestInfo.BundlesEnqueued.Num() > 0)
		{
			PendingBundles = MoveTemp(RequestInfo.BundlesEnqueued);
			IInstallBundleManager::InstallBundleCompleteDelegate.AddRaw(this, &FGameFeaturePluginState_Mounting::OnInstallBundleCompleted);
			if (UE::GameFeatures::ShouldLogMountedFiles)
			{
				// Track with a delegate handle to avoid unbinding if we don't use this. Unbinding this causes an occaisonal perf spike.
				PakFileMountedDelegateHandle = FCoreDelegates::GetOnPakFileMounted2().AddRaw(this, &FGameFeaturePluginState_Mounting::OnPakFileMounted);
			}
		}
	}

	void UpdateState_MountPlugin(bool bLoadPluginIniHierarchy)
	{
		if (EnumHasAnyFlags(StartedSubStates, ESubState::MountPlugin))
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Mounting_Plugin);

		StartedSubStates |= ESubState::MountPlugin;

		if (Result.HasError())
		{
			CompletedSubStates |= ESubState::MountPlugin;
			return;
		}

		// Pre-mount
		// Normally the shader library itself listens to a "New Plugin mounted" (and "New Pakfile mounted") callback and the library is opened automatically. This switch governs whether the manual behavior is wanted.
		bool bManuallyOpenPluginShaderLibrary = true;
		{
			FGameFeaturePreMountingContext Context;
			UGameFeaturesSubsystem::Get().OnGameFeaturePreMounting(StateProperties.PluginName, StateProperties.PluginIdentifier, Context);
			bManuallyOpenPluginShaderLibrary = Context.bOpenPluginShaderLibrary;
		}

		checkf(!StateProperties.PluginInstalledFilename.IsEmpty(), TEXT("PluginInstalledFilename must be set by the Mounting. PluginURL: %s"), *StateProperties.PluginIdentifier.GetFullPluginURL());
		checkf(FPaths::GetExtension(StateProperties.PluginInstalledFilename) == TEXT("uplugin"), TEXT("PluginInstalledFilename must have a uplugin extension. PluginURL: %s"), *StateProperties.PluginIdentifier.GetFullPluginURL());

		// refresh the plugins list to let the plugin manager know about it
		const TSharedPtr<IPlugin> MaybePlugin = IPluginManager::Get().FindPlugin(StateProperties.PluginName);
		const bool bNeedsPluginMount = (MaybePlugin == nullptr || MaybePlugin->GetDescriptor().bExplicitlyLoaded);

		if (MaybePlugin)
		{
			if (!FPaths::IsSamePath(MaybePlugin->GetDescriptorFileName(), StateProperties.PluginInstalledFilename))
			{
				Result = GetErrorResult(TEXT("Plugin_Name_Already_In_Use"));
			}
		}
		else
		{
			const bool bAddedPlugin = IPluginManager::Get().AddToPluginsList(StateProperties.PluginInstalledFilename);
			if (bAddedPlugin)
			{
				StateProperties.bAddedPluginToManager = true;
			}
			else
			{
				Result = GetErrorResult(TEXT("Failed_To_Register_Plugin"));
			}
		}

		// now load ini files if desired, now that we know the plugin has been loaded
		if (bLoadPluginIniHierarchy)
		{
			UGameFeatureData::InitializeBasePluginIniFile(StateProperties.PluginInstalledFilename);
		}


		if (Result.HasError() || !bNeedsPluginMount)
		{
			CompletedSubStates |= ESubState::MountPlugin;
			return;
		}

		if (bManuallyOpenPluginShaderLibrary)
		{
			// We want to control opening the shader lib
			FShaderCodeLibrary::DontOpenPluginShaderLibraryOnMount(StateProperties.PluginName);
		}

		if (!UseAsyncLoading() || UE::GameFeatures::CVarForceSyncLoadShaderLibrary.GetValueOnGameThread())
		{
			verify(IPluginManager::Get().MountExplicitlyLoadedPlugin(StateProperties.PluginName));
			if (!UE::GameFeatures::ShouldDeferLocalizationDataLoad())
			{
				UGameFeaturePluginStateMachine* CurrentMachine = UGameFeaturesSubsystem::Get().FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier);
				UE::GameFeatures::MountLocalizationData(CurrentMachine, StateProperties);
			}
			if (bManuallyOpenPluginShaderLibrary)
			{
				TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(StateProperties.PluginName);
				FShaderCodeLibrary::OpenPluginShaderLibrary(*Plugin);
			}
			CompletedSubStates |= ESubState::MountPlugin;
			return;
		}

		verify(IPluginManager::Get().MountExplicitlyLoadedPlugin(StateProperties.PluginName));
		if (!UE::GameFeatures::ShouldDeferLocalizationDataLoad())
		{
			UGameFeaturePluginStateMachine* CurrentMachine = UGameFeaturesSubsystem::Get().FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier);
			UE::GameFeatures::MountLocalizationData(CurrentMachine, StateProperties);
		}

		// Now load the shader lib in the background
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(StateProperties.PluginName);
		if (bManuallyOpenPluginShaderLibrary && Plugin->CanContainContent() && Plugin->IsEnabled())
		{
			// TEMP HACK - use a pipe because if this goes too wide we can end up blocking all available tasks.
			ShaderlibPipe.Launch(UE_SOURCE_LOCATION, [this, Plugin]
			{
				FShaderCodeLibrary::OpenPluginShaderLibrary(*Plugin);

				ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]
				{
					CompletedSubStates |= ESubState::MountPlugin;
					UpdateStateMachineImmediate();
				});

			}, UE::Tasks::ETaskPriority::BackgroundHigh);

			return;
		}

		CompletedSubStates |= ESubState::MountPlugin;
	}

	void UpdateState_LoadAssetRegistry()
	{
		if (EnumHasAnyFlags(StartedSubStates, ESubState::LoadAssetRegistry))
		{
			return;
		}

		StartedSubStates |= ESubState::LoadAssetRegistry;

		if (Result.HasError())
		{
			CompletedSubStates |= ESubState::LoadAssetRegistry;
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Mounting_AR);

		// After the new plugin is mounted add the asset registry for that plugin.
		TSharedPtr<IPlugin> NewlyMountedPlugin = IPluginManager::Get().FindPlugin(StateProperties.PluginName);
		if (!NewlyMountedPlugin || !NewlyMountedPlugin->CanContainContent())
		{
			CompletedSubStates |= ESubState::LoadAssetRegistry;
			return;
		}

		FString PluginAssetRegistry;
		{
			const FString PluginFolder = FPaths::GetPath(StateProperties.PluginInstalledFilename);
			TArray<FString> PluginAssetRegistrySearchPaths;
			FPakPlatformFile* PakPlatformFile = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(FPakPlatformFile::GetTypeName()));
			// For GFPs cooked as DLC
			PluginAssetRegistrySearchPaths.Add(PluginFolder / TEXT("AssetRegistry.bin"));
			// For GFPs with a unique chunk
			PluginAssetRegistrySearchPaths.Add(FPaths::ProjectDir() / FString::Printf(TEXT("AssetRegistry_GFP_%s.bin"), *StateProperties.PluginName));
			for (FString& Path : PluginAssetRegistrySearchPaths)
			{
				// Optimization: if we're using pak files then only search paks (avoid unnecessary fallback to loose)
				bool bFileExists = PakPlatformFile != nullptr ? PakPlatformFile->FindFileInPakFiles(*Path) : IFileManager::Get().FileExists(*Path);
				if (bFileExists)
				{
					PluginAssetRegistry = MoveTemp(Path);
					break;
				}
			}

			if (PluginAssetRegistry.IsEmpty())
			{
				CompletedSubStates |= ESubState::LoadAssetRegistry;
				return;
			}
		}

		auto RefreshPackageLocalizationCacheForPlugin = [NewlyMountedPlugin]()
		{
			// We need to refresh the package localization cache for a GFP if it loaded cooked asset registry state, 
			// as we need the asset registry data to correctly build the package localization cache for the GFP
			if (NewlyMountedPlugin && NewlyMountedPlugin->CanContainContent())
			{
				FPackageLocalizationManager::Get().InvalidateRootSourcePath(NewlyMountedPlugin->GetMountedAssetPath());
			}
		};

		if (!UseAsyncLoading())
		{
			IAssetRegistry& AssetRegistry = UAssetManager::Get().GetAssetRegistry();
			if (AssetRegistry.ActiveMountsRegisterAndLoadFilePathSynchronous(PluginAssetRegistry))
			{
				RefreshPackageLocalizationCacheForPlugin();
			}
			else
			{
				Result = GetErrorResult(TEXT("Failed_To_Load_Plugin_AssetRegistry"));
			}

			CompletedSubStates |= ESubState::LoadAssetRegistry;
			return;
		}

		const bool bForceSyncAssetRegistryAppend = UE::GameFeatures::CVarForceSyncAssetRegistryAppend.GetValueOnGameThread();
		if (bForceSyncAssetRegistryAppend)
		{
			// ActiveMountsRegisterAndLoadFilePath does not support bForceSyncAssetRegistryAppend; we have to load the AssetRegistry file
			// ourselves into an FAssetRegistryState to be able to append it on the game thread.
			UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, PluginAssetRegistry=MoveTemp(PluginAssetRegistry), RefreshPackageLocalizationCacheForPlugin]
			{
				TSharedPtr<FAssetRegistryState> PluginAssetRegistryState = MakeShared<FAssetRegistryState>();
				bool bSuccess = FAssetRegistryState::LoadFromDisk(*PluginAssetRegistry, FAssetRegistryLoadOptions(),
					*PluginAssetRegistryState);

				ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, PluginAssetRegistryState, bSuccess, RefreshPackageLocalizationCacheForPlugin]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Mounting_ARComplete);

					if (!bSuccess)
					{
						Result = GetErrorResult(TEXT("Failed_To_Load_Plugin_AssetRegistry"));
					}
					else
					{
						IAssetRegistry& AssetRegistry = UAssetManager::Get().GetAssetRegistry();
						AssetRegistry.AppendState(*PluginAssetRegistryState);
						RefreshPackageLocalizationCacheForPlugin();
					}

					CompletedSubStates |= ESubState::LoadAssetRegistry;
					UpdateStateMachineImmediate();
				});

			}, UE::Tasks::ETaskPriority::BackgroundHigh);
		}
		else
		{
			IAssetRegistry& AssetRegistry = UAssetManager::Get().GetAssetRegistry();
			AssetRegistry.ActiveMountsRegisterAndLoadFilePathAsync(PluginAssetRegistry,
				[this, RefreshPackageLocalizationCacheForPlugin](bool bSuccess)
				{
					if (bSuccess)
					{
						RefreshPackageLocalizationCacheForPlugin();
					}
					ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, bSuccess]
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Mounting_ARComplete);
							if (!bSuccess)
							{
								Result = GetErrorResult(TEXT("Failed_To_Load_Plugin_AssetRegistry"));
							}
							CompletedSubStates |= ESubState::LoadAssetRegistry;
							UpdateStateMachineImmediate();
						});
				}, UE::Tasks::ETaskPriority::BackgroundHigh);
		}
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		// Check if waiting for install bundles
		if (PendingBundles.Num() > 0)
		{
			return;
		}

		// Check if post-mount is paused
		if (NumExpectedPostMountPausers > 0)
		{
			// Check if post-mount unpaused
			if (NumExpectedPostMountPausers == NumObservedPostMountPausers)
			{
				NumExpectedPostMountPausers = INDEX_NONE;
				TransitionOut(StateStatus);
			}
			return;
		}

		if (!bCheckedRealtimeMode)
		{
			bCheckedRealtimeMode = true;
			if (UE::GameFeatures::RealtimeMode)
			{
				UE::GameFeatures::RealtimeMode->AddUpdateRequest(StateProperties);
				return;
			}
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Mounting);
		
		UpdateState_MountPlugin(AllowIniLoading());
		UpdateState_LoadAssetRegistry();

		const bool bComplete = EnumHasAllFlags(CompletedSubStates, ESubState::MountPlugin | ESubState::LoadAssetRegistry);

		// Post-mount
		if (bComplete)
		{
			FGameFeaturePostMountingContext Context(StateProperties.PluginName, [this](FStringView InPauserTag) { OnPostMountPauserCompleted(InPauserTag); });
			NumExpectedPostMountPausers = INDEX_NONE;
			UGameFeaturesSubsystem::Get().OnGameFeaturePostMounting(StateProperties.PluginName, StateProperties.PluginIdentifier, Context);
			NumExpectedPostMountPausers = Context.NumPausers;

			// Check if we got post-mount paused
			if (NumExpectedPostMountPausers <= 0)
			{
				TransitionOut(StateStatus);
			}
		}
	}

	void TransitionOut(FGameFeaturePluginStateStatus& StateStatus)
	{
		if (Result.HasError())
		{
			StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorMounting, Result);
		}
		else
		{
			static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Mounting] == EGameFeaturePluginState::WaitingForDependencies);
			static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Mounting] == EGameFeaturePluginState::WaitingForDependencies);
			StateStatus.SetTransition(EGameFeaturePluginState::WaitingForDependencies);
		}
	}

	virtual void EndState() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Mounting_EndState);
		IInstallBundleManager::InstallBundleCompleteDelegate.RemoveAll(this);
		if (PakFileMountedDelegateHandle.IsValid())
		{
			FCoreDelegates::GetOnPakFileMounted2().RemoveAll(this);
			PakFileMountedDelegateHandle.Reset();
		}
	}
};
ENUM_CLASS_FLAGS(FGameFeaturePluginState_Mounting::ESubState);
UE::Tasks::FPipe FGameFeaturePluginState_Mounting::ShaderlibPipe(TEXT("FGameFeaturePluginState_Mounting::ShaderlibPipe"));

struct FWaitingForDependenciesTransitionPolicy
{
	static bool GetPluginDependencyStateMachines(const FGameFeaturePluginStateMachineProperties& InStateProperties, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines)
	{
		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();

		return GameFeaturesSubsystem.FindOrCreatePluginDependencyStateMachines(
			InStateProperties.PluginIdentifier.GetFullPluginURL(), InStateProperties, OutDependencyMachines);
	}

	static FGameFeaturePluginStateRange GetDependencyStateRange()
	{
		return FGameFeaturePluginStateRange(EGameFeaturePluginState::Registered, EGameFeaturePluginState::Active);
	}

	static EGameFeaturePluginState GetTransitionState()
	{
		return UE::GameFeatures::CVarEnableAssetStreaming.GetValueOnGameThread() ? EGameFeaturePluginState::AssetDependencyStreaming : EGameFeaturePluginState::Registering;
	}

	static EGameFeaturePluginState GetErrorState()
	{
		return EGameFeaturePluginState::ErrorWaitingForDependencies;
	}

	static bool ExcludeDepedenciesFromBatchProcessing()
	{
		return false;
	}

	static bool ShouldWaitForDependencies()
	{
		return true;
	}
};

struct FGameFeaturePluginState_WaitingForDependencies : public FTransitionDependenciesGameFeaturePluginState<FWaitingForDependenciesTransitionPolicy>
{
	using Super = FTransitionDependenciesGameFeaturePluginState<FWaitingForDependenciesTransitionPolicy>;
	using Super::Super;
};

struct FGameFeaturePluginState_AssetDependencyStreamOut : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	virtual void BeginState() override
	{
		if (StateProperties.GetPluginProtocol() != EGameFeaturePluginProtocol::InstallBundle)
		{
			return;
		}

		const FInstallBundlePluginProtocolMetaData& MetaData = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();

		UE::GameFeatures::GFPSharedInstallTracker.Release(
			StateProperties.PluginName, UE::GameFeatures::EGFPInstallLevel::AssetStream, MetaData.InstallBundlesWithAssetDependencies);
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::AssetDependencyStreamOut] == EGameFeaturePluginState::Unmounting);
		static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::AssetDependencyStreamOut] == EGameFeaturePluginState::Unmounting);
		StateStatus.SetTransition(EGameFeaturePluginState::Unmounting);
	}
};

struct FGameFeaturePluginState_ErrorAssetDependencyStreaming : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorAssetDependencyStreaming] == EGameFeaturePluginState::AssetDependencyStreamOut);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorAssetDependencyStreaming] == EGameFeaturePluginState::AssetDependencyStreaming);
};

struct FGameFeaturePluginState_AssetDependencyStreaming : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	virtual ~FGameFeaturePluginState_AssetDependencyStreaming()
	{
		Cleanup();
	}

	struct FIoStoreOnDemandProgress
	{
		FName InstallBundle;
		UE::IoStore::FOnDemandInstallProgress Progress;
	};

	// Required for callback lifetime safety
	struct FIoStoreOnDemandContext
	{
		TArray<UE::IoStore::FOnDemandInstallRequest> InstallRequests;
		TArray<FIoStoreOnDemandProgress> Progress;
		int32 PendingInstalls = 0;
		bool bStateValid = true;
	};

	TSharedPtr<FIoStoreOnDemandContext> IoStoreOnDemandContext;

	UE::GameFeatures::FResult Result{ MakeValue() };
	bool bComplete = false;

	void Cleanup()
	{
		Result = MakeValue();
		bComplete = false;

		if (IoStoreOnDemandContext)
		{
			for (UE::IoStore::FOnDemandInstallRequest& Request : IoStoreOnDemandContext->InstallRequests)
			{
				Request.Cancel();
			}
			IoStoreOnDemandContext->bStateValid = false;
			IoStoreOnDemandContext = nullptr;
		}
	}

	virtual void BeginState() override
	{
		Cleanup();
#if !UE_BUILD_SHIPPING
		if (UE::GameFeatures::ShouldDebugGFP(StateProperties.PluginName))
		{
			static int32 I = 0; 
			++I;
		}
#endif
		if (StateProperties.GetPluginProtocol() != EGameFeaturePluginProtocol::InstallBundle)
		{
			bComplete = true;
			return;
		}

		const FInstallBundlePluginProtocolMetaData& MetaData = StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();

		if (MetaData.InstallBundlesWithAssetDependencies.IsEmpty())
		{
			bComplete = true;
			return;
		}

		UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
		if (!IoStore)
		{
			Result = GetErrorResult(TEXT("IoStoreOnDemand.ModuleNotFound"));
			return;
		}

		TValueOrError<TArray<EStreamingAssetInstallMode>, FString> MaybeInstallModes = 
			UGameFeaturesSubsystem::Get().GetPolicy().GetStreamingAssetInstallModes(
				StateProperties.PluginIdentifier.GetFullPluginURL(), MetaData.InstallBundlesWithAssetDependencies);

		if (MaybeInstallModes.HasError())
		{
			Result = GetErrorResult(TEXT("IoStoreOnDemand.InstallMode"), MaybeInstallModes.GetError());
			return;
		}

		const FInstallBundlePluginProtocolOptions& Options = StateProperties.ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();

		UE::GameFeatures::GFPSharedInstallTracker.AddBundleRefs(
			StateProperties.PluginName, UE::GameFeatures::EGFPInstallLevel::AssetStream, MetaData.InstallBundlesWithAssetDependencies);

		IoStoreOnDemandContext = MakeShared<FIoStoreOnDemandContext>();
		IoStoreOnDemandContext->PendingInstalls = MetaData.InstallBundlesWithAssetDependencies.Num();
		IoStoreOnDemandContext->Progress.Reserve(MetaData.InstallBundlesWithAssetDependencies.Num());

		const TArray<EStreamingAssetInstallMode>& InstallModes = MaybeInstallModes.GetValue();
		for (int i = 0; const FName InstallBundle : MetaData.InstallBundlesWithAssetDependencies)
		{
			IoStoreOnDemandContext->Progress.Emplace(FIoStoreOnDemandProgress{InstallBundle});

			const EStreamingAssetInstallMode InstallMode = InstallModes[i++];

			UE::IoStore::FOnDemandInstallArgs InstallArgs;
			InstallArgs.MountId = InstallBundle.ToString();
			if (InstallMode == EStreamingAssetInstallMode::GfpRequiredOnly)
			{
				InstallArgs.TagSets.Emplace(TEXTVIEW("required"));
				InstallArgs.TagSets.Emplace(StateProperties.PluginName);
			}
			InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::InstallSoftReferences;
			InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::CallbackOnGameThread;
			if (Options.bDoNotDownload && !UE::GameFeatures::CVarForceGFPIADDownload.GetValueOnGameThread())
			{
				InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::DoNotDownload;
			}
			if (UE::GameFeatures::CVarAllowMissingOnDemandDependencies.GetValueOnGameThread())
			{
				InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::AllowMissingDependencies;
			}

			InstallArgs.DebugName = StateProperties.PluginName;

			InstallArgs.ContentHandle = UE::GameFeatures::GFPSharedInstallTracker.AddOnDemandContentHandle(
				InstallBundle, UE::GameFeatures::EGFPInstallLevel::AssetStream);

			IoStoreOnDemandContext->InstallRequests.Add(IoStore->Install(MoveTemp(InstallArgs),
				// On Complete
				[this, LambdaOnDemandContext = IoStoreOnDemandContext, DebugName = InstallArgs.DebugName](const UE::IoStore::FOnDemandInstallResult& OnDemandInstallResult)
				{
					if (!LambdaOnDemandContext->bStateValid)
					{
						// Owning state got cleaned up, bail
						return;
					}

					if (!OnDemandInstallResult.IsOk() && !Result.HasError())
					{
						const FText ErrorMessage	= OnDemandInstallResult.Error.GetValue().GetErrorMessage();
						FString ErrorCode			= FString(OnDemandInstallResult.Error.GetValue().GetModuleIdAndErrorCodeString());

						ErrorCode.ReplaceCharInline(TEXT(' '), TEXT('_'), ESearchCase::CaseSensitive);
						Result = GetErrorResult(TEXT("IoStoreOnDemand.OnComplete."), ErrorCode, ErrorMessage);
						UE_LOGFMT(LogGameFeatures, Error, "Failed to install request {DebugName} due to error {Error}",
							("DebugName", DebugName),
							("Error", Result.GetError()));
						TryCancelState();
					}

					--IoStoreOnDemandContext->PendingInstalls;
					if (IoStoreOnDemandContext->PendingInstalls == 0)
					{
						bComplete = true;
						UpdateStateMachineImmediate();
					}

				},
				// On Progress
				[this, LambdaOnDemandContext = IoStoreOnDemandContext, InstallBundle](const UE::IoStore::FOnDemandInstallProgress& Progress)
				{
					if (!LambdaOnDemandContext->bStateValid)
					{
						// Owning state got cleaned up, bail
						return;
					}

					FIoStoreOnDemandProgress* MyProgress = Algo::FindBy(
						LambdaOnDemandContext->Progress, InstallBundle, &FIoStoreOnDemandProgress::InstallBundle);
					check(MyProgress);
					MyProgress->Progress = Progress;

					const UE::IoStore::FOnDemandInstallProgress SumProgress = Algo::TransformAccumulate(
						LambdaOnDemandContext->Progress,
						&FIoStoreOnDemandProgress::Progress,
						UE::IoStore::FOnDemandInstallProgress(),
						&UE::IoStore::FOnDemandInstallProgress::Combine);
					const float OverallProgress = SumProgress.GetRelativeProgress();

					StateProperties.StateProgress = OverallProgress;
				}));
		}
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (!Result.HasValue())
		{
			StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorAssetDependencyStreaming, Result);
			return;
		}

		if (!bComplete)
		{
			return;
		}

		static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::AssetDependencyStreaming] == EGameFeaturePluginState::Registering);
		static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::AssetDependencyStreaming] == EGameFeaturePluginState::Registering);
		StateStatus.SetTransition(EGameFeaturePluginState::Registering);
	}

	virtual void EndState() override
	{
		Cleanup();
	}

	virtual void TryCancelState() override
	{
		if (IoStoreOnDemandContext)
		{
			for (UE::IoStore::FOnDemandInstallRequest& InstallRequest : IoStoreOnDemandContext->InstallRequests)
			{
				InstallRequest.Cancel();
			}
		}
	}
};

struct FGameFeaturePluginState_Unregistering : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	bool bHasUnloaded = false;
#if WITH_EDITOR
	bool bRequestedUnloadPluginAssets = false;
#endif //if WITH_EDITOR

	virtual void BeginState() override
	{
		bHasUnloaded = false;
#if WITH_EDITOR
		bRequestedUnloadPluginAssets = false;
#endif //if WITH_EDITOR
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (bHasUnloaded)
		{
			static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Unregistering] == EGameFeaturePluginState::AssetDependencyStreamOut);
			static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Unregistering] == EGameFeaturePluginState::AssetDependencyStreamOut);
			StateStatus.SetTransition(EGameFeaturePluginState::AssetDependencyStreamOut);
			return;
		}

#if WITH_EDITOR
		if (bRequestedUnloadPluginAssets)
		{
			bHasUnloaded = true;
			UpdateStateMachineDeferred();
			return;
		}
#endif //if WITH_EDITOR

		if (StateProperties.GameFeatureData)
		{
			UStringTable::OnPluginUnloaded(StateProperties.PluginName);
			UGameFeaturesSubsystem::Get().OnGameFeatureUnregistering(StateProperties.GameFeatureData, StateProperties.PluginName, StateProperties.PluginIdentifier);

			UGameFeaturesSubsystem::RemoveGameFeatureFromAssetManager(StateProperties.GameFeatureData, StateProperties.PluginName, StateProperties.AddedPrimaryAssetTypes);
			StateProperties.AddedPrimaryAssetTypes.Empty();

			UGameFeaturesSubsystem::UnloadGameFeatureData(StateProperties.GameFeatureData);
		}
		StateProperties.GameFeatureData = nullptr;

		// Try to remove the gameplay tags, this might be ignored depending on project settings
		const FString PluginFolder = FPaths::GetPath(StateProperties.PluginInstalledFilename);
		UGameplayTagsManager::Get().RemoveTagIniSearchPath(PluginFolder / TEXT("Config") / TEXT("Tags"));

#if WITH_EDITOR
		// This will properly unload any plugin asset that could be opened in the editor
		// and ensure standalone packages get unloaded as well
		if (FApp::IsGame())
		{
			verify(FPluginUtils::UnloadPluginAssets(StateProperties.PluginName));

			bHasUnloaded = true;
			UpdateStateMachineDeferred();
		}
		else
		{
			bRequestedUnloadPluginAssets = true;
			UE::GameFeatures::ScheduleUnloadPluginAssets(StateProperties.PluginName, StateProperties);
		}
#else
		bHasUnloaded = true;
		UpdateStateMachineDeferred();
#endif
	}
};

struct FGameFeaturePluginState_Registering : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	enum class ELoadGFDState : uint8
	{
		Pending = 0,
		Success,
		Cancelled,
		Failed
	};

	TSharedPtr<FStreamableHandle> GameFeatureDataHandle;
	ELoadGFDState LoadGFDState = ELoadGFDState::Pending;
	bool bCheckedRealtimeMode = false;
	TArray<FString, TInlineAllocator<2>> GameFeatureDataPaths;

	void TryAsyncLoadGameFeatureData(int32 Attempt = 0)
	{
		check(LoadGFDState == ELoadGFDState::Pending);

		if (!GameFeatureDataPaths.IsValidIndex(Attempt))
		{
			GFP_STATE_LOG(VeryVerbose, "Ran out of game feature paths to try");
			LoadGFDState = ELoadGFDState::Failed;
			UpdateStateMachineDeferred();
			return;
		}

		bool bIsLoading = false;

		GFP_STATE_LOG(VeryVerbose, "Trying to load game feature data {GameFeatureData}", GameFeatureDataPaths[Attempt]);
		GameFeatureDataHandle = UGameFeaturesSubsystem::LoadGameFeatureData(GameFeatureDataPaths[Attempt], true /*bStartStalled*/);
		if (GameFeatureDataHandle && GameFeatureDataHandle->IsLoadingInProgress())
		{
			GFP_STATE_LOG(VeryVerbose, "Waiting for load of game feature data {GameFeatureData}", GameFeatureDataPaths[Attempt]);
			// CurrentMachine owns `this`, so use it as a lifetime check for the `this` captured in the lambdas below, as they may be called after `this` is destroyed
			UGameFeaturePluginStateMachine* CurrentMachine = UGameFeaturesSubsystem::Get().FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier);

			GameFeatureDataHandle->BindCancelDelegate(FStreamableDelegateWithHandle::CreateWeakLambda(CurrentMachine, [this](const TSharedPtr<FStreamableHandle>& Handle)
			{
				if (GameFeatureDataHandle != Handle)
				{
					GFP_STATE_LOG(VeryVerbose, "Ignoring cancel of game feature data load as we are no longer in a relevant state");
					// We're no longer in a state where we want to process this callback (maybe we already went through EndState)
					return;
				}

				const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
				GFP_STATE_LOG(Error, "Game Feature Data loading was cancelled for URL {ShortURL}", ShortUrl);

				LoadGFDState = ELoadGFDState::Cancelled;
				UpdateStateMachineDeferred();
			}));

			GameFeatureDataHandle->BindCompleteDelegate(FStreamableDelegateWithHandle::CreateWeakLambda(CurrentMachine, [this, Attempt](const TSharedPtr<FStreamableHandle>& Handle)
			{
				if (GameFeatureDataHandle != Handle)
				{
					GFP_STATE_LOG(VeryVerbose, "Ignoring completion of game feature data load as we are no longer in a relevant state");
					// We're no longer in a state where we want to process this callback (maybe we already went through EndState)
					return;
				}

				StateProperties.GameFeatureData = Cast<UGameFeatureData>(GameFeatureDataHandle->GetLoadedAsset());
				if (!StateProperties.GameFeatureData && GameFeatureDataPaths.IsValidIndex(Attempt + 1))
				{
					TryAsyncLoadGameFeatureData(Attempt + 1);
					return;
				}

				if (StateProperties.GameFeatureData)
				{
					GFP_STATE_LOG(VeryVerbose, "Loaded GameFeatureData {GameFeatureData}", FSoftObjectPath(StateProperties.GameFeatureData));
					LoadGFDState = ELoadGFDState::Success;
				}
				else
				{
					// This will set LoadGFDState
					HandleMissingGameFeatureData();
				}
				
				UpdateStateMachineDeferred();
			}));

			bIsLoading = true;
			GameFeatureDataHandle->StartStalledHandle();
		}

		if (!bIsLoading)
		{
			TryAsyncLoadGameFeatureData(Attempt + 1);
		}
	}

	virtual bool UseAsyncLoading() const override
	{
		if (UE::GameFeatures::CVarForceSyncRegisterStartupPlugins.GetValueOnGameThread())
		{
			if (UGameFeaturesSubsystem::Get().GetPolicy().IsLoadingStartupPlugins())
			{
				return false;
			}
		}
		return FGameFeaturePluginState::UseAsyncLoading();
	}

	UGameFeatureData* CreateTransientGameFeatureData()
	{
		return NewObject<UGameFeatureData>(GetTransientPackage(), FName(*StateProperties.PluginName), RF_Transient);
	}

	void HandleMissingGameFeatureData()
	{
		const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
		if (UGameFeaturesSubsystem::Get().GetPolicy().ShouldAllowMissingGameFeatureData(StateProperties.PluginName))
		{
			UE_LOGF(LogGameFeatures, Log, "Game Feature Data package not found for URL %.*ls. Creating a transient GameFeatureData as a placeholder.", ShortUrl.Len(), ShortUrl.GetData());
			StateProperties.GameFeatureData = CreateTransientGameFeatureData();

			LoadGFDState = ELoadGFDState::Success;
		}
		else
		{
			if (GameFeatureDataHandle.IsValid())
			{
				if (GameFeatureDataHandle->HasError())
				{
					const TOptional<UE::UnifiedError::FError>& Error = GameFeatureDataHandle->GetError();
					UE_LOGF(LogGameFeatures, Error, "Game Feature Data loading failed with error [%ls] for URL %.*ls", *LexToString(*Error), ShortUrl.Len(), ShortUrl.GetData());
				}
				else
				{
					UE_LOGF(LogGameFeatures, Error, "Game Feature Data loading failed without an error for URL %.*ls", ShortUrl.Len(), ShortUrl.GetData());
				}
			}
			else
			{
				UE_LOGF(LogGameFeatures, Error, "Game Feature Data package not found for URL %.*ls", ShortUrl.Len(), ShortUrl.GetData());
			}

			LoadGFDState = ELoadGFDState::Failed;
		}
	}

	virtual void BeginState() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Registering_Begin);

		bCheckedRealtimeMode = false;

		const FString PluginFolder = FPaths::GetPath(StateProperties.PluginInstalledFilename);

		if (AllowIniLoading())
		{
			UGameplayTagsManager::Get().AddTagIniSearchPath(PluginFolder / TEXT("Config") / TEXT("Tags"), GConfig->GetStagedPluginConfigCache(FName(*StateProperties.PluginName)));
		}

		LoadGFDState = ELoadGFDState::Pending;

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(StateProperties.PluginName);
		ensure(Plugin.IsValid());

		// If the plugin contains content then load the GameFeatureData otherwise procedurally create one that is transient.
		if (!Plugin->GetDescriptor().bCanContainContent)
		{
			GFP_STATE_LOG(VeryVerbose, "Creating transient GameFeatureData for non-content plugin");
			StateProperties.GameFeatureData = CreateTransientGameFeatureData();
			LoadGFDState = ELoadGFDState::Success;
			return;
		}

		FString BackupGameFeatureDataPath = FString::Printf(TEXT("/%s/%s.%s"), *StateProperties.PluginName, *StateProperties.PluginName, *StateProperties.PluginName);

		FString PreferredGameFeatureDataPath = TEXT("/") + StateProperties.PluginName + TEXT("/GameFeatureData.GameFeatureData");

		if (AllowIniLoading())
		{
			// Allow game feature location to be overriden globally and from within the plugin
			FString OverrideIniPathName = StateProperties.PluginName + TEXT("_Override");
			FString OverridePath = GConfig->GetStr(TEXT("GameFeatureData"), *OverrideIniPathName, GGameIni);
			if (OverridePath.IsEmpty())
			{
				const FString SettingsOverride = PluginFolder / TEXT("Config") / TEXT("Settings.ini");
				if (FPaths::FileExists(SettingsOverride))
				{
					GConfig->LoadFile(SettingsOverride);
					OverridePath = GConfig->GetStr(TEXT("GameFeatureData"), TEXT("Override"), SettingsOverride);
					GConfig->UnloadFile(SettingsOverride);
				}
			}
			if (!OverridePath.IsEmpty())
			{
				PreferredGameFeatureDataPath = MoveTemp(OverridePath);
			}
		}

		// Temporary workaround for UE-309330
		// FPackageName::DoesPackageExist is unreliable when running with cook-on-the-fly, instead we must make multiple attempts
		// to load the cooked packages and handle failure/repeats
		if (IsRunningCookOnTheFly())
		{
			GameFeatureDataPaths.Emplace(MoveTemp(PreferredGameFeatureDataPath));
			GameFeatureDataPaths.Emplace(MoveTemp(BackupGameFeatureDataPath));
		}
		else
		{
			if (FPackageName::DoesPackageExist(PreferredGameFeatureDataPath))
			{
				GameFeatureDataPaths.Emplace(MoveTemp(PreferredGameFeatureDataPath));
			}
			else if (FPackageName::DoesPackageExist(BackupGameFeatureDataPath))
			{
				GameFeatureDataPaths.Emplace(MoveTemp(BackupGameFeatureDataPath));
			}
			else
			{
				// This will set LoadGFDState
				HandleMissingGameFeatureData();
				return;
			}
		}

		if (UseAsyncLoading())
		{
			TryAsyncLoadGameFeatureData();
		}
		else
		{
			FScopedSlowTask LoadingGameFeatureData(1.0f, 
				FText::Format(
					LOCTEXT("LoadingGameFeatureData", "Loading Game Feature Data for Plugin: {0}"), 
					FText::FromString(StateProperties.PluginName)));
			LoadingGameFeatureData.Visibility = ESlowTaskVisibility::Important;
			
			for (const FString& Path : GameFeatureDataPaths)
			{
				GFP_STATE_LOG(VeryVerbose, "Synchronously loading GameFeatureData {Path}", Path);
				GameFeatureDataHandle = UGameFeaturesSubsystem::LoadGameFeatureData(Path);
				if (GameFeatureDataHandle)
				{
					GameFeatureDataHandle->WaitUntilComplete(0.0f, false);
					StateProperties.GameFeatureData = Cast<UGameFeatureData>(GameFeatureDataHandle->GetLoadedAsset());
				}
				if (StateProperties.GameFeatureData)
				{
					break;
				}
			}

			if (StateProperties.GameFeatureData)
			{
				LoadGFDState = ELoadGFDState::Success;
			}
			else
			{
				GFP_STATE_LOG(VeryVerbose, "Failed to synchronously load GameFeatureData");
				// This will set LoadGFDState
				HandleMissingGameFeatureData();
			}
		}
	}

	virtual void EndState() override
	{
		GameFeatureDataHandle = nullptr;
		GameFeatureDataPaths.Empty();
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Registering_Update);
		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();
		
		if (!bCheckedRealtimeMode)
		{
			bCheckedRealtimeMode = true;
			if (UE::GameFeatures::RealtimeMode)
			{
				UE::GameFeatures::RealtimeMode->AddUpdateRequest(StateProperties);
				return;
			}
		}

		if (!StateProperties.GameFeatureData)
		{
			check(LoadGFDState != ELoadGFDState::Success);

			if (LoadGFDState == ELoadGFDState::Pending)
			{
				return;
			}

			if (LoadGFDState == ELoadGFDState::Cancelled)
			{
				StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorRegistering, GetErrorResult(TEXT("Load_Cancelled_GameFeatureData")));
				return;
			}
		}

		if (StateProperties.GameFeatureData)
		{
			check(LoadGFDState == ELoadGFDState::Success);

			static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Registering] == EGameFeaturePluginState::Registered);
			static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Registering] == EGameFeaturePluginState::Registered);
			StateStatus.SetTransition(EGameFeaturePluginState::Registered);

			check(StateProperties.AddedPrimaryAssetTypes.Num() == 0);
			GameFeaturesSubsystem.AddGameFeatureToAssetManager(StateProperties.GameFeatureData, StateProperties.PluginName, StateProperties.AddedPrimaryAssetTypes);

			GameFeaturesSubsystem.OnGameFeatureRegistering(StateProperties.GameFeatureData, StateProperties.PluginName, StateProperties.PluginIdentifier);
		}
		else
		{
			check(LoadGFDState == ELoadGFDState::Failed);

			// The gamefeaturedata does not exist. The pak file may not be openable or this is a builtin plugin where the pak file does not exist.
			StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorRegistering, GetErrorResult(TEXT("Plugin_Missing_GameFeatureData")));
			if (UGameFeaturePluginStateMachine* CurrentMachine = UGameFeaturesSubsystem::Get().FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier))
			{
				const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
				UE_LOGF(LogGameFeatures, Error, "Setting %.*ls to be in unrecoverable error as GameFeatureData is missing", ShortUrl.Len(), ShortUrl.GetData());
				GameFeaturesSubsystem.SetUnrecoverableError(CurrentMachine);
			}
		}
	}
};

struct FGameFeaturePluginState_Registered : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Registered] == EGameFeaturePluginState::Loading);
	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Registered] == EGameFeaturePluginState::Unregistering);
};

struct FGameFeaturePluginState_ErrorLoading : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorLoading] == EGameFeaturePluginState::Unloading);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorLoading] == EGameFeaturePluginState::Loading);
};

struct FGameFeaturePluginState_Unloading : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	virtual void BeginState() override
	{
		if (UE::GameFeatures::ShouldDeferLocalizationDataLoad())
		{
			IPluginManager::Get().UnmountExplicitlyLoadedPluginLocalizationData(StateProperties.PluginName);
		}
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		UnloadGameFeatureBundles(StateProperties.GameFeatureData);
		UGameFeaturesSubsystem::Get().OnGameFeatureUnloading(StateProperties.GameFeatureData, StateProperties.PluginIdentifier);

		static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Unloading] == EGameFeaturePluginState::Registered);
		static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Unloading] == EGameFeaturePluginState::Registered);
		StateStatus.SetTransition(EGameFeaturePluginState::Registered);
	}


	void UnloadGameFeatureBundles(const UGameFeatureData* GameFeatureToLoad)
	{
		if (GameFeatureToLoad == nullptr)
		{
			return;
		}

		const UGameFeaturesProjectPolicies& Policy = UGameFeaturesSubsystem::Get().GetPolicy();

		// Remove all bundles from feature data and completely unload everything else
		FPrimaryAssetId GameFeatureAssetId = GameFeatureToLoad->GetPrimaryAssetId();
		TSharedPtr<FStreamableHandle> Handle = UAssetManager::Get().ChangeBundleStateForPrimaryAssets({ GameFeatureAssetId }, {}, {}, /*bRemoveAllBundles=*/ true);
		ensureAlways(Handle == nullptr || Handle->HasLoadCompleted()); // Should be no handle since nothing is being loaded

		TArray<FPrimaryAssetId> AssetIds = Policy.GetPreloadAssetListForGameFeature(GameFeatureToLoad, /*bIncludeLoadedAssets=*/true);

		// Don't unload game feature data asset yet, that will happen in FGameFeaturePluginState_Unregistering
		ensureAlways(AssetIds.RemoveSwap(GameFeatureAssetId, EAllowShrinking::No) == 0);

		if (AssetIds.Num() > 0)
		{
			UAssetManager::Get().UnloadPrimaryAssets(AssetIds);
		}
	}
};

struct FGameFeaturePluginState_Loading : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	TSharedPtr<FStreamableHandle> BundleHandle;

	virtual void BeginState() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Loading_Begin);
		check(StateProperties.GameFeatureData);

		if (UE::GameFeatures::ShouldDeferLocalizationDataLoad())
		{
			UGameFeaturePluginStateMachine* CurrentMachine = UGameFeaturesSubsystem::Get().FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier);
			UE::GameFeatures::MountLocalizationData(CurrentMachine, StateProperties);
		}

		LoadGameFeatureBundles(StateProperties.GameFeatureData);
	}

	virtual void EndState() override
	{
		BundleHandle = nullptr;
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Loading_Update);
		check(StateProperties.GameFeatureData);

		if (BundleHandle)
		{
			if (!UseAsyncLoading())
			{
				BundleHandle->WaitUntilComplete(0.0f, false);
			}

			if (BundleHandle->IsLoadingInProgress())
			{
				return;
			}

			if (BundleHandle->WasCanceled())
			{
				BundleHandle.Reset();
				StateStatus.SetTransitionError(EGameFeaturePluginState::ErrorLoading, GetErrorResult(TEXT("Load_Cancelled_Preload")));
				return;
			}
		}

		UGameFeaturesSubsystem::Get().OnGameFeatureLoading(StateProperties.GameFeatureData, StateProperties.PluginIdentifier);

		static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Loading] == EGameFeaturePluginState::Loaded);
		static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Loading] == EGameFeaturePluginState::Loaded);
		StateStatus.SetTransition(EGameFeaturePluginState::Loaded);
	}

	/** Loads primary assets and bundles for the specified game feature */
	void LoadGameFeatureBundles(const UGameFeatureData* GameFeatureToLoad)
	{
		check(GameFeatureToLoad);

		const UGameFeaturesProjectPolicies& Policy = UGameFeaturesSubsystem::Get().GetPolicy<UGameFeaturesProjectPolicies>();

		TArray<FPrimaryAssetId> AssetIdsToLoad = Policy.GetPreloadAssetListForGameFeature(GameFeatureToLoad);

		FPrimaryAssetId GameFeatureAssetId = GameFeatureToLoad->GetPrimaryAssetId();
		if (GameFeatureAssetId.IsValid())
		{
			AssetIdsToLoad.Add(GameFeatureAssetId);
		}

		if (AssetIdsToLoad.Num() > 0)
		{
			// CurrentMachine owns `this`, so use it as a lifetime check for the `this` captured in the lambdas below, as they may be called after `this` is destroyed
			UGameFeaturePluginStateMachine* CurrentMachine = UGameFeaturesSubsystem::Get().FindGameFeaturePluginStateMachine(StateProperties.PluginIdentifier);

			FAssetManagerLoadParams LoadParams;
			LoadParams.OnCancel = FStreamableDelegateWithHandle::CreateWeakLambda(CurrentMachine, [this](const TSharedPtr<FStreamableHandle>& Handle)
			{
				if (BundleHandle != Handle)
				{
					// We're no longer in a state where we want to process this callback (maybe we already went through EndState)
					return;
				}

				const FStringView ShortUrl = StateProperties.PluginIdentifier.GetIdentifyingString();
				UE_LOGF(LogGameFeatures, Error, "Game Feature preloading was cancelled for URL %.*ls", ShortUrl.Len(), ShortUrl.GetData());
				UpdateStateMachineDeferred();
			});

			// This can't be bound to the handle after its created, AM may bind it internally
			LoadParams.OnComplete = FStreamableDelegateWithHandle::CreateWeakLambda(CurrentMachine, [this](const TSharedPtr<FStreamableHandle>& Handle)
			{
				if (BundleHandle != Handle)
				{
					// We're no longer in a state where we want to process this callback (maybe we already went through EndState)
					return;
				}

				UpdateStateMachineDeferred();
			});
			
			FString PluginName;
			GameFeatureToLoad->GetPluginName(PluginName);
			BundleHandle = UAssetManager::Get().LoadPrimaryAssets(AssetIdsToLoad, Policy.GetPreloadBundleStateForPlugin(PluginName), MoveTemp(LoadParams));
		}
		else
		{
			BundleHandle.Reset();
		}
	}
};

struct FGameFeaturePluginState_Loaded : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Loaded] == EGameFeaturePluginState::ActivatingDependencies);
	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Loaded] == EGameFeaturePluginState::Unloading);
};

struct FGameFeaturePluginState_ErrorDeactivatingDependencies : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorDeactivatingDependencies] == EGameFeaturePluginState::DeactivatingDependencies);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorDeactivatingDependencies] == EGameFeaturePluginState::DeactivatingDependencies);
};

struct FDeactivatingDependenciesTransitionPolicy
{
	static bool GetPluginDependencyStateMachines(const FGameFeaturePluginStateMachineProperties& InStateProperties, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines)
	{
		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();

		return GameFeaturesSubsystem.FindPluginDependencyStateMachinesToDeactivate(
			InStateProperties.PluginIdentifier.GetFullPluginURL(), InStateProperties.PluginInstalledFilename, OutDependencyMachines);
	}

	static FGameFeaturePluginStateRange GetDependencyStateRange()
	{
		return FGameFeaturePluginStateRange(EGameFeaturePluginState::Terminal, EGameFeaturePluginState::Loaded);
	}

	static EGameFeaturePluginState GetTransitionState()
	{
		return EGameFeaturePluginState::Deactivating;
	}

	static EGameFeaturePluginState GetErrorState()
	{
		return EGameFeaturePluginState::ErrorDeactivatingDependencies;
	}

	static bool ExcludeDepedenciesFromBatchProcessing()
	{
		return false;
	}

	static bool ShouldWaitForDependencies()
	{
		return UE::GameFeatures::CVarWaitForDependencyDeactivation.GetValueOnGameThread();
	}
};

struct FGameFeaturePluginState_DeactivatingDependencies : public FTransitionDependenciesGameFeaturePluginState<FDeactivatingDependenciesTransitionPolicy>
{
	using Super = FTransitionDependenciesGameFeaturePluginState<FDeactivatingDependenciesTransitionPolicy>;
	using Super::Super;
};

struct FGameFeaturePluginState_Deactivating : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	int32 NumObservedPausers = 0;
	int32 NumExpectedPausers = 0;
	bool bInProcessOfDeactivating = false;
	bool bHasUnloaded = false;

	virtual void BeginState() override
	{
		NumObservedPausers = 0;
		NumExpectedPausers = 0;
		bInProcessOfDeactivating = false;
		bHasUnloaded = false;

		static bool bUseNewDynamicLayers = IConsoleManager::Get().FindConsoleVariable(TEXT("ini.UseNewDynamicLayers"))->GetInt() != 0;
		if (bUseNewDynamicLayers)
		{
			FName Tag = *StateProperties.PluginName;
			UE::DynamicConfig::PerformDynamicConfig(Tag, [Tag](FConfigModificationTracker* ChangeTracker)
			{
				// note that we are unloading the GFP so we don't call LoadObject iuuselessly
				ChangeTracker->bIsUnloadingPlugin = true;
				FConfigCacheIni::RemoveTagFromAllBranches(Tag, ChangeTracker);
				IConsoleManager::Get().UnsetAllConsoleVariablesWithTag(Tag);
				UE::DynamicConfig::RemovePluginHotfixesFromDynamicBranch.Broadcast(Tag);
			});
		}
	}

	void OnPauserCompleted(FStringView InPauserTag)
	{
		check(IsInGameThread());
		ensure(NumExpectedPausers != INDEX_NONE);
		++NumObservedPausers;

		UE_LOGF(LogGameFeatures, Display, "Deactivation of %ls resumed by %.*ls", *StateProperties.PluginName, InPauserTag.Len(), InPauserTag.GetData());

		if (NumObservedPausers == NumExpectedPausers)
		{
			UpdateStateMachineImmediate();
		}
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		if (bHasUnloaded)
		{
			check(NumExpectedPausers == NumObservedPausers);
			static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Deactivating] == EGameFeaturePluginState::Loaded);
			static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Deactivating] == EGameFeaturePluginState::Loaded);
			StateStatus.SetTransition(EGameFeaturePluginState::Loaded);
			return;
		}

		if (!bInProcessOfDeactivating)
		{
			// Make sure we won't complete the transition prematurely if someone registers as a pauser but fires immediately
			bInProcessOfDeactivating = true;
			NumExpectedPausers = INDEX_NONE;
			NumObservedPausers = 0;

			// Deactivate
			FGameFeatureDeactivatingContext Context(StateProperties.PluginName, [this](FStringView InPauserTag) { OnPauserCompleted(InPauserTag); });
			UGameFeaturesSubsystem::Get().OnGameFeatureDeactivating(StateProperties.GameFeatureData, StateProperties.PluginName, Context, StateProperties.PluginIdentifier);
			NumExpectedPausers = Context.NumPausers;

			// Since we are pausing work during this deactivation, also notify the OnGameFeaturePauseChange delegate
			if (NumExpectedPausers > 0)
			{
				FGameFeaturePauseStateChangeContext PauseContext(UE::GameFeatures::ToString(EGameFeaturePluginState::Deactivating), TEXT("PendingDeactivationCallbacks"), true);
				UGameFeaturesSubsystem::Get().OnGameFeaturePauseChange(StateProperties.PluginIdentifier, StateProperties.PluginName, PauseContext);
			}
		}

		if (NumExpectedPausers == NumObservedPausers)
		{
			//If we previously sent an OnGameFeaturePauseChange delegate we need to send that work is now unpaused
			if (NumExpectedPausers > 0)
			{
				FGameFeaturePauseStateChangeContext PauseContext(UE::GameFeatures::ToString(EGameFeaturePluginState::Deactivating), TEXT(""), false);
				UGameFeaturesSubsystem::Get().OnGameFeaturePauseChange(StateProperties.PluginIdentifier, StateProperties.PluginName, PauseContext);
			}

			if (!bHasUnloaded && StateProperties.Destination.MaxState == EGameFeaturePluginState::Loaded)
			{
				// If we aren't going farther than Loaded, GC now
				// otherwise we will defer until closer to our destination state
				bHasUnloaded = true;
				UpdateStateMachineDeferred();
			}
			else
			{
				StateStatus.SetTransition(EGameFeaturePluginState::Loaded);
			}
		}
		else
		{
			UE_LOGF(LogGameFeatures, Log, "Game feature %ls deactivation paused until %d observer tasks complete their deactivation", *GetPathNameSafe(StateProperties.GameFeatureData), NumExpectedPausers - NumObservedPausers);
		}
	}
};

struct FGameFeaturePluginState_ErrorActivatingDependencies : public FErrorGameFeaturePluginState
{
	using FErrorGameFeaturePluginState::FErrorGameFeaturePluginState;

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::ErrorActivatingDependencies] == EGameFeaturePluginState::Unloading);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::ErrorActivatingDependencies] == EGameFeaturePluginState::ActivatingDependencies);
};

struct FActivatingDependenciesTransitionPolicy
{
	static bool GetPluginDependencyStateMachines(const FGameFeaturePluginStateMachineProperties& InStateProperties, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines)
	{
		UGameFeaturesSubsystem& GameFeaturesSubsystem = UGameFeaturesSubsystem::Get();

		return GameFeaturesSubsystem.FindPluginDependencyStateMachinesToActivate(
			InStateProperties.PluginIdentifier.GetFullPluginURL(), InStateProperties.PluginInstalledFilename, OutDependencyMachines);
	}

	static FGameFeaturePluginStateRange GetDependencyStateRange()
	{
		return FGameFeaturePluginStateRange(EGameFeaturePluginState::Active, EGameFeaturePluginState::Active);
	}

	static EGameFeaturePluginState GetTransitionState()
	{
		return EGameFeaturePluginState::Activating;
	}

	static EGameFeaturePluginState GetErrorState()
	{
		return EGameFeaturePluginState::ErrorActivatingDependencies;
	}

	static bool ExcludeDepedenciesFromBatchProcessing()
	{
		return true;
	}

	static bool ShouldWaitForDependencies()
	{
		return true;
	}
};

struct FGameFeaturePluginState_ActivatingDependencies : public FTransitionDependenciesGameFeaturePluginState<FActivatingDependenciesTransitionPolicy>
{
	using Super = FTransitionDependenciesGameFeaturePluginState<FActivatingDependenciesTransitionPolicy>;
	using Super::Super;
};

struct FGameFeaturePluginState_Activating : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	bool CanBatchProcess() const override
	{
		return FGameFeaturePluginState::CanBatchProcess() && AllowIniLoading();
	}

	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Activating);
		check(GEngine);
		check(StateProperties.GameFeatureData);

		// If this plugin caused localization data to load, we need that load to finish before marking it as active
		if (StateProperties.bIsLoadingLocalizationData)
		{
			if (AllowAsyncLoading())
			{
				return;
			}

			FTextLocalizationManager::Get().WaitForAsyncTasks();
			StateProperties.bIsLoadingLocalizationData = false;
		}

		if (IsWaitingForBatchProcessing())
		{
			return;
		}

		if(!WasBatchProcessed())
		{
			if (AllowIniLoading())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Activating_InitIni);
				StateProperties.GameFeatureData->InitializeHierarchicalPluginIniFiles(StateProperties.PluginInstalledFilename);
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Activating_SendEvents);
			FGameFeatureActivatingContext Context;
			UGameFeaturesSubsystem::Get().OnGameFeatureActivating(StateProperties.GameFeatureData, StateProperties.PluginName, Context, StateProperties.PluginIdentifier);
		}

		static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Activating] == EGameFeaturePluginState::Active);
		static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Activating] == EGameFeaturePluginState::Active);
		StateStatus.SetTransition(EGameFeaturePluginState::Active);
	}

	static void BatchProcess(TConstArrayView<UGameFeaturePluginStateMachine*> GFPs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_BatchProcess_OnFenceCompleteActivating);
		UGameFeaturesSubsystem& GFPSubSys = UGameFeaturesSubsystem::Get();

		TArray<FString> PluginInstalledFilenames;
		PluginInstalledFilenames.Reserve(GFPs.Num());

		for (const UGameFeaturePluginStateMachine* GFPSM : GFPs)
		{
			PluginInstalledFilenames.Emplace(GFPSubSys.GetProperties(GFPSM).PluginInstalledFilename);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GFP_BatchActivating_InitIni);
			UGameFeatureData::InitializeHierarchicalPluginIniFiles(PluginInstalledFilenames);
		}
	}
};

struct FGameFeaturePluginState_Active : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	virtual void BeginState() override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Active);
		check(GEngine);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GFP_Active_SendEvents);
			UGameFeaturesSubsystem::Get().OnGameFeatureActivated(StateProperties.GameFeatureData, StateProperties.PluginName, StateProperties.PluginIdentifier);
		}
	}

	static_assert(UE::GameFeatures::NextStateTowardsTerminal[EGameFeaturePluginState::Active] == EGameFeaturePluginState::DeactivatingDependencies);
	static_assert(UE::GameFeatures::NextStateTowardsActive[EGameFeaturePluginState::Active] == EGameFeaturePluginState::Active);
};

/*
=========================================================
  State Machine
=========================================================
*/
namespace UE::GameFeatures
{
	/** Helper classes that look for existence of batch processing functions on states so we can identify these compile time */
	template<class T, class = void>
	struct TBatchProcessHelperWrapper
	{
		static bool ImplementsBatchProcess()
		{
			return false;
		}
		static void BatchProcess(TConstArrayView<UGameFeaturePluginStateMachine*>)
		{
			check(!"Not implemented");
		}
	};

	template<class T>
	struct TBatchProcessHelperWrapper<T, std::enable_if_t<std::is_invocable_r<void, decltype(T::BatchProcess), TConstArrayView<UGameFeaturePluginStateMachine*>>::value>>
	{
		static bool ImplementsBatchProcess()
		{
			return true;
		}

		static void BatchProcess(TConstArrayView<UGameFeaturePluginStateMachine*> GFPSMs)
		{
			T::BatchProcess(GFPSMs);
		}
	};

	struct FBatchProcessHelperFunctors
	{
		using FImplementsBatchProcessPtr = bool(*)();
		using FBatchProcessPtr = void (*)(TConstArrayView<UGameFeaturePluginStateMachine*>);

		FBatchProcessHelperFunctors() = default;

		FBatchProcessHelperFunctors(FImplementsBatchProcessPtr InImplementsBatchProcessPtr, FBatchProcessPtr InBatchProcessPtr)
			: ImplementsBatchProcess(InImplementsBatchProcessPtr)
			, BatchProcess(InBatchProcessPtr)
		{

		}
		
		FImplementsBatchProcessPtr ImplementsBatchProcess = nullptr;
		FBatchProcessPtr BatchProcess = nullptr;
	};

	#define GAME_FEATURE_PLUGIN_STATE_MAKE_BATCH_PROCESS_FN(inEnum, inText) FBatchProcessHelperFunctors(&TBatchProcessHelperWrapper<FGameFeaturePluginState_##inEnum>::ImplementsBatchProcess, &TBatchProcessHelperWrapper<FGameFeaturePluginState_##inEnum>::BatchProcess), 
	static TStaticArray<FBatchProcessHelperFunctors, EGameFeaturePluginState::MAX> BatchProcessingHelperFunctors = {
		GAME_FEATURE_PLUGIN_STATE_LIST(GAME_FEATURE_PLUGIN_STATE_MAKE_BATCH_PROCESS_FN)
	};
	#undef GAME_FEATURE_PLUGIN_STATE_MAKE_BATCH_PROCESS_FN
}

UGameFeaturePluginStateMachine::UGameFeaturePluginStateMachine(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bInUpdateStateMachine(false)
	, bRegisteredAsTransitioningGFPSM(false)
{

}

FDestinationGameFeaturePluginState* AsDestinationState(FGameFeaturePluginState* InState)
{
	if (InState)
	{
		return InState->AsDestinationState();
	}
	return nullptr;
}

UE::GameFeatures::FScopedPauseStateMachineExecution::FScopedPauseStateMachineExecution(UE::GameFeatures::FStateMachineExecutor* InExecutor)
: Executor(InExecutor)
{
	if (Executor)
	{
		bOldPaused = Executor->PauseExecution();	
	}
}

UE::GameFeatures::FScopedPauseStateMachineExecution::~FScopedPauseStateMachineExecution()
{
	if (Executor)
	{
		Executor->TryResumeExecution(bOldPaused);
	}
}

UE::GameFeatures::FStateMachineExecutor::FStateMachineExecutor(UGameFeaturesSubsystem* InOwningSubsystem)
	: OwningSubsystem(InOwningSubsystem)
{
}
UE::GameFeatures::FStateMachineExecutor::~FStateMachineExecutor()
{
}

bool UE::GameFeatures::FStateMachineExecutor::PauseExecution()
{
	bool OldValue = bPauseImmediateExecution;
	bPauseImmediateExecution = true;
	return OldValue;
}

void UE::GameFeatures::FStateMachineExecutor::TryResumeExecution(bool bStayPaused)
{
	bPauseImmediateExecution = bStayPaused;
	if (!bPauseImmediateExecution)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FStateMachineExecutor::TryResumeExecution")
		// No time limit based on delta time here, just use the per-frame budget
		UpdateStateMachines(0.0f); 
	}
}

bool UE::GameFeatures::FStateMachineExecutor::UpdateStateMachines(float DeltaTime, int32 MaxIterations)
{
	const double StartTime = UE::GameFeatures::bRealtimeMode ? FPlatformTime::Seconds() : 0.0;
	UAssetManager& LocalAssetManager = UAssetManager::Get();
	LocalAssetManager.PushBulkScanning();
	ON_SCOPE_EXIT 
	{
		LocalAssetManager.PopBulkScanning();
	};

	if (UE::GameFeatures::ExecutionOptions.bGrouped)
	{
		// New execution style, group by state and only allow state machines to make one transition at a time, hoisting the repetition to an outer loop here 
		// Pause execution of re-entrant calls but do not use TryResumeExecution which will try and execute anything queued immediately.
		TGuardValue<bool> GuardPause(bPauseImmediateExecution, true);
		// Separate queues of GFPs to update by current state
		TArray<UGameFeaturePluginStateMachine*> Queues[EGameFeaturePluginState::MAX];
		for (int32 Iteration = 0; Iteration < MaxIterations; ++Iteration)
		{ 
			if (UE::GameFeatures::ExecutionOptions.bShuffle)
			{
				Algo::RandomShuffle(OrderedUpdateQueue);
			}
			else if (UE::GameFeatures::ExecutionOptions.bReverse)
			{
				Algo::Reverse(OrderedUpdateQueue);
			}

			// On every iteration, gather newly queued updates per state so that batches are as large as possible
			for (FStateMachineId Id : OrderedUpdateQueue)
			{
				UGameFeaturePluginStateMachine* StateMachine = OwningSubsystem->GameFeaturePluginStateMachines.FindRef(Id);
				if (ensureMsgf(StateMachine, TEXT("State machine %lld wasn't present in subsystem when trying to update it"), Id))
				{
					StateMachine->StateProperties.bQueuedForUpdate = false;
					Queues[StateMachine->StateProperties.CurrentState].Add(StateMachine);
				}
			}
			OrderedUpdateQueue.Reset();

			// Find the earliest state with work to do
			EGameFeaturePluginState State = EGameFeaturePluginState::UnknownStatus;
			for (EGameFeaturePluginState It : TEnumRange<EGameFeaturePluginState>())
			{
				if (Queues[It].Num() > 0)
				{
					State = It;
					break;
				}
			}

			if (Queues[State].IsEmpty())
			{
				// No more work to do
				break;
			}

			bool bTimeLimitReached = UpdateStateMachinesGrouped(StartTime, DeltaTime, State, MakeArrayView(Queues[State]));
			Queues[State].Reset();
			if (bTimeLimitReached)
			{
				OrderedUpdateQueue.Reserve(OrderedUpdateQueue.Num() + Algo::TransformAccumulate(Queues, UE_PROJECTION(GetNum), 0));
				// UpdateStateMachinesGrouped will already have requeued Queues[State], all other queues need to be requeued
				// They can be queued at the end because we'll sort again next time
				for (EGameFeaturePluginState It : TEnumRange<EGameFeaturePluginState>())
				{
					Algo::Transform(Queues[It], OrderedUpdateQueue, [](UGameFeaturePluginStateMachine* StateMachine)
					{
						StateMachine->StateProperties.bQueuedForUpdate = true;
						return StateMachine->StateProperties.Id;
					});
				}
				break;
			}
		}
	}
	else 
	{
		if (UE::GameFeatures::ExecutionOptions.bShuffle)
		{
			Algo::RandomShuffle(OrderedUpdateQueue);
		}
		else if (UE::GameFeatures::ExecutionOptions.bReverse)
		{
			Algo::Reverse(OrderedUpdateQueue);
		}

		// Closer to original execution style, update all deferred state machines and allow them to make as many transitions as they want, and update other state machines immediately
		TArray<UGameFeaturePluginStateMachine*> LocalUpdateQueue;
		LocalUpdateQueue.Reserve(OrderedUpdateQueue.Num());
		for (const FStateMachineId& Id : OrderedUpdateQueue)
		{
			UGameFeaturePluginStateMachine* StateMachine = OwningSubsystem->GameFeaturePluginStateMachines.FindRef(Id);
			if (ensureMsgf(StateMachine, TEXT("State machine %lld wasn't present in subsystem when trying to update it"), Id))
			{
				LocalUpdateQueue.Add(StateMachine);
				StateMachine->StateProperties.bQueuedForUpdate = false;
			}
		}
		OrderedUpdateQueue.Reset();

		UpdateStateMachinesRealtime(StartTime, DeltaTime, LocalUpdateQueue);
	}

	// Return if this function should be called again next frame
	return OrderedUpdateQueue.Num() > 0;
}

bool UE::GameFeatures::FStateMachineExecutor::UpdateStateMachinesRealtime(double StartTime, float DeltaTime, TConstArrayView<UGameFeaturePluginStateMachine*> LocalUpdateQueue)
{
	if (bRealtimeMode)
	{
		for (int32 i=0; i < LocalUpdateQueue.Num(); ++i)
		{
			UGameFeaturePluginStateMachine* StateMachine = LocalUpdateQueue[i];
			OwningSubsystem->UpdateStateMachine(StateMachine, {});

			const double ElapsedTime = FPlatformTime::Seconds() - StartTime;
			if ((ElapsedTime > RealtimeAllottedTime) || ((DeltaTime + ElapsedTime) > RealtimeMaxFrameTime))
			{
				// Put anything we didn't finish back at the head of the queue
				//-1 because we updated the machine at index i
				TConstArrayView<UGameFeaturePluginStateMachine*> Remaining = LocalUpdateQueue.RightChop(i+1);
				OrderedUpdateQueue.InsertZeroed(0, Remaining.Num());
				for (int32 j=0; j < Remaining.Num(); ++j)
				{
					UGameFeaturePluginStateMachine* StateMachineToRequeue = Remaining[j];
					StateMachineToRequeue->StateProperties.bQueuedForUpdate = true;
					OrderedUpdateQueue[j] = StateMachineToRequeue->StateProperties.Id;
				}

				return true;
			}
		}
	}
	else
	{ 
		for (UGameFeaturePluginStateMachine* StateMachine : LocalUpdateQueue)
		{
			OwningSubsystem->UpdateStateMachine(StateMachine, {});
		}
	}
	return false;
}

bool UE::GameFeatures::FStateMachineExecutor::UpdateStateMachinesGrouped(double StartTime, float DeltaTime, EGameFeaturePluginState State, TConstArrayView<UGameFeaturePluginStateMachine*> StateMachines)
{
	switch (State)
	{
#define GFP_STATE_OP(STATE_NAME, TXT) \
	case EGameFeaturePluginState::STATE_NAME: \
		{ \
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("UpdateStateMachinesGrouped<" #STATE_NAME ">"); \
			return UpdateStateMachinesGrouped<EGameFeaturePluginState::STATE_NAME>(StartTime, DeltaTime, StateMachines);\
		} \
		break;
	GAME_FEATURE_PLUGIN_STATE_LIST(GFP_STATE_OP)
#undef GFP_STATE_OP
	}
	return false;
}

template<EGameFeaturePluginState State>
bool UE::GameFeatures::FStateMachineExecutor::UpdateStateMachinesGrouped(double StartTime, float DeltaTime, TConstArrayView<UGameFeaturePluginStateMachine*> StateMachines)
{
	return UpdateStateMachinesRealtime(StartTime, DeltaTime, StateMachines);
}

void UE::GameFeatures::FStateMachineExecutor::QueueUpdateStateMachine(FGameFeaturePluginStateMachineProperties& InProperties)
{
	if (!InProperties.bQueuedForUpdate)
	{ 
		ensureMsgf(OwningSubsystem->GameFeaturePluginStateMachines.FindRef(InProperties.Id), 
			TEXT("State machine %lld wasn't present in subsystem when trying to update it"), InProperties.Id);
		OwningSubsystem->EnableTick();
		OrderedUpdateQueue.Add(InProperties.Id);
		InProperties.bQueuedForUpdate = true;
	}
}

void UE::GameFeatures::FStateMachineExecutor::TryUpdateStateMachineImmediate(int64 Id)
{
	UGameFeaturePluginStateMachine* StateMachine = OwningSubsystem->GameFeaturePluginStateMachines.FindRef(Id);
	if (ensureMsgf(StateMachine, TEXT("State machine %lld wasn't present in subsystem when trying to update it immediately"), Id))
	{
		if (StateMachine->StateProperties.bQueuedForUpdate)
		{
			return;
		}
		else if (bPauseImmediateExecution)
		{
			QueueUpdateStateMachine(StateMachine->StateProperties);
		}
		else
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("FStateMachineExecutor Exectuting State Machine Immediately");
			OwningSubsystem->UpdateStateMachine(StateMachine);
		}
	}
}

void UE::GameFeatures::FStateMachineExecutor::TryUpdateStateMachineImmediate(FGameFeaturePluginStateMachineProperties& InProperties)
{
	if (InProperties.bQueuedForUpdate)
	{
		return;
	}
	else if (bPauseImmediateExecution)
	{
		QueueUpdateStateMachine(InProperties);
	}
	else 
	{
		UGameFeaturePluginStateMachine* StateMachine = OwningSubsystem->GameFeaturePluginStateMachines.FindRef(InProperties.Id);
		if (ensureMsgf(StateMachine, TEXT("State machine %lld wasn't present in subsystem when trying to update it immediately"), InProperties.Id))
		{
			OwningSubsystem->UpdateStateMachine(StateMachine);
		}
	}
}

void UGameFeaturesSubsystem::InitStateMachine(
	UGameFeaturePluginStateMachine* This,
	int64 InId,
	FGameFeaturePluginIdentifier InPluginIdentifier,
	const FGameFeatureProtocolOptions& InProtocolOptions) 
{
	LLM_SCOPE_BYTAG(GFP);

	check(GetCurrentState(This) == EGameFeaturePluginState::Uninitialized);
	FString PluginName{InPluginIdentifier.GetPluginName()};

	FGameFeaturePluginRequestUpdateStateMachine UpdateCallback;
	if (!StateMachineExecutor)
	{
		UpdateCallback.BindWeakLambda(this, 
			[this, PluginName, WeakThis=MakeWeakObjectPtr(This)]()
			{
				if (UGameFeaturePluginStateMachine* This = WeakThis.Get())
				{
					UpdateStateMachine(This);
				}
				else
				{
					UE_LOGFMT(LogGameFeatures, Warning, "State machine pointer for {Plugin} became unexpectedly null.", PluginName);
				}
			});
	}

	constexpr EGameFeaturePluginState InitialState  = EGameFeaturePluginState::UnknownStatus;
	This->StateProperties = FGameFeaturePluginStateMachineProperties(
		InId,
		MoveTemp(InPluginIdentifier),
		InitialState,
		StateMachineExecutor.Get(),
		UpdateCallback);

	This->StateProperties.ProtocolOptions = InProtocolOptions;
	This->Executor = StateMachineExecutor.Get();

#define GAME_FEATURE_PLUGIN_STATE_MAKE_STATE(inEnum, inText) This->AllStates[EGameFeaturePluginState::inEnum] = MakeUnique<FGameFeaturePluginState_##inEnum>(EGameFeaturePluginState::inEnum, This->StateProperties);
	GAME_FEATURE_PLUGIN_STATE_LIST(GAME_FEATURE_PLUGIN_STATE_MAKE_STATE)
#undef GAME_FEATURE_PLUGIN_STATE_MAKE_STATE

	CheckAddBatchingRequestForCurrentState(This);
	This->AllStates[This->StateProperties.CurrentState]->BeginState();
}

bool UGameFeaturesSubsystem::SetDestination(UGameFeaturePluginStateMachine* This, FGameFeaturePluginStateRange InDestination, FGameFeatureStateTransitionComplete OnFeatureStateTransitionComplete, FDelegateHandle* OutCallbackHandle /*= nullptr*/)
{
	LLM_SCOPE_BYTAG(GFP);

	check(IsValidDestinationState(This, InDestination.MinState));
	check(IsValidDestinationState(This, InDestination.MaxState));

	bool bDestinationSet = false;
	bool bDestinationChanged = false;

#if !UE_BUILD_SHIPPING
	ON_SCOPE_EXIT
	{
		if (bDestinationChanged && GetPluginDebugStateEnabled(GetPluginURL(This)))
		{
			PLATFORM_BREAK();
		}
	};
#endif


	if (!InDestination.IsValid())
	{
		// Invalid range
		return false;
	}
	else if (This->StateProperties.CurrentState == EGameFeaturePluginState::Terminal && !InDestination.Contains(EGameFeaturePluginState::Terminal))
	{
		// Can't transition away from terminal state
		return false;
	}

	// Limit destination to allowed maximum. If requested minimum is beyond that we fail, if requested maximum is beyond it we simply clamp.
	if (InDestination.MinState > This->StateProperties.MaximumState)
	{
		UE_LOGFMT(LogGameFeatures, Warning, "Ignoring request to change game feature destination to {DestinationRange} because maximum state is {MaximumState}",
			InDestination, This->StateProperties.MaximumState);
		return false;
	}

	if (InDestination.MaxState > This->StateProperties.MaximumState)
	{
		InDestination.MaxState = This->StateProperties.MaximumState;
		UE_LOGFMT(LogGameFeatures, Log, "Limited state transition for GFP {Name} to maximum state {MaximumState}", GetGameFeatureName(This), This->StateProperties.MaximumState);
	}

	if (!IsRunning(This))
	{
		// Not running so any new range is acceptable

		if (OutCallbackHandle)
		{
			OutCallbackHandle->Reset();
		}

		FDestinationGameFeaturePluginState* CurrState = This->AllStates[This->StateProperties.CurrentState]->AsDestinationState();

		if (InDestination.Contains(This->StateProperties.CurrentState))
		{
			OnFeatureStateTransitionComplete.ExecuteIfBound(This, MakeValue());
		}
		else
		{
			if (This->StateProperties.CurrentState < InDestination)
			{
				FDestinationGameFeaturePluginState* MinDestState = This->AllStates[InDestination.MinState]->AsDestinationState();
				FDelegateHandle CallbackHandle = MinDestState->OnDestinationStateReached.Add(MoveTemp(OnFeatureStateTransitionComplete));
				if (OutCallbackHandle)
				{
					*OutCallbackHandle = CallbackHandle;
				}
			}
			else if (This->StateProperties.CurrentState > InDestination)
			{
				FDestinationGameFeaturePluginState* MaxDestState = This->AllStates[InDestination.MaxState]->AsDestinationState();
				FDelegateHandle CallbackHandle = MaxDestState->OnDestinationStateReached.Add(MoveTemp(OnFeatureStateTransitionComplete));
				if (OutCallbackHandle)
				{
					*OutCallbackHandle = CallbackHandle;
				}
			}

			This->StateProperties.Destination = InDestination;
			This->StateProperties.UpdateStateMachineImmediate();
			bDestinationChanged = true;
		}

		bDestinationSet = true;
	}
	else if (TOptional<FGameFeaturePluginStateRange> NewDestination = This->StateProperties.Destination.Intersect(InDestination))
	{
		// The machine is already running so we can only transition to this range if it overlaps with our current range.
		// We can satisfy both ranges in this case.

		if (OutCallbackHandle)
		{
			OutCallbackHandle->Reset();
		}

		if (This->StateProperties.CurrentState < This->StateProperties.Destination)
		{
			This->StateProperties.Destination = *NewDestination;

			if (InDestination.Contains(This->StateProperties.CurrentState))
			{
				// TODO: Consider deferring for executor 
				OnFeatureStateTransitionComplete.ExecuteIfBound(This, MakeValue());
			}
			else
			{
				FDestinationGameFeaturePluginState* MinDestState = This->AllStates[InDestination.MinState]->AsDestinationState();
				FDelegateHandle CallbackHandle = MinDestState->OnDestinationStateReached.Add(MoveTemp(OnFeatureStateTransitionComplete));
				if (OutCallbackHandle)
				{
					*OutCallbackHandle = CallbackHandle;
				}
				bDestinationChanged = true;
			}
		}
		else if(This->StateProperties.CurrentState > This->StateProperties.Destination)
		{
			This->StateProperties.Destination = *NewDestination;

			if (InDestination.Contains(This->StateProperties.CurrentState))
			{
				// TODO: Consider deferring for executor 
				OnFeatureStateTransitionComplete.ExecuteIfBound(This, MakeValue());
			}
			else
			{
				FDestinationGameFeaturePluginState* MaxDestState = This->AllStates[InDestination.MaxState]->AsDestinationState();
				FDelegateHandle CallbackHandle = MaxDestState->OnDestinationStateReached.Add(MoveTemp(OnFeatureStateTransitionComplete));
				if (OutCallbackHandle)
				{
					*OutCallbackHandle = CallbackHandle;
				}
				bDestinationChanged = true;
			}
		}
		else
		{
			checkf(false, TEXT("IsRunning() returned true but state machine has reached destination!"));
		}

		bDestinationSet = true;
	}
	else
	{
		// The requested state range is completely outside the the current state range so reject the request
	}

	return bDestinationSet;
}

bool UGameFeaturesSubsystem::TryCancel(UGameFeaturePluginStateMachine* This, FGameFeatureStateTransitionCanceled OnFeatureStateTransitionCanceled, FDelegateHandle* OutCallbackHandle /*= nullptr*/) const
{
	if (!IsRunning(This))
	{
		return false;
	}

	LLM_SCOPE_BYTAG(GFP);

	This->StateProperties.bTryCancel = true;
	FDelegateHandle CallbackHandle = This->StateProperties.OnTransitionCanceled.Add(MoveTemp(OnFeatureStateTransitionCanceled));
	if(OutCallbackHandle)
	{
		*OutCallbackHandle = CallbackHandle;
	}

	const EGameFeaturePluginState CurrentState = GetCurrentState(This);
	This->AllStates[CurrentState]->TryCancelState();

	return true;
}

UE::GameFeatures::FResult UGameFeaturesSubsystem::TryUpdatePluginProtocolOptions(UGameFeaturePluginStateMachine* This, const FGameFeatureProtocolOptions& InOptions, bool& bOutDidUpdate) const
{
	bOutDidUpdate = false;

	if (This->StateProperties.ProtocolOptions == InOptions)
	{
		return MakeValue();
	}

	LLM_SCOPE_BYTAG(GFP);

	const EGameFeaturePluginState CurrentState = GetCurrentState(This);
	UE::GameFeatures::FResult Result = This->AllStates[CurrentState]->TryUpdateProtocolOptions(InOptions);
	bOutDidUpdate = Result.HasValue();

	return Result;
}

void UGameFeaturesSubsystem::RemovePendingTransitionCallback(UGameFeaturePluginStateMachine* This, FDelegateHandle InHandle) const
{
	for (std::underlying_type<EGameFeaturePluginState>::type iState = 0;
		iState < EGameFeaturePluginState::MAX;
		++iState)
	{
		if (FDestinationGameFeaturePluginState* DestState = AsDestinationState(This->AllStates[iState].Get()))
		{
			if (DestState->OnDestinationStateReached.Remove(InHandle))
			{
				break;
			}
		}
	}
}

void UGameFeaturesSubsystem::RemovePendingTransitionCallback(UGameFeaturePluginStateMachine* This, FDelegateUserObject DelegateObject) const
{
	for (std::underlying_type<EGameFeaturePluginState>::type iState = 0;
		iState < EGameFeaturePluginState::MAX;
		++iState)
	{
		if (FDestinationGameFeaturePluginState* DestState = AsDestinationState(This->AllStates[iState].Get()))
		{
			if (DestState->OnDestinationStateReached.RemoveAll(DelegateObject))
			{
				break;
			}
		}
	}
}

void UGameFeaturesSubsystem::RemovePendingCancelCallback(UGameFeaturePluginStateMachine* This, FDelegateHandle InHandle) const
{
	This->StateProperties.OnTransitionCanceled.Remove(InHandle);
}

void UGameFeaturesSubsystem::RemovePendingCancelCallback(UGameFeaturePluginStateMachine* This, FDelegateUserObject DelegateObject) const
{
	This->StateProperties.OnTransitionCanceled.RemoveAll(DelegateObject);
}

const FString& UGameFeaturesSubsystem::GetGameFeatureName(const UGameFeaturePluginStateMachine* This) const
{
	FString PluginFilename;
	if (!This->StateProperties.PluginName.IsEmpty())
	{
		return This->StateProperties.PluginName;
	}
	else
	{
		return This->StateProperties.PluginIdentifier.GetFullPluginURL();
	}
}

const FGameFeaturePluginIdentifier& UGameFeaturesSubsystem::GetPluginIdentifier(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.PluginIdentifier;
}

const FString& UGameFeaturesSubsystem::GetPluginURL(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.PluginIdentifier.GetFullPluginURL();
}

const FGameFeatureProtocolMetadata& UGameFeaturesSubsystem::GetProtocolMetadata(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.ProtocolMetadata;
}

const FGameFeatureProtocolOptions& UGameFeaturesSubsystem::GetProtocolOptions(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.ProtocolOptions;
}

FGameFeatureProtocolOptions UGameFeaturesSubsystem::RecycleProtocolOptions(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.RecycleProtocolOptions();
}

const FString& UGameFeaturesSubsystem::GetPluginName(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.PluginName;
}

bool UGameFeaturesSubsystem::GetPluginFilename(const UGameFeaturePluginStateMachine* This, FString& OutPluginFilename) const
{
	OutPluginFilename = This->StateProperties.PluginInstalledFilename;
	return !OutPluginFilename.IsEmpty();
}

EGameFeaturePluginState UGameFeaturesSubsystem::GetCurrentState(const UGameFeaturePluginStateMachine* This) const
{
	return GetCurrentStateInfo(This).State;
}

FGameFeaturePluginStateRange UGameFeaturesSubsystem::GetDestination(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.Destination;
}

FGameFeaturePluginStateInfo UGameFeaturesSubsystem::GetCurrentStateInfo(const UGameFeaturePluginStateMachine* This) const
{
	return FGameFeaturePluginStateInfo{ This->StateProperties.CurrentState, This->StateProperties.StateProgress };
}

bool UGameFeaturesSubsystem::IsRunning(const UGameFeaturePluginStateMachine* This) const
{
	return !This->StateProperties.Destination.Contains(This->StateProperties.CurrentState);
}

bool UGameFeaturesSubsystem::IsStatusKnown(const UGameFeaturePluginStateMachine* This) const
{
	return GetCurrentState(This) == EGameFeaturePluginState::ErrorUnavailable || 
		   GetCurrentState(This) == EGameFeaturePluginState::Uninstalling || 
		   GetCurrentState(This) == EGameFeaturePluginState::ErrorUninstalling ||
		   GetCurrentState(This) >= EGameFeaturePluginState::StatusKnown;
}

bool UGameFeaturesSubsystem::IsAvailable(const UGameFeaturePluginStateMachine* This) const
{
	ensure(IsStatusKnown(This));
	return GetCurrentState(This) >= EGameFeaturePluginState::StatusKnown;
}

bool UGameFeaturesSubsystem::IsInErrorState(const UGameFeaturePluginStateMachine* This) const
{
	return IsValidErrorState(This, GetCurrentState(This));
}

bool UGameFeaturesSubsystem::AllowAsyncLoading(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.AllowAsyncLoading();
}

bool UGameFeaturesSubsystem::HasAssetStreamingDependencies(const UGameFeaturePluginStateMachine* This) const
{
	ensure(IsStatusKnown(This));
	if (This->StateProperties.ProtocolMetadata.HasSubtype<FInstallBundlePluginProtocolMetaData>())
	{
		const FInstallBundlePluginProtocolMetaData& ProtocolData = This->StateProperties.ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();
		return !ProtocolData.InstallBundlesWithAssetDependencies.IsEmpty();
	}
	return false;
}

void UGameFeaturesSubsystem::SetWasLoadedAsBuiltIn(UGameFeaturePluginStateMachine* This)
{
	This->StateProperties.bWasLoadedAsBuiltInGameFeaturePlugin = true;
}

bool UGameFeaturesSubsystem::WasLoadedAsBuiltIn(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties.bWasLoadedAsBuiltInGameFeaturePlugin;
}

UGameFeatureData* UGameFeaturesSubsystem::GetGameFeatureDataForActivePlugin(UGameFeaturePluginStateMachine* This) const
{
	if (GetCurrentState(This) == EGameFeaturePluginState::Active)
	{
		return This->StateProperties.GameFeatureData;
	}

	return nullptr;
}

const UGameFeatureData* UGameFeaturesSubsystem::GetGameFeatureDataForActivePlugin(const UGameFeaturePluginStateMachine* This) const
{
	if (GetCurrentState(This) == EGameFeaturePluginState::Active)
	{
		return This->StateProperties.GameFeatureData;
	}

	return nullptr;
}

UGameFeatureData* UGameFeaturesSubsystem::GetGameFeatureDataForRegisteredPlugin(UGameFeaturePluginStateMachine* This, bool bCheckForRegistering /*= false*/) const
{
	const EGameFeaturePluginState CurrentState = GetCurrentState(This);

	if (CurrentState >= EGameFeaturePluginState::Registered || (bCheckForRegistering && (CurrentState == EGameFeaturePluginState::Registering)))
	{
		return This->StateProperties.GameFeatureData;
	}

	return nullptr;
}

const UGameFeatureData* UGameFeaturesSubsystem::GetGameFeatureDataForRegisteredPlugin(const UGameFeaturePluginStateMachine* This, bool bCheckForRegistering /*= false*/) const
{
	const EGameFeaturePluginState CurrentState = GetCurrentState(This);

	if (CurrentState >= EGameFeaturePluginState::Registered || (bCheckForRegistering && (CurrentState == EGameFeaturePluginState::Registering)))
	{
		return This->StateProperties.GameFeatureData;
	}

	return nullptr;
}

const FGameFeaturePluginStateMachineProperties& UGameFeaturesSubsystem::GetProperties(const UGameFeaturePluginStateMachine* This) const
{
	return This->StateProperties;
}

bool UGameFeaturesSubsystem::IsErrorStateUnrecoverable(const UGameFeaturePluginStateMachine* This) const
{
	return This->bIsInUnrecoverableError;
}

void UGameFeaturesSubsystem::SetUnrecoverableError(UGameFeaturePluginStateMachine* This) const
{
	This->bIsInUnrecoverableError = true;
}

bool UGameFeaturesSubsystem::IsValidTransitionState(const UGameFeaturePluginStateMachine* This, EGameFeaturePluginState InState) const
{
	check(InState != EGameFeaturePluginState::MAX);
	return This->AllStates[InState]->GetStateType() == EGameFeaturePluginStateType::Transition;
}

bool UGameFeaturesSubsystem::IsValidDestinationState(const UGameFeaturePluginStateMachine* This, EGameFeaturePluginState InDestinationState) const
{
	check(InDestinationState != EGameFeaturePluginState::MAX);
	return This->AllStates[InDestinationState]->GetStateType() == EGameFeaturePluginStateType::Destination;
}

bool UGameFeaturesSubsystem::IsValidErrorState(const UGameFeaturePluginStateMachine* This, EGameFeaturePluginState InDestinationState) const
{
	check(InDestinationState != EGameFeaturePluginState::MAX);
	return This->AllStates[InDestinationState]->GetStateType() == EGameFeaturePluginStateType::Error;
}

UE_TRACE_EVENT_BEGIN(Cpu, GFP_UpdateStateMachine, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PluginName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, GFP_UpdateStateMachine_BeginState, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, State)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, GFP_UpdateStateMachine_UpdateState, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, State)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Cpu, GFP_UpdateStateMachine_EndState, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, State)
UE_TRACE_EVENT_END()

void UGameFeaturesSubsystem::UpdateStateMachine(UGameFeaturePluginStateMachine* This, TOptional<int32> InMaxTransitions) 
{
	LLM_SCOPE_BYTAG(GFP);

	const EGameFeaturePluginState InitialState = GetCurrentState(This);
	EGameFeaturePluginState CurrentState = InitialState;
	if (This->bInUpdateStateMachine)
	{
		UE_LOGF(LogGameFeatures, Verbose, "Game feature state machine skipping update for %ls in ::UpdateStateMachine. Current State: %ls", *GetGameFeatureName(This), *UE::GameFeatures::ToString(CurrentState));
		return;
	}

	UE_TRACE_LOG_SCOPED_T(Cpu, GFP_UpdateStateMachine, CpuChannel)
		<< GFP_UpdateStateMachine.PluginName(*GetGameFeatureName(This));

	TOptional<TGuardValue<bool>> ScopeGuard(InPlace, This->bInUpdateStateMachine, true);
	
	using StateIt = std::underlying_type<EGameFeaturePluginState>::type;

	auto DoCallbacks = [This](const UE::GameFeatures::FResult& Result, StateIt Begin, StateIt End)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_UpdateStateMachine_DoCallbacks);

		for (StateIt iState = Begin; iState < End; ++iState)
		{
			if (FDestinationGameFeaturePluginState* DestState = This->AllStates[iState]->AsDestinationState())
			{
				// Use a local callback on the stack. If SetDestination() is called from the callback then we don't want to stomp the callback
				// for the new state transition request.
				// Callback from terminal state could also trigger a GC that would destroy the state machine
				UE::GameFeatures::FBroadcastingOnDestinationStateReached LocalOnDestinationStateReached(MoveTemp(DestState->OnDestinationStateReached));
				DestState->OnDestinationStateReached.Clear();

				LocalOnDestinationStateReached.CallbackDelegate.Broadcast(This, Result);
			}
		}
	};

	auto DoCallback = [&DoCallbacks](const UE::GameFeatures::FResult& Result, StateIt InState)
	{
		DoCallbacks(Result, InState, InState + 1);
	};

	RegisterAsTransitioningStateMachine(This);

	bool bKeepProcessing = false;
	int32 NumTransitions = 0;
	const int32 MaxTransitions = InMaxTransitions.Get(10000);
	do
	{
		bKeepProcessing = false;

		FGameFeaturePluginStateStatus StateStatus;
		{
			const FGameFeaturePluginStateMachineProperties& Props = This->AllStates[CurrentState]->StateProperties;
			FName LoaderName = Props.GameFeatureData ? Props.GameFeatureData->GetPackage()->GetFName() : FName(Props.PluginName);
			UE_TRACK_REFERENCING_PACKAGE_SCOPED(LoaderName, UE::GameFeatures::GetStateName(CurrentState));

			UE_TRACE_LOG_SCOPED_T(Cpu, GFP_UpdateStateMachine_UpdateState, CpuChannel)
				<< GFP_UpdateStateMachine_UpdateState.State(*UE::GameFeatures::ToString(CurrentState));
			This->AllStates[CurrentState]->UpdateState(StateStatus);
		}

		if (StateStatus.TransitionToState == CurrentState)
		{
			UE_LOGF(LogGameFeatures, Fatal, "Game feature state %ls transitioning to itself. GameFeature: %ls", *UE::GameFeatures::ToString(CurrentState), *GetGameFeatureName(This));
		}

		if (StateStatus.TransitionToState != EGameFeaturePluginState::Uninitialized)
		{
			UE_LOGF(LogGameFeatures, Verbose, "Game feature '%ls' transitioning state (%ls -> %ls)", *GetGameFeatureName(This), *UE::GameFeatures::ToString(CurrentState), *UE::GameFeatures::ToString(StateStatus.TransitionToState));
			{
				UE_TRACE_LOG_SCOPED_T(Cpu, GFP_UpdateStateMachine_EndState, CpuChannel)
					<< GFP_UpdateStateMachine_EndState.State(*UE::GameFeatures::ToString(CurrentState));
				This->AllStates[CurrentState]->EndState();
				CheckAndCancelBatchingRequestForCurrentState(This);
			}
			This->StateProperties.CurrentState = StateStatus.TransitionToState;
			This->StateProperties.StateProgress = 0.0f;
			CurrentState = StateStatus.TransitionToState;
			check(CurrentState != EGameFeaturePluginState::MAX);

			const FGameFeaturePluginStateMachineProperties& Props = This->AllStates[CurrentState]->StateProperties;
			FName LoaderName = Props.GameFeatureData ? Props.GameFeatureData->GetPackage()->GetFName() : FName(Props.PluginName);
			UE_TRACK_REFERENCING_PACKAGE_SCOPED(LoaderName, UE::GameFeatures::GetStateName(CurrentState));

			{
				UE_TRACE_LOG_SCOPED_T(Cpu, GFP_UpdateStateMachine_BeginState, CpuChannel)
					<< GFP_UpdateStateMachine_BeginState.State(*UE::GameFeatures::ToString(CurrentState));
				CheckAddBatchingRequestForCurrentState(This);
				This->AllStates[CurrentState]->BeginState();
			}

			if (CurrentState == EGameFeaturePluginState::Terminal)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(GFP_UpdateStateMachine_BeginTerm);

				// Remove from gamefeature subsystem before calling back in case this GFP is reloaded on callback,
				// but make sure we don't get destroyed from a GC during a callback
				BeginTermination(This);
			}

			if (This->StateProperties.bTryCancel && This->AllStates[CurrentState]->GetStateType() != EGameFeaturePluginStateType::Transition)
			{
				This->StateProperties.Destination = FGameFeaturePluginStateRange(CurrentState);

				This->StateProperties.bTryCancel = false;
				bKeepProcessing = false;

				// Make sure This->bInUpdateStateMachine is not set while processing callbacks if we are at our destination
				ScopeGuard.Reset();

				// For all callbacks, return the CanceledResult
				DoCallbacks(UE::GameFeatures::CanceledResult, 0, EGameFeaturePluginState::MAX);

				// Must be called after transtition callbacks, UGameFeaturesSubsystem::ChangeGameFeatureTargetStateComplete may remove the this machine from the subsystem
				UE::GameFeatures::FBroadcastingOnTransitionCanceled LocalOnTransitionCanceled(MoveTemp(This->StateProperties.OnTransitionCanceled));
				This->StateProperties.OnTransitionCanceled.Clear();
				LocalOnTransitionCanceled.CallbackDelegate.Broadcast(This);
			}
			else if (const bool bError = !StateStatus.TransitionResult.HasValue(); bError)
			{
				check(IsValidErrorState(This, CurrentState));
				This->StateProperties.Destination = FGameFeaturePluginStateRange(CurrentState);

				bKeepProcessing = false;
				
				// Make sure This->bInUpdateStateMachine is not set while processing callbacks if we are at our destination
				ScopeGuard.Reset();

				// In case of an error, callback all possible callbacks
				DoCallbacks(StateStatus.TransitionResult, 0, EGameFeaturePluginState::MAX);
			}
			else
			{
				bKeepProcessing = This->AllStates[CurrentState]->GetStateType() == EGameFeaturePluginStateType::Transition || !This->StateProperties.Destination.Contains(CurrentState);
				if (!bKeepProcessing)
				{
					// Make sure This->bInUpdateStateMachine is not set while processing callbacks if we are at our destination
					ScopeGuard.Reset();
				}

				DoCallback(StateStatus.TransitionResult, CurrentState);
			}

			if (!bKeepProcessing)
			{
				UnregisterAsTransitioningStateMachine(This);
			}

			if (CurrentState == EGameFeaturePluginState::Terminal)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(GFP_UpdateStateMachine_FinishTerm);

				UnregisterAsTransitioningStateMachine(This);

				check(bKeepProcessing == false);
				// Now that callbacks are done this machine can be cleaned up
				FinishTermination(This);
				This->MarkAsGarbage();
			}
		}
		else if(!IsRunning(This))
		{
			UnregisterAsTransitioningStateMachine(This);
		}

		// Log our final state if we've finished transitioning
		if (!bKeepProcessing && InitialState != CurrentState)
		{
			if (!StateStatus.TransitionResult.HasValue())
			{
				constexpr static const TCHAR ErrLogFmt[] = TEXT("Game feature '%s' transition failed. Ending state: %s [%s, %s]. Result: %s");
				if (StateStatus.bSuppressErrorLog)
				{
					UE_LOG(LogGameFeatures, Display, ErrLogFmt,
						*GetGameFeatureName(This),
						*UE::GameFeatures::ToString(CurrentState),
						*UE::GameFeatures::ToString(This->StateProperties.Destination.MinState),
						*UE::GameFeatures::ToString(This->StateProperties.Destination.MaxState),
						*UE::GameFeatures::ToString(StateStatus.TransitionResult));
				}
				else
				{
					UE_LOG(LogGameFeatures, Error, ErrLogFmt,
						*GetGameFeatureName(This),
						*UE::GameFeatures::ToString(CurrentState),
						*UE::GameFeatures::ToString(This->StateProperties.Destination.MinState),
						*UE::GameFeatures::ToString(This->StateProperties.Destination.MaxState),
						*UE::GameFeatures::ToString(StateStatus.TransitionResult));
				}
			}
			else if (This->StateProperties.Destination.Contains(CurrentState))
			{
				UE_LOGF(LogGameFeatures, Display, "Game feature '%ls' transitioned successfully. Ending state: %ls [%ls, %ls]",
					*GetGameFeatureName(This),
					*UE::GameFeatures::ToString(CurrentState),
					*UE::GameFeatures::ToString(This->StateProperties.Destination.MinState),
					*UE::GameFeatures::ToString(This->StateProperties.Destination.MaxState));
			}
		}

		if (++NumTransitions >= MaxTransitions)
		{
			UE_CLOGF(!InMaxTransitions.IsSet(), LogGameFeatures, Fatal, "Infinite loop in game feature state machine transitions. Current state %ls. GameFeature: %ls", *UE::GameFeatures::ToString(CurrentState), *GetGameFeatureName(This));
			// If we aren't attempting to do all transitions at once, make sure we queue up another UpdateStateMachine if there was still work to do
			// (e.g. we just transitioned to a new state which wants UpdateState to be called for the first time)
			if (bKeepProcessing)
			{
				This->StateProperties.UpdateStateMachineDeferred(0.0f);
			}
			break;
		}
	} while (bKeepProcessing);
}

void UGameFeaturesSubsystem::RegisterAsTransitioningStateMachine(UGameFeaturePluginStateMachine* This) 
{
	if (This->bRegisteredAsTransitioningGFPSM)
	{
		return;
	}

	RegisterRunningStateMachine(This);
	This->bRegisteredAsTransitioningGFPSM = true;
}

void UGameFeaturesSubsystem::UnregisterAsTransitioningStateMachine(UGameFeaturePluginStateMachine* This) 
{
	if (!This->bRegisteredAsTransitioningGFPSM)
	{
		return;
	}

	UnregisterRunningStateMachine(This);
	This->bRegisteredAsTransitioningGFPSM = false;
}

void UGameFeaturesSubsystem::CheckAddBatchingRequestForCurrentState(UGameFeaturePluginStateMachine* This) 
{
	check(!This->StateProperties.BatchProcessingHandle.IsValid());

	const bool bCanBatchProcess =
		UE::GameFeatures::BatchProcessingHelperFunctors[This->StateProperties.CurrentState].ImplementsBatchProcess() &&
		This->StateProperties.CanBatchProcess() &&
		This->AllStates[This->StateProperties.CurrentState]->CanBatchProcess();
	
	if (bCanBatchProcess)
	{
		UE_LOGF(LogGameFeatures, Verbose, "Game feature '%ls' awaiting batch processing of state (%ls)", *GetGameFeatureName(This), *UE::GameFeatures::ToString(This->StateProperties.CurrentState));
		This->StateProperties.BatchProcessingHandle = AddBatchingRequest(
			This->StateProperties.CurrentState,
			This->StateProperties.OnRequestUpdateStateMachine);
	}
}

void UGameFeaturesSubsystem::CheckAndCancelBatchingRequestForCurrentState(UGameFeaturePluginStateMachine* This) 
{
	// Reset batch processing state.
	This->StateProperties.bWasBatchProcessed = false;

	// If we are currently awaiting batch processing, cancel and update the state machine.
	if (This->StateProperties.IsWaitingForBatchProcessing())
	{
		UE_LOGF(LogGameFeatures, Verbose, "Game feature '%ls' cancelled batch processing of state (%ls)", *GetGameFeatureName(This), *UE::GameFeatures::ToString(This->StateProperties.CurrentState));
		CancelBatchingRequest(
			This->StateProperties.CurrentState,
			This->StateProperties.BatchProcessingHandle);

		This->StateProperties.BatchProcessingHandle.Reset();
		UpdateStateMachine(This);
	}
}

/* static */ void UGameFeaturesSubsystem::BatchProcess(EGameFeaturePluginState State, TConstArrayView<UGameFeaturePluginStateMachine*> GFPSMs)
{
	UE::GameFeatures::BatchProcessingHelperFunctors[State].BatchProcess(GFPSMs);
	for (UGameFeaturePluginStateMachine* GFPSM : GFPSMs)
	{
		GFPSM->StateProperties.BatchProcessingHandle.Reset();
		GFPSM->StateProperties.bWasBatchProcessed = true;
	}
}

void UGameFeaturesSubsystem::ExcludeFromBatchProcessing(UGameFeaturePluginStateMachine* This) 
{
	// Ensure protocol options are updated to reflect the exclusion of this state machine from batch processing.
	if (This->StateProperties.ProtocolOptions.bBatchProcess)
	{
		UE_LOGF(LogGameFeatures, Verbose, "Game feature '%ls' excluded from batch processing", *GetGameFeatureName(This));

		FGameFeatureProtocolOptions NewOptions = This->StateProperties.ProtocolOptions;
		NewOptions.bBatchProcess = false;

		bool bOutDidUpdate = false;
		TryUpdatePluginProtocolOptions(This, NewOptions, bOutDidUpdate);
		check(bOutDidUpdate);
		CheckAndCancelBatchingRequestForCurrentState(This);
	}	
}

void UGameFeaturesSubsystem::TryUpdateStateMachineImmediate(int64 Id)
{
	if (StateMachineExecutor)
	{
		StateMachineExecutor->TryUpdateStateMachineImmediate(Id);
	}
	else if (UGameFeaturePluginStateMachine* StateMachine = GameFeaturePluginStateMachines.FindRef(Id))
	{
		UpdateStateMachine(StateMachine);
	}
}

FGameFeaturePluginStateMachineProperties::FGameFeaturePluginStateMachineProperties(
	int64 InId,
	FGameFeaturePluginIdentifier InPluginIdentifier,
	EGameFeaturePluginState InitialState,
	UE::GameFeatures::FStateMachineExecutor* InExecutor,
	const FGameFeaturePluginRequestUpdateStateMachine& RequestUpdateStateMachineDelegate)
	: Id(InId)
	, CurrentState(InitialState)
	, PluginIdentifier(MoveTemp(InPluginIdentifier))
	, Destination(FGameFeaturePluginStateRange(InitialState))
	, OnRequestUpdateStateMachine(RequestUpdateStateMachineDelegate)
	, Executor(InExecutor)
{
}

void FGameFeaturePluginStateMachineProperties::UpdateStateMachineImmediate() 
{
	if (Executor)
	{
		Executor->TryUpdateStateMachineImmediate(*this);
	}
	else
	{
		OnRequestUpdateStateMachine.ExecuteIfBound();
	}
}

bool FGameFeaturePluginStateMachineProperties::UpdateStateMachineDeferred(float Delay) 
{
	if (Executor)
	{
		Executor->QueueUpdateStateMachine(*this);
		return true;
	}
	return false;
}

EGameFeaturePluginProtocol FGameFeaturePluginStateMachineProperties::GetPluginProtocol() const
{
	return PluginIdentifier.GetPluginProtocol();
}

FString FInstallBundlePluginProtocolMetaData::ToString() const
{
	FString ReturnedString;

	//Always encode InstallBundles
	ReturnedString = FString(UE::GameFeatures::PluginURLStructureInfo::OptionSeperator) +
		LexToString(EGameFeatureURLOptions::Bundles) + UE::GameFeatures::PluginURLStructureInfo::OptionAssignOperator;

	FNameBuilder NameBuilder;
	const FString BundlesList = FString::JoinBy(InstallBundles, UE::GameFeatures::PluginURLStructureInfo::OptionListSeperator,
	[&NameBuilder](const FName& BundleName)
	{
		BundleName.ToString(NameBuilder);
		return FStringView(NameBuilder);
	});
	ReturnedString.Append(BundlesList);

	// Only the generic version of CountBits is constexpr...
	static_assert(FGenericPlatformMath::CountBits64(static_cast<uint64>(EGameFeatureURLOptions::All)) == 1, "Update this function to handle the newly added EGameFeatureInstallBundleProtocolOptions value!");

	return ReturnedString;
}

TValueOrError<FInstallBundlePluginProtocolMetaData, FString> FInstallBundlePluginProtocolMetaData::FromString(FStringView URLOptionsString)
{
	TArray<FName> InstallBundles;

	bool bParseSuccess = UGameFeaturesSubsystem::ParsePluginURLOptions(URLOptionsString, EGameFeatureURLOptions::Bundles,
	[&InstallBundles](EGameFeatureURLOptions Option, FStringView OptionName, FStringView OptionValue)
	{
		check(Option == EGameFeatureURLOptions::Bundles);
		InstallBundles.Emplace(OptionValue);
	});

	//We require to have InstallBundle names for this URL parse to be correct
	if (!bParseSuccess || InstallBundles.Num() == 0)
	{
		bParseSuccess = false;
		UE_LOGF(LogGameFeatures, Error, "Error parsing InstallBundle protocol options URL %.*ls", URLOptionsString.Len(), URLOptionsString.GetData());
		return MakeError(TEXTVIEW("Bad_PluginURL"));
	}

	FInstallBundlePluginProtocolMetaData Ret;
	Ret.InstallBundles = MoveTemp(InstallBundles);

	return MakeValue<FInstallBundlePluginProtocolMetaData>(MoveTemp(Ret));
}

TValueOrError<void, FString> FGameFeaturePluginStateMachineProperties::ParseURL()
{
	const FStringView BadUrlError = TEXTVIEW("Bad_PluginURL");

	if (!ensureMsgf(!PluginIdentifier.IdentifyingURLSubset.IsEmpty(), TEXT("Unexpected empty IdentifyingURLSubset while parsing URL!")))
	{
		return MakeError(BadUrlError);
	}

	FStringView PluginPathFromURL;
	FStringView URLOptions;
	if (!UGameFeaturesSubsystem::ParsePluginURL(PluginIdentifier.GetFullPluginURL(), nullptr, &PluginPathFromURL, &URLOptions))
	{
		return MakeError(BadUrlError);
	}

	PluginInstalledFilename = PluginPathFromURL;
	PluginName = FPaths::GetBaseFilename(PluginInstalledFilename);

	if (PluginInstalledFilename.IsEmpty() || !PluginInstalledFilename.EndsWith(TEXT(".uplugin")))
	{
		ensureMsgf(false, TEXT("PluginInstalledFilename must have a uplugin extension. PluginInstalledFilename: %s"), *PluginInstalledFilename);
		return MakeError(BadUrlError);
	}

	//Do additional parsing of our Metadata from the options on our remaining URL
	if (GetPluginProtocol() == EGameFeaturePluginProtocol::InstallBundle)
	{
		TValueOrError<FInstallBundlePluginProtocolMetaData, FString> MaybeMetaData = FInstallBundlePluginProtocolMetaData::FromString(URLOptions);
		if (MaybeMetaData.HasError())
		{
			ensureMsgf(false, TEXT("Failure to parse URL %s into a valid FInstallBundlePluginProtocolMetaData"), *PluginIdentifier.GetFullPluginURL());
			return MakeError(MaybeMetaData.StealError());
		}

		FInstallBundlePluginProtocolMetaData& MetaData = *ProtocolMetadata.SetSubtype<FInstallBundlePluginProtocolMetaData>();
		MetaData = MaybeMetaData.StealValue();

		// Add default protocol options if they are not set yet
		if (!ProtocolOptions.HasSubtype<FInstallBundlePluginProtocolOptions>())
		{
			if (ProtocolOptions.HasSubtype<FNull>())
			{
				ProtocolOptions.SetSubtype<FInstallBundlePluginProtocolOptions>();
			}
			else
			{
				ensureMsgf(false, TEXT("Protocol options type is incorrect for URL %s"), *PluginIdentifier.GetFullPluginURL());
				return MakeError(BadUrlError);
			}
		}
	}
	else
	{
		// No protocol options for other (file) protocols right now
		if (!ProtocolOptions.HasSubtype<FNull>())
		{
			ensureMsgf(false, TEXT("Protocol options type is incorrect for URL %s"), *PluginIdentifier.GetFullPluginURL());
			return MakeError(BadUrlError);
		}
	}

	static_assert(static_cast<uint8>(EGameFeaturePluginProtocol::Count) == 3, "Update FGameFeaturePluginStateMachineProperties::ParseURL to handle any new Metadata parsing required for new EGameFeaturePluginProtocol. If no metadata is required just increment this counter.");

	return MakeValue();
}

UE::GameFeatures::FResult FGameFeaturePluginStateMachineProperties::ValidateProtocolOptionsUpdate(const FGameFeatureProtocolOptions& NewProtocolOptions) const
{
	if (GetPluginProtocol() == EGameFeaturePluginProtocol::InstallBundle)
	{
		const FStringView ShortUrl = PluginIdentifier.GetIdentifyingString();

		//Should never change our PluginProtocol
		if (!ensureAlwaysMsgf(NewProtocolOptions.HasSubtype<FInstallBundlePluginProtocolOptions>()
								,TEXT("Error with InstallBundle protocol FGameFeaturePluginStateMachineProperties having an invalid ProtocolOptions. URL: %.*s")
								,ShortUrl.Len(), ShortUrl.GetData()))
		{
			return MakeError(UE::GameFeatures::StateMachineErrorNamespace + TEXT("ProtocolOptions.Invalid_Protocol"));
		}

		if (ProtocolOptions.HasSubtype<FInstallBundlePluginProtocolOptions>())
		{
			const FInstallBundlePluginProtocolOptions& OldOptions = ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();
			const FInstallBundlePluginProtocolOptions& NewOptions = NewProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();

			if (!ensureMsgf(OldOptions.bAllowIniLoading == NewOptions.bAllowIniLoading, TEXT("Unexpected change to AllowIniLoading when updating ProtocolOptions. URL: %.*s "), ShortUrl.Len(), ShortUrl.GetData()))
			{
				return MakeError(UE::GameFeatures::StateMachineErrorNamespace + TEXT("ProtocolOptions.Invalid_Update"));
			}
		}

		return MakeValue();
	}

	if (NewProtocolOptions.HasSubtype<FNull>())
	{
		return MakeValue();
	}

	return MakeError(UE::GameFeatures::StateMachineErrorNamespace + TEXT("ProtocolOptions.Unknown_Protocol"));
}

FGameFeatureProtocolOptions FGameFeaturePluginStateMachineProperties::RecycleProtocolOptions() const
{
	FGameFeatureProtocolOptions Result = ProtocolOptions;
	if (Result.HasSubtype<FInstallBundlePluginProtocolOptions>())
	{
		// Don't allow unexpected uninstalls, otherwise respect any flags previously set by the game
		Result.GetSubtype<FInstallBundlePluginProtocolOptions>().bUninstallBeforeTerminate = false;
	}
	return Result;
}

bool FGameFeaturePluginStateMachineProperties::AllowAsyncLoading() const
{
	// Ticking is required for async loading
	// The local bForceSyncLoading should take precedence over UE::GameFeatures::CVarForceAsyncLoad
	return
		!ProtocolOptions.bForceSyncLoading &&
		UGameFeaturesSubsystem::Get().GetPolicy().AllowAsyncLoad(PluginIdentifier.GetFullPluginURL());
}

bool FGameFeaturePluginStateMachineProperties::CanBatchProcess() const
{
	return ProtocolOptions.bBatchProcess && !UE::GameFeatures::bUseNewExecutor && UE::GameFeatures::CVarEnableBatchProcessing.GetValueOnGameThread();
}

bool FGameFeaturePluginStateMachineProperties::IsWaitingForBatchProcessing() const
{
	return BatchProcessingHandle.IsValid();
}

bool FGameFeaturePluginStateMachineProperties::WasBatchProcessed() const
{
	return bWasBatchProcessed;
}

#undef LOCTEXT_NAMESPACE 
