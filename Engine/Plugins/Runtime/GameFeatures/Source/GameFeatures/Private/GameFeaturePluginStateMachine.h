// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Union.h"
#include "GameFeaturesSubsystem.h"
#include "GameFeaturePluginOperationResult.h"
#include "GameFeatureTypes.h"
#include "GameFeaturePluginStateMachine.generated.h"

class UGameFeatureData;
class UGameFrameworkComponentManager;
class UGameFeaturePluginStateMachine;
struct FComponentRequestHandle;
enum class EInstallBundleResult : uint32;
enum class EInstallBundleReleaseResult : uint32;

namespace UE::GameFeatures
{
	extern TAutoConsoleVariable<bool> CVarAllowMissingOnDemandDependencies;
	extern bool bUseNewExecutor;
	extern bool bLoadBuiltin_DeferExecution;
}

/*
*************** GameFeaturePlugin state machine graph ***************
Descriptions for each state are below in EGameFeaturePluginState.
Destination states have a *. These are the only states that external sources can ask to transition to via SetDestinationState().
Error states have !. These states become destinations if an error occurs during a transition.
Transition states are expected to transition the machine to another state after doing some work.

                         +--------------+
                         |              |
                         |Uninitialized |
                         |              |
                         +------+-------+
     +------------+             |
     |     *      |             |
     |  Terminal  <-------------~-----------------------------------------------
     |            |             |                                              |
     +--^------^--+             ----------------------------                   |
        |      |                                           |                   |
        |      |                                    +------v--------+          |
        |      |                                    |      *        |          |
        |      -------------------------------------+ UnknownStatus |          |
        |           ^                      ^        |               |          |
        |           |                      |        +-------+-------+          |
        |           |                      |                |                  |
        |    +------+-------+              |                |                  |
        |    |      *       |              |                |                  |
        |    | Uninstalled  +--------------~--------------->|                  |
        |    |              |              |                |                  |
        |    +------^-------+              |                |                  |
        |           |                      |                |                  |
        |    +------+-------+    *---------+---------+      |                  |
        |    |              |    |         !         |      |                  |
        |    | Uninstalling <----> ErrorUninstalling |      |                  |
        |    |              |    |                   |      |                  |
        |    +---^----------+    +---------+---------+      |                  |
        |        |                         |                |                  |
        |        |    ----------------------                |                  |
        |        |    |                                     |                  |
        |        |    |                     -----------------                  |
        |        |    |                     |                                  |
        |        |    |         +-----------v---+     +--------------------+   |
        |        |    |         |               |     |         !          |   |
        |        |    |         |CheckingStatus <-----> ErrorCheckingStatus+-->|
        |        |    |         |               |     |                    |   |
        |        |    |         +------+------^-+     +--------------------+   |
        |        |    |                |      |                                |
        |        |    |                |      |       +--------------------+   |
        ---------~    |                |      |       |         !          |   |
                 |    |<----------------      --------> ErrorUnavailable   +----
                 |    |                               |                    |
                 |    |                               +--------------------+
                 |    |
            +----+----v----+
            |      *       |
         ---> StatusKnown  +----------------------------------------------
         |  |              |                                 |           |
         |  +----------^---+                                 |           |
         |                                                   |           |
         |                                                   |           |
         |                                                   |           |
         |                                                   |           |
         |                                                   |           |
      +--+---------+      +-------------------+       +------v-------+   |
      |            |      |         !         |       |              |   |
      | Releasing  <------> ErrorManagingData <-------> Downloading  |   |
      |            |      |                   |       |              |   |
      +--^---------+      +-------------------+       +-------+------+   |
         |                                                   |           |
         |                                                   |           |
         |     +-------------+                               |           |
         |     |      *      |                               v           |
         ------+ Installed   <--------------------------------------------
               |             |
               +-^---------+-+
                 |         |
           ------~---------~--------------------------------
           |     |         |                               |
        +--v-----+--+    +-v---------+               +-----v--------------+
        |           |    |           |               |         !          |
        |Unmounting |    | Mounting  <---------------> ErrorMounting      |
        |           |    |           |               |                    |
        +--^-----^--+    +--+--------+               +--------------------+
           |     |          |
           ------~----------~-------------------------------
                 |          |                              |
                 |       +--v--------------------+   +-----+-----------------------+
                 |       |                       |   |         !                   |
                 |       |WaitingForDependencies <---> ErrorWaitingForDependencies |
                 |       |                       |   |                             |
                 |       +-----+-----------------+   +-----------------------------+
                 |             |
                 |    ---------~-----------------------------------
                 |    |        |                                  |
+----------------+----v---+ +--v----------------------+     +-----v-------------------------+
|                         | |                         |     |             !                 |
|AssetDependencyStreamOut | |AssetDependencyStreaming <-----> ErrorAssetDependencyStreaming |
|                         | |                         |     |                               |
+----------------^--------+ +--+----------------------+     +-------------------------------+
                 |             |
           ------~-------------~----------------------------
           |     |             |                           |
        +--v-----+----+  +-----v----- +              +-----v--------------+
        |             |  |            |              |         !          |
        |Unregistering|  |Registering <--------------> ErrorRegistering   |
        |             |  |            |              |                    |
        +--------^----+  ++-----------+              +--------------------+
                 |        |
               +-+--------v-+
               |      *     |
               | Registered |
               |            |
               +-^--------+-+
                 |        |
           ------~--------~---------------------------------------
           |     |        |                               ^      |
        +--v-----+--+  +--v--------+                      |    +-+------------+
        |           |  |           |                      |    |      !       |
        | Unloading |  |  Loading  <----------------------~----> ErrorLoading |
        |           |  |           |                      |    |              |
        +--------^--+  +--+--------+                      |    +--------------+
                 |        |                               |
               +-+--------v-+                             |
               |      *     |                             |
               |   Loaded   |                             |
               |            |                             |
               +-^--------+-+                             |
		         |        |                               |
        +--------+---+  +-v------------------------+   +--+--------------------------+
        |            |  |                          |   |             !               |
        |Deactivating|  |  ActivatingDependencies  <---> ErrorActivatingDependencies |
        |            |  |                          |   |                             |
        +-^----------+  +---------------------+----+   +-----------------------------+
          |                                   |
		  |  +-----------------------------+  |
		  |  |              !              |  |
		  |  |ErrorDeactivatingDependencies|  |
		  |  |                             |  |
		  |  +--^--------------------------+  |
		  |     |                             |
		+-+-----v----------------+          +-v----------+
		|		                 |          |            |
		|DeactivatingDependencies|          | Activating |
		|	   	                 |          |            |
		+----------------------^-+          +---+--------+
		                       |                |
		     	             +-+----------------v-+
                             |          *         |
                             |       Active       |
                             |                    |
                             +--------------------+
*/

namespace UE::GameFeatures
{
	class FStateMachineExecutor
	{
		// State machines are addressed by FGameFeaturePluginStateMachineProperties::Id
		using FStateMachineId = int64;

		// Lists of state machines requiring updates stored by the order in which a deferred update request was made
		TArray<FStateMachineId> OrderedUpdateQueue;
		UGameFeaturesSubsystem* OwningSubsystem = nullptr;

		/** 
		 * If this is true, state machine update requests through TryUpdateStateMachineImmediate will be queued as if
		 * QueueUpdateStateMachine had been called instead 
		 */
		bool bPauseImmediateExecution = false;

		/** 
		 * Update a set of state machines which are expected to be in the same state. 
		 * Returns true in realtime mode if time limit was exhausted.
		 */
		bool UpdateStateMachinesGrouped(double StartTime, float DeltaTime, EGameFeaturePluginState State, TConstArrayView<UGameFeaturePluginStateMachine*> StateMachines);
		
		// Templated to allow custom implementation per state 
		template<EGameFeaturePluginState State>
		bool UpdateStateMachinesGrouped(double StartTime, float DeltaTime, TConstArrayView<UGameFeaturePluginStateMachine*> StateMachines);

		// Helper to update a list of state machines, requeing if we run out of time 
		bool UpdateStateMachinesRealtime(double StartTime, float DeltaTime, TConstArrayView<UGameFeaturePluginStateMachine*> StateMachines);

	public:
		FStateMachineExecutor(UGameFeaturesSubsystem* InOwningSubsystem);
		~FStateMachineExecutor();
		UE_NONCOPYABLE(FStateMachineExecutor);

		/** Update all state machines which are not at their target state and are not waiting on async work to complete */
		bool UpdateStateMachines(float DeltaTime, int32 MaxIterations = 10000);	

		/** 
		 * Pause immediate execution of state machine updates and return whether it was already paused.
		 * The return value should be passed back to TryResumeExecution from a scoped object. 
		 */
		bool PauseExecution();

		/** 
		 * Try and resume immediately executing state machines.
		 * If the argument is false, execution is resumed and any queued updates are performed.
		 */
		void TryResumeExecution(bool bInStayPaused);

		/**
		 * Store the given state machine as needing to be updated to transition to the next state 
		 * TODO: Do we need a delay timer, or can that be replaced with proper events?
		 */
		void QueueUpdateStateMachine(FGameFeaturePluginStateMachineProperties& InProperties);

		/**
		 * If immediate execution is allowed, update the given state machine.
		 * Otherwise queue it for deferred execution.
		 */
		void TryUpdateStateMachineImmediate(FGameFeaturePluginStateMachineProperties& InProperties);
		void TryUpdateStateMachineImmediate(int64 Id);
	};

	struct FScopedPauseStateMachineExecution
	{
		FScopedPauseStateMachineExecution(FStateMachineExecutor* InExecutor);
		~FScopedPauseStateMachineExecution();
		UE_NONCOPYABLE(FScopedPauseStateMachineExecution);
	
	private:
		FStateMachineExecutor* Executor = nullptr;
		bool bOldPaused = false;
	};

}

/* Represents an inclusive range of states which it is acceptable for a GFP to be in */
struct FGameFeaturePluginStateRange
{
	EGameFeaturePluginState MinState = EGameFeaturePluginState::Uninitialized;
	EGameFeaturePluginState MaxState = EGameFeaturePluginState::Uninitialized;

	FGameFeaturePluginStateRange() = default;

	FGameFeaturePluginStateRange(EGameFeaturePluginState InMinState, EGameFeaturePluginState InMaxState)
		: MinState(InMinState), MaxState(InMaxState)
	{}

	explicit FGameFeaturePluginStateRange(EGameFeaturePluginState InState)
		: MinState(InState), MaxState(InState)
	{}

	bool IsValid() const { return MinState <= MaxState; }

	/* Returns whether a given single state is inside this range */
	bool Contains(EGameFeaturePluginState InState) const
	{
		return InState >= MinState && InState <= MaxState;
	}

	/* Returns whether there are any states in common between this range and another range */
	bool Overlaps(const FGameFeaturePluginStateRange& Other) const
	{
		return Other.MinState <= MaxState && Other.MaxState >= MinState;
	}

	/* If there are any states in common between this range and another range, return a new range representing those states. */
	TOptional<FGameFeaturePluginStateRange> Intersect(const FGameFeaturePluginStateRange& Other) const
	{
		TOptional<FGameFeaturePluginStateRange> Intersection;

		if (Overlaps(Other))
		{
			Intersection.Emplace(FMath::Max(Other.MinState, MinState), FMath::Min(Other.MaxState, MaxState));
		}

		return Intersection;
	}

	bool operator==(const FGameFeaturePluginStateRange& Other) const { return MinState == Other.MinState && MaxState == Other.MaxState; }
	bool operator<(const FGameFeaturePluginStateRange& Other) const { return MaxState < Other.MinState; }
	bool operator>(const FGameFeaturePluginStateRange& Other) const { return MinState > Other.MaxState; }
};

inline bool operator<(EGameFeaturePluginState State, const FGameFeaturePluginStateRange& StateRange)
{
	return State < StateRange.MinState;
}

inline bool operator<(const FGameFeaturePluginStateRange& StateRange, EGameFeaturePluginState State)
{
	return StateRange.MaxState < State;
}

inline bool operator>(EGameFeaturePluginState State, const FGameFeaturePluginStateRange& StateRange)
{
	return State > StateRange.MaxState;
}

inline bool operator>(const FGameFeaturePluginStateRange& StateRange, EGameFeaturePluginState State)
{
	return StateRange.MinState > State;
}

void SerializeForLog(FCbWriter& Writer, FGameFeaturePluginStateRange InRange);


struct FInstallBundlePluginProtocolMetaData
{
	FInstallBundlePluginProtocolMetaData() = default;
	FInstallBundlePluginProtocolMetaData(TArray<FName> InInstallBundles) : InstallBundles(MoveTemp(InInstallBundles)) {}

	TArray<FName> InstallBundles;
	TArray<FName> InstallBundlesWithAssetDependencies;

	/** Functions to convert to/from the URL FString representation of this metadata **/
	FString ToString() const;
	static TValueOrError<FInstallBundlePluginProtocolMetaData, FString> FromString(FStringView URLOptionsString);
};

struct FGameFeatureProtocolMetadata : public TUnion<FInstallBundlePluginProtocolMetaData, FNull>
{
	FGameFeatureProtocolMetadata() { SetSubtype<FNull>(); }
	FGameFeatureProtocolMetadata(const FInstallBundlePluginProtocolMetaData& InData) : TUnion(InData) {}
	FGameFeatureProtocolMetadata(FNull InOptions) { SetSubtype<FNull>(InOptions); }
};

/** Notification that a state transition is complete */
DECLARE_DELEGATE_TwoParams(FGameFeatureStateTransitionComplete, UGameFeaturePluginStateMachine* /*Machine*/, const UE::GameFeatures::FResult& /*Result*/);

/** Notification that a state transition is canceled */
DECLARE_DELEGATE_OneParam(FGameFeatureStateTransitionCanceled, UGameFeaturePluginStateMachine* /*Machine*/);

/** A request for other state machine dependencies */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FGameFeaturePluginRequestStateMachineDependencies, const FString& /*DependencyPluginURL*/, TArray<UGameFeaturePluginStateMachine*>& /*OutDependencyMachines*/);

/** A request to update progress for the current state */
DECLARE_DELEGATE_OneParam(FGameFeatureStateProgressUpdate, float Progress);

/** The common properties that can be accessed by the states of the state machine */
USTRUCT()
struct FGameFeaturePluginStateMachineProperties
{
	GENERATED_BODY()

	/** 
	 * Integer identifier for this plugin when the UGameFeaturePluginStateMachine was created.
	 * Is not stable across termination and recreation of the state machine for this plugin. 
	 */
	int64 Id = 0;

	/** Current state of the state machine. Should not be modified by states themselves, only by UpdateStateMachine or FStateMachineExecutor */
	EGameFeaturePluginState CurrentState = EGameFeaturePluginState::Uninitialized;

	/** Maximum allowed state for this plugin to reach.
	 * e.g. some plugins may only be allowed to reach Registered in the editor for their assets to be worked on, but they cannot be activated in PIE. 
	 */
	EGameFeaturePluginState MaximumState = EGameFeaturePluginState::Active;
	
	/** Whether this GFP is in the queue in FStateMachineExecutor */
	bool bQueuedForUpdate = false;

	/** The progress of the current state. Relevant only for transition states. */
	float StateProgress = 0.0f;

	/** 
	* The Identifier used to find this Plugin. Parsed from the supplied PluginURL at creation.
	* Every protocol will have its own style of identifier URL that will get parsed to generate this.
	* For example, if the file is simply on disk, you can use file:../../../YourGameModule/Plugins/MyPlugin/MyPlugin.uplugin
	**/
	FGameFeaturePluginIdentifier PluginIdentifier;

	/** Filename on disk of the .uplugin file. */
	FString PluginInstalledFilename;
	
	/** Name of the plugin. */
	FString PluginName;

	/** Metadata parsed from the URL for a specific protocol. */
	FGameFeatureProtocolMetadata ProtocolMetadata;

	/** Additional options for a specific protocol. */
	FGameFeatureProtocolOptions ProtocolOptions;

	/** The desired state during a transition. */
	FGameFeaturePluginStateRange Destination;

	/** Whether this plugin has an async localization load in-flight */
	bool bIsLoadingLocalizationData : 1 = false;

	/** Tracks whether or not this state machine added the plugin to the plugin manager. */
	bool bAddedPluginToManager : 1 = false;

	/** Was this plugin loaded using LoadBuiltInGameFeaturePlugin */
	bool bWasLoadedAsBuiltInGameFeaturePlugin : 1 = false;

	/** Whether this state machine should attempt to cancel the current transition */
	bool bTryCancel : 1 = false;

	/** Tracks if the current state was batch processed */
	bool bWasBatchProcessed : 1 = false;

	/** Batch Processing handle if requested for batch processed */
	FDelegateHandle BatchProcessingHandle;

	TArray<FName> AddedPrimaryAssetTypes;

	/** The data asset describing this game feature */
	UPROPERTY(Transient)
	TObjectPtr<UGameFeatureData> GameFeatureData = nullptr;

	/** Callbacks for when the current state transition is cancelled */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTransitionCanceled, UGameFeaturePluginStateMachine* /*Machine*/);
	FOnTransitionCanceled OnTransitionCanceled;

	FGameFeaturePluginStateMachineProperties() = default;
	FGameFeaturePluginStateMachineProperties(
		int64 InId,
		FGameFeaturePluginIdentifier InPluginIdentifier,
		EGameFeaturePluginState InitialState,
		UE::GameFeatures::FStateMachineExecutor* Executor,
		const FGameFeaturePluginRequestUpdateStateMachine& RequestUpdateStateMachineDelegate);

	EGameFeaturePluginProtocol GetPluginProtocol() const;

	TValueOrError<void, FString> ParseURL();

	/** Checks to see if any invalid data was changed during a URL update. True if data updated was all values expected to be changed. */
	UE::GameFeatures::FResult ValidateProtocolOptionsUpdate(const FGameFeatureProtocolOptions& NewProtocolOptions) const;

	/** Returns protocol options suitable for reuse by another state machine */
	FGameFeatureProtocolOptions RecycleProtocolOptions() const;

	/** Whether this machine is allowed to be asynchronous */
	bool AllowAsyncLoading() const;

	bool CanBatchProcess() const;
	bool IsWaitingForBatchProcessing() const;
	bool WasBatchProcessed() const;

	void UpdateStateMachineImmediate();
	/** 
	 * Queues an update for this state machine in the executor if it is being used and returns true.
	 * Returns false if the executor is not being used 
	 */
	bool UpdateStateMachineDeferred(float Delay);

private:
	friend class UGameFeaturesSubsystem;

	/** Delegate to request the state machine be updated. */
	FGameFeaturePluginRequestUpdateStateMachine OnRequestUpdateStateMachine;
	
	/* Object responsible for scheduling state machine updates when UE::GameFeatures::bUseNewExecutor is true */
	UE::GameFeatures::FStateMachineExecutor* Executor = nullptr;
};

/** Input and output information for a state's UpdateState */
struct FGameFeaturePluginStateStatus
{
private:
	/** Holds the current error for any state transition. */
	UE::GameFeatures::FResult TransitionResult = MakeValue();

	/** The state to transition to after UpdateState is complete. */
	EGameFeaturePluginState TransitionToState = EGameFeaturePluginState::Uninitialized;

	/** Whether to suppress error logging if TransitionResult is an error. */
	bool bSuppressErrorLog = false;

	friend class UGameFeaturePluginStateMachine;
	friend class UGameFeaturesSubsystem;

public:
	void SetTransition(EGameFeaturePluginState InTransitionToState);
	void SetTransitionError(EGameFeaturePluginState TransitionToErrorState, UE::GameFeatures::FResult TransitionResult, bool bInSuppressErrorLog = false);
};

enum class EGameFeaturePluginStateType : uint8
{
	Transition,
	Destination,
	Error
};

struct FDestinationGameFeaturePluginState;
struct FErrorGameFeaturePluginState;

/** Base class for all game feature plugin states */
struct FGameFeaturePluginState
{
	FGameFeaturePluginState(EGameFeaturePluginState InState, FGameFeaturePluginStateMachineProperties& InStateProperties) 
		: StateProperties(InStateProperties)
		, State(InState) 
	{
	}
	virtual ~FGameFeaturePluginState();

	/** Called when this state becomes the active state */
	virtual void BeginState() {}

	/** Process the state's logic to decide if there should be a state transition. */
	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) {}

	/** Attempt to cancel any pending state transition. */
	virtual void TryCancelState() {}

	/** Called if we have updated the protocol options for this FGameFeaturePluginState.
		Returns false if no update occured or the update failed. True on successful update. */
	virtual UE::GameFeatures::FResult TryUpdateProtocolOptions(const FGameFeatureProtocolOptions& NewOptions);
	
	/** Called when this state is no longer the active state */
	virtual void EndState() {}

	/** Returns the type of state this is */
	virtual EGameFeaturePluginStateType GetStateType() const { return EGameFeaturePluginStateType::Transition; }

	FDestinationGameFeaturePluginState* AsDestinationState();
	FErrorGameFeaturePluginState* AsErrorState();

	/** The common properties that can be accessed by the states of the state machine */
	FGameFeaturePluginStateMachineProperties& StateProperties;
	EGameFeaturePluginState State;

	void UpdateStateMachineDeferred(float Delay = 0.0f) const;
	void UpdateStateMachineImmediate() const;

	void UpdateProgress(float Progress) const;

	virtual bool CanBatchProcess() const;
	bool IsWaitingForBatchProcessing() const;
	bool WasBatchProcessed() const;

protected:
	/** Builds an end FResult with some minimal error information with overrides for common types we 
		need to generate errors from */
	UE::GameFeatures::FResult GetErrorResult(const FString& ErrorCode, const FText OptionalErrorText = FText()) const;
	UE::GameFeatures::FResult GetErrorResult(const FString& ErrorNamespaceAddition, const FString& ErrorCode, const FText OptionalErrorText = FText()) const;
	UE::GameFeatures::FResult GetErrorResult(const FString& ErrorNamespaceAddition, const EInstallBundleResult ErrorResult) const;
	UE::GameFeatures::FResult GetErrorResult(const FString& ErrorNamespaceAddition, const EInstallBundleReleaseResult ErrorResult) const;

	/** Returns true if this state should transition to the Uninstalled state. 
	    returns False if it should just go directly to the Terminal state instead. */
	bool ShouldVisitUninstallStateBeforeTerminal() const;

	bool AllowIniLoading() const;

	bool AllowAsyncLoading() const;
	virtual bool UseAsyncLoading() const;

private:
	void CleanupDeferredUpdateCallbacks() const;

	mutable FTSTicker::FDelegateHandle TickHandle;
};

/** Base class for destination game feature plugin states */
struct FDestinationGameFeaturePluginState : public FGameFeaturePluginState
{
	using FGameFeaturePluginState::FGameFeaturePluginState;

	/** Called when this state is no longer the active state */
	virtual void EndState() override final;
	/** Attempt to cancel any pending state transition. */
	virtual void TryCancelState() override final;

	/** Returns the type of state this is */
	virtual EGameFeaturePluginStateType GetStateType() const override { return EGameFeaturePluginStateType::Destination; }

	/** Process the state's logic to decide if there should be a state transition. */
	virtual void UpdateState(FGameFeaturePluginStateStatus& StateStatus) override final;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDestinationStateReached, UGameFeaturePluginStateMachine* /*Machine*/, const UE::GameFeatures::FResult& /*Result*/);
	FOnDestinationStateReached OnDestinationStateReached;
};

/** Base class for error game feature plugin states */
struct FErrorGameFeaturePluginState : public FDestinationGameFeaturePluginState
{
	using FDestinationGameFeaturePluginState::FDestinationGameFeaturePluginState;

	/** Returns the type of state this is */
	virtual EGameFeaturePluginStateType GetStateType() const override { return EGameFeaturePluginStateType::Error; }
};

/** Information about a given plugin state, used to expose information to external code */
struct FGameFeaturePluginStateInfo
{
	/** The state this info represents */
	EGameFeaturePluginState State = EGameFeaturePluginState::Uninitialized;

	/** The progress of this state. Relevant only for transition states. */
	float Progress = 0.0f;

	FGameFeaturePluginStateInfo() = default;
	explicit FGameFeaturePluginStateInfo(EGameFeaturePluginState InState, float InProgress = 0.0f)
		: State(InState)
		, Progress(InProgress)
	{
	}
};

/** A state machine to manage transitioning a game feature plugin from just a URL into a fully loaded and active plugin, including registering its contents with other game systems */
UCLASS()
class UGameFeaturePluginStateMachine : public UObject
{
	GENERATED_BODY()

public:
	UGameFeaturePluginStateMachine(const FObjectInitializer& ObjectInitializer);

	FDestinationGameFeaturePluginState* AsDestinationState(FGameFeaturePluginState* InState);


	/** The common properties that can be accessed by the states of the state machine */
	UPROPERTY(transient)
	FGameFeaturePluginStateMachineProperties StateProperties;

	/** All state machine state objects */
	TUniquePtr<FGameFeaturePluginState> AllStates[EGameFeaturePluginState::MAX];

	/* Object responsible for scheduling state machine updates when UE::GameFeatures::bUseNewExecutor is true */
	UE::GameFeatures::FStateMachineExecutor* Executor = nullptr;

	/** True when we are currently executing UpdateStateMachine, to avoid reentry */
	bool bInUpdateStateMachine;

	/** True when the state machine can not transition out of the current error state or if requested to would likely fail */
	bool bIsInUnrecoverableError = false;

	/** True when we are registered as a state machine with in flight transitions */
	bool bRegisteredAsTransitioningGFPSM;
};
