// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureStateHandle.h"
#include "GameFeatureStateHandleInternal.h"
#include "GameFeatureStateHandleReferenceController.h"
#include "GameFeaturesSubsystem.h"

FGameFeatureStateHandle::FGameFeatureStateHandle(const FString& Owner, EGameFeatureStateHandleOptions Options)
{
	InitAndRegister(Owner, Options);
}

FGameFeatureStateHandle::~FGameFeatureStateHandle()
{
	ReleaseAndUnregister();
}

FGameFeatureStateHandle::FGameFeatureStateHandle(FGameFeatureStateHandle&& Other)
	: UniqueId(Other.UniqueId)
{
	Other.UniqueId.Invalidate();
}

FGameFeatureStateHandle& FGameFeatureStateHandle::operator=(FGameFeatureStateHandle&& Other)
{
	if (this != &Other)
	{
		ReleaseAndUnregister();

		UniqueId = Other.UniqueId;
		Other.UniqueId.Invalidate();
	}

	return *this;
}

bool FGameFeatureStateHandle::InitAndRegister(const FString& Owner, EGameFeatureStateHandleOptions Options)
{
	if (IsValid())
	{
		return false;
	}

	if (FGameFeatureStateHandleReferenceController::Get().IsLifetimeControlEnabled())
	{
		UniqueId = FGuid::NewGuid();
		FGameFeatureStateHandleReferenceController::Get().RegisterNewStateHandle(*this, Owner, Options);
	}

	return true;
}

void FGameFeatureStateHandle::ReleaseAndUnregister(TFunction<void(bool)> ResetCompleteCallback)
{
	if (IsValid() && !IsEngineExitRequested())
	{
		FGameFeatureStateHandleReferenceController::Get().ResetGameFeatureStateHandle(*this, ResetCompleteCallback);
		FGameFeatureStateHandleReferenceController::Get().UnregisterStateHandle(*this);
		UniqueId.Invalidate();
	}
	else if (ResetCompleteCallback)
	{
		ResetCompleteCallback(false);
	}
}

FGuid FGameFeatureStateHandle::GetUniqueId() const
{
	return UniqueId;
}

void FGameFeatureStateHandle::TakeOwnership(FGameFeatureStateHandle&& FromStateHandle)
{
	if (!IsValid())
	{
		// If we are not valid, lets just take over this state handle
		*this = MoveTemp(FromStateHandle);
	}
	else
	{
		FGameFeatureStateHandleReferenceController::Get().MergeGameFeatureStateHandle(FromStateHandle, *this);
	}
}

FString FGameFeatureStateHandle::ToString() const
{
	return UniqueId.ToString(EGuidFormats::DigitsWithHyphensInParentheses);
}

bool FGameFeatureStateHandle::IsValid() const
{
	return UniqueId.IsValid();
}

#if !UE_BUILD_SHIPPING
int32 FGameFeatureStateHandle::CompareAndLogGameFeatureDifferences(const FGameFeatureStateHandle& FinalStateHandle)
{
	return FGameFeatureStateHandleReferenceController::Get().CompareAndLogGameFeatureDifferences(*this, FinalStateHandle);
}
#endif // !UE_BUILD_SHIPPING

bool FGameFeatureStateHandle::operator==(const FGameFeatureStateHandle& Other) const
{
	return UniqueId == Other.UniqueId;
}

FGameFeatureStateHandleInternal::FGameFeatureStateHandleInternal(const FString& InOwner, EGameFeatureStateHandleOptions InOptions)
	: Owner(InOwner)
	, Options(InOptions)
{
}

FGameFeatureStateHandleInternal::~FGameFeatureStateHandleInternal()
{
}

void FGameFeatureStateHandleInternal::TakeOwnership(FGameFeatureStateHandleInternal& StateHandle)
{
	StateHandleData.Append(StateHandle.StateHandleData);
	StateHandle.Empty();
}

void FGameFeatureStateHandleInternal::AddOrUpdateOwnershipIfHighestDestState(const FString& PluginName, EGameFeaturePluginState DestPluginState)
{
	EGameFeaturePluginState& PluginState = StateHandleData.FindOrAdd(PluginName);
	if (DestPluginState > PluginState)
	{
		PluginState = DestPluginState;
	}
}

bool FGameFeatureStateHandleInternal::FindPluginRequiredState(const FString& PluginName, EGameFeaturePluginState& OutPluginState) const
{
	if (const EGameFeaturePluginState* FoundPluginState = StateHandleData.Find(PluginName))
	{
		OutPluginState = *FoundPluginState;
		return true;
	}

	OutPluginState = EGameFeaturePluginState::UnknownStatus;
	return false;
}

const TArray<FString> FGameFeatureStateHandleInternal::GetPlugins() const
{
	TArray<FString> OutPlugins;
	StateHandleData.GetKeys(OutPlugins);

	return OutPlugins;
}

int FGameFeatureStateHandleInternal::GetPluginCount() const
{
	return StateHandleData.Num();
}

void FGameFeatureStateHandleInternal::Remove(const FString& PluginName)
{
	StateHandleData.Remove(PluginName);
}

// Should only be called after you ResetGameFeatureStateHandle, otherwise you will leak references
void FGameFeatureStateHandleInternal::Empty()
{
	StateHandleData.Empty();
}

bool FGameFeatureStateHandleInternal::IsEmpty() const
{
	return StateHandleData.IsEmpty();
}

FString FGameFeatureStateHandleInternal::ToString() const
{
	TStringBuilder<512> Out;
	Out.Appendf(TEXT("(%i) ("), StateHandleData.Num());
	for (const TPair<FString, EGameFeaturePluginState>& StateHandle : StateHandleData)
	{
		Out.Appendf(TEXT("%s (%s) "), *StateHandle.Key, *UE::GameFeatures::ToString(StateHandle.Value));
	}
	Out.Appendf(TEXT(")"));

	return Out.ToString();
}

FString FGameFeatureStateHandleInternal::GetOwner() const
{
	return Owner;
}

EGameFeatureStateHandleOptions FGameFeatureStateHandleInternal::GetOptions() const
{
	return Options;
}
