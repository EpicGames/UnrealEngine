// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "GameFeatureTypesFwd.h"
#include "GameFeatureStateHandle.h"

struct FScriptContainerElement;

// TODO move this actually in the private area. For now its in a public header, but in the Engine it will be hidden
class FGameFeatureStateHandleInternal
{
public:
	FGameFeatureStateHandleInternal(const FString& Owner, EGameFeatureStateHandleOptions Options);
	FGameFeatureStateHandleInternal(FGameFeatureStateHandleInternal&&) = default;
	FGameFeatureStateHandleInternal& operator=(FGameFeatureStateHandleInternal&& OtherStateHandle) = default;

	~FGameFeatureStateHandleInternal();

	// no copying allowed
	FGameFeatureStateHandleInternal(const FGameFeatureStateHandleInternal& OtherStateHandle) = delete;
	FGameFeatureStateHandleInternal& operator=(const FGameFeatureStateHandleInternal& OtherStateHandle) = delete;

	void TakeOwnership(FGameFeatureStateHandleInternal& StateHandle);

	void AddOrUpdateOwnershipIfHighestDestState(const FString& PluginName, EGameFeaturePluginState DestPluginState);
	bool FindPluginRequiredState(const FString& PluginName, EGameFeaturePluginState& OutPluginState) const;

	const TArray<FString> GetPlugins() const;
	int GetPluginCount() const;

	void Empty();
	bool IsEmpty() const;

	void Remove(const FString& PluginName);

	FString ToString() const;
	FString GetOwner() const;
	EGameFeatureStateHandleOptions GetOptions() const;

private:
	/** Mapping of PluginName -> ref counted MinRequireState */
	TMap<FString, EGameFeaturePluginState> StateHandleData;

	/** Copy of the public options and string owners so we can get this info with out needing a copy of the public state handle */
	FString Owner;
	EGameFeatureStateHandleOptions Options;
};
