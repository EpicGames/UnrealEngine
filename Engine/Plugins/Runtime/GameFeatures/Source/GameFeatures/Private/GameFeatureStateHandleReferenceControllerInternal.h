// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureStateHandle.h"
#include "GameFeatureStateHandleInternal.h"
#include "GameFeatureTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"

class World;
class FOutputDevice;

// Each Ref will be in a map (GFPUrl) -> RefCount
// Each Ref will store the StateHandle that owns a Ref to this GFPUrl and desired min PluginState, and will store the HighestPluginState to make it easier to grab the highest PluginState this GFP should be at
struct FGameFeatureStateRefCount
{
public:
	void AddRef(const FGuid& StateHandleId, EGameFeaturePluginState DestPluginState);

	[[nodiscard]] bool RemoveRefAndRequiresDowngrading(const FGuid& StateHandleId);

	bool GetHighestRefCountDestPluginState(EGameFeaturePluginState& OutDestPluginState) const;
	bool IsEmpty() const;

	const TMap<FGuid, EGameFeaturePluginState>& GetOwnerToDestMap() const;

private:
	// Keep track of the StateHandleID + its highest Destination State
	// Adding a new Owner + Dest -> update highest DestState tracked
	// Removing an Owner -> search owners for next highest Destination State
	TMap<FGuid, EGameFeaturePluginState> OwnerToDestState;
	EGameFeaturePluginState HighestDestPluginState = EGameFeaturePluginState::Uninitialized;
};

/* The private data of the Reference Controller
 * Map of GFPURL -> RefCount
 * Map of Public StateHandle (GUID) -> Private Internal StateHandle
 */
struct FGameFeatureStateHandleReferenceControllerInternal
{
	void ListGameFeatureStateHandles(const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar);

	/** Mapping of GFP Name to RefCount */
	TMap<FString, FGameFeatureStateRefCount> GFPReferenceCount;

	/** Mapping of public StateHandles (GUIDs) to internal storage of our GFPs + PluginState */
	FCriticalSection StateHandlesLock; // a lock for any time StateHandles is used to prevent other threads from causing issues
	TMap<FGuid, FGameFeatureStateHandleInternal> StateHandles;
};
