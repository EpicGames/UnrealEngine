// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureTypes.h"

#include "Containers/UnrealString.h"
#include "Containers/StringView.h"

namespace UE::GameFeatures
{
#define GAME_FEATURE_PLUGIN_STATE_TO_STRING(inEnum, inText) case EGameFeaturePluginState::inEnum: return TEXT(#inEnum);
	FString ToString(EGameFeaturePluginState InType)
	{
		switch (InType)
		{
			GAME_FEATURE_PLUGIN_STATE_LIST(GAME_FEATURE_PLUGIN_STATE_TO_STRING)
		default:
			check(0);
			return FString();
		}
	}
#undef GAME_FEATURE_PLUGIN_STATE_TO_STRING
}

const TCHAR* LexToString(EGameFeatureURLOptions InOption)
{
#define GAME_FEATURE_PLUGIN_URL_OPTIONS_STRING(inEnum, inVal) case EGameFeatureURLOptions::inEnum: return TEXT(#inEnum);
	switch (InOption)
	{
		GAME_FEATURE_PLUGIN_URL_OPTIONS_LIST(GAME_FEATURE_PLUGIN_URL_OPTIONS_STRING)
	}
#undef GAME_FEATURE_PLUGIN_URL_OPTIONS_STRING

	check(0);
	return TEXT("");
}

void LexFromString(EGameFeatureURLOptions& ValueOut, const FStringView& StringIn)
{
	ValueOut = EGameFeatureURLOptions::None;
	
	for (EGameFeatureURLOptions OptionToCheck : MakeFlagsRange(EGameFeatureURLOptions::All))
	{
		if (FStringView(LexToString(OptionToCheck)) == StringIn)
		{
			ValueOut = OptionToCheck;
			return;
		}
	}
}

