// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturesProjectPolicies.h"
#include "GameFeaturesSubsystemSettings.h"
#include "GameFeatureData.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeaturesProjectPolicies)

namespace UE::GameFeatures
{
	static TAutoConsoleVariable<bool> CVarForceAsyncLoad(TEXT("GameFeaturePlugin.ForceAsyncLoad"),
		false,
		TEXT("Enable to force use of async loading even if normally not allowed"));

	const TAutoConsoleVariable<bool>& GetCVarForceAsyncLoad()
	{
		return CVarForceAsyncLoad;
	}
}


EGameFeaturesProjectPoliciesThreadSafetyVersion UGameFeaturesProjectPolicies::GetThreadSafetyVersion() const
{
	if (   GetClass() == UGameFeaturesProjectPolicies::StaticClass()
		|| GetClass() == UDefaultGameFeaturesProjectPolicies::StaticClass())
	{
		return EGameFeaturesProjectPoliciesThreadSafetyVersion::Initial;
	}
	return EGameFeaturesProjectPoliciesThreadSafetyVersion::None;
}

void UDefaultGameFeaturesProjectPolicies::InitGameFeatureManager()
{
	UE_LOGF(LogGameFeatures, Log, "Scanning for built-in game feature plugins");

	auto AdditionalFilter = [&](const FString& PluginFilename, const FGameFeaturePluginDetails& PluginDetails, FBuiltInGameFeaturePluginBehaviorOptions& OutOptions) -> bool
	{
		// By default, force all initially loaded plugins to synchronously load, this overrides the behavior of GameFeaturePlugin.AsyncLoad which will be used for later loads
		OutOptions.bForceSyncLoading = true;

		// By default, no plugins are filtered so we expect all built-in dependencies to be created before their parent GFPs
		OutOptions.bLogWarningOnForcedDependencyCreation = true;

		return true;
	};

	UGameFeaturesSubsystem::Get().LoadBuiltInGameFeaturePlugins(AdditionalFilter);
}

void UDefaultGameFeaturesProjectPolicies::GetGameFeatureLoadingMode(bool& bLoadClientData, bool& bLoadServerData) const
{
	// By default, load both unless we are a dedicated server or client only cooked build
	bLoadClientData = !IsRunningDedicatedServer();
	bLoadServerData = !IsRunningClientOnly();
}

const TArray<FName> UDefaultGameFeaturesProjectPolicies::GetPreloadBundleStateForPlugin(const FString& PluginName) const
{
	// By default, use the bundles corresponding to loading mode
	bool bLoadClientData, bLoadServerData;
	GetGameFeatureLoadingMode(bLoadClientData, bLoadServerData);

	TArray<FName> FeatureBundles;
	if (bLoadClientData)
	{
		FeatureBundles.Add(UGameFeaturesSubsystemSettings::LoadStateClient);
	}
	if (bLoadServerData)
	{
		FeatureBundles.Add(UGameFeaturesSubsystemSettings::LoadStateServer);
	}
	return FeatureBundles;
}

bool UGameFeaturesProjectPolicies::IsLoadingStartupPlugins() const
{
	if (GIsRunning && GFrameCounter > 2)
	{
		// Initial loading can take 2 frames
		return false;
	}

	if (IsRunningCommandlet() && GEngine && GEngine->IsInitialized())
	{
		// Commandlets may not tick, so done after initialization
		return false;
	}

	return true;
}

bool UGameFeaturesProjectPolicies::GetGameFeaturePluginURL(const TSharedRef<IPlugin>& Plugin, FString& OutPluginURL) const
{
	// It could still be a GFP, but state machine may not have been created for it yet
	// Check if it is a built-in GFP
	const FString& PluginDescriptorFilename = Plugin->GetDescriptorFileName();
	if (!PluginDescriptorFilename.IsEmpty())
	{
		if (GetDefault<UGameFeaturesSubsystemSettings>()->IsValidGameFeaturePlugin(FPaths::ConvertRelativePathToFull(PluginDescriptorFilename)))
		{
			OutPluginURL = UGameFeaturesSubsystem::GetPluginURL_FileProtocol(PluginDescriptorFilename);
		}
		else
		{
			OutPluginURL = TEXT("");
		}
		return true;
	}
	return false;
}

bool UGameFeaturesProjectPolicies::WillPluginBeCooked(const FString& PluginFilename, const FGameFeaturePluginDetails& PluginDetails) const
{
	return true;
}

TValueOrError<FString, FString> UGameFeaturesProjectPolicies::ResolvePluginDependency(const FString& PluginURL, const FString& DependencyName, FPluginDependencyDetails& OutDetails) const
{
	OutDetails = {};
	return ResolvePluginDependency(PluginURL, DependencyName);
}

TValueOrError<FString, FString> UGameFeaturesProjectPolicies::ResolvePluginDependency(const FString& PluginURL, const FString& DependencyName) const
{
	FString DependencyURL;
	bool bResolvedDependency = false;

	// Check if UGameFeaturesSubsystem is already aware of it
	if (UGameFeaturesSubsystem::Get().GetPluginURLByName(DependencyName, DependencyURL))
	{
		bResolvedDependency = true;
	}
	// Check if the dependency plugin exists yet (should be true for all built-in plugins)
	else if (TSharedPtr<IPlugin> DependencyPlugin = IPluginManager::Get().FindPlugin(DependencyName))
	{
		bResolvedDependency = GetGameFeaturePluginURL(DependencyPlugin.ToSharedRef(), DependencyURL);
	}

	if (bResolvedDependency)
	{
		return MakeValue(MoveTemp(DependencyURL));
	}

	return MakeError(TEXT("NotFound"));
}

TValueOrError<TArray<EStreamingAssetInstallMode>, FString> UGameFeaturesProjectPolicies::GetStreamingAssetInstallModes(FStringView PluginURL, TConstArrayView<FName> InstallBundleNames) const
{
	TArray<EStreamingAssetInstallMode> InstallModes;
	InstallModes.Reserve(InstallBundleNames.Num());
	while (InstallModes.Num() < InstallBundleNames.Num())
	{
		InstallModes.Emplace(EStreamingAssetInstallMode::Full);
	}

	return MakeValue(MoveTemp(InstallModes));
}

void UGameFeaturesProjectPolicies::ExplicitLoadGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate, const bool bActivateGameFeatures)
{
	if (bActivateGameFeatures)
	{
		UGameFeaturesSubsystem::Get().LoadAndActivateGameFeaturePlugin(PluginURL, CompleteDelegate);
	}
	else
	{
		UGameFeaturesSubsystem::Get().LoadGameFeaturePlugin(PluginURL, CompleteDelegate);
	}
}

bool UGameFeaturesProjectPolicies::AllowAsyncLoad(FStringView /*PluginURL*/) const
{
	return !IsRunningCommandlet() || UE::GameFeatures::CVarForceAsyncLoad.GetValueOnGameThread();
}

FString UGameFeaturesProjectPolicies::GetInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist /*= false*/)
{
	return UGameFeatureData::GetInstallBundleName(PluginName, bEvenIfDoesntExist);
}

FString UGameFeaturesProjectPolicies::GetOptionalInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist /*= false*/)
{
	return UGameFeatureData::GetOptionalInstallBundleName(PluginName, bEvenIfDoesntExist);
}
