// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturesSubsystem.h"
#include "Algo/Accumulate.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/TopologicalSort.h"
#include "Algo/Transform.h"
#include "Algo/Unique.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Mutex.h"
#include "Async/ParallelFor.h"
#include "BundlePrereqCombinedStatusHelper.h"
#include "Containers/Array.h"
#include "Experimental/UnifiedError/UnifiedError.h"
#include "Dom/JsonValue.h"
#include "GameFeaturePluginOperationResult.h"
#include "GameFeaturesSubsystemSettings.h"
#include "GameFeaturesProjectPolicies.h"
#include "GameFeatureData.h"
#include "GameFeaturePluginStateMachine.h"
#include "GameFeatureStateChangeObserver.h"
#include "GameFeatureStateHandleReferenceController.h"
#include "GameplayTagsManager.h"
#include "HAL/Platform.h"
#include "Interfaces/IPluginManager.h"
#include "IO/IoStoreOnDemand.h"
#include "JsonUtils/JsonConversion.h"
#include "JsonUtils/RapidJsonUtils.h"
#include "Logging/LogVerbosity.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/JsonSerializer.h"
#include "Stats/StatsMisc.h"
#include "String/ParseTokens.h"
#include "Tasks/Task.h"
#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"
#include "Engine/AssetManagerSettings.h"
#include "InstallBundleTypes.h"
#include "RenderAssetUpdate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeaturesSubsystem)

DEFINE_LOG_CATEGORY(LogGameFeatures);

FSimpleMulticastDelegate UGameFeaturesSubsystem::OnGameFeaturePolicyPreInit;
FSimpleMulticastDelegate UGameFeaturesSubsystem::OnGameFeaturePolicyPostInit;

namespace UE::GameFeatures
{
	static const FString SubsystemErrorNamespace(TEXT("GameFeaturePlugin.Subsystem."));

	namespace PluginURLStructureInfo
	{
		const TCHAR* OptionAssignOperator = TEXT("=");
		const TCHAR* OptionSeperator = TEXT("?");
		const TCHAR* OptionListSeperator = TEXT(",");
	}

	namespace CommonErrorCodes
	{
		const TCHAR* PluginNotAllowed = TEXT("Plugin_Denied_By_GameSpecificPolicy");
		const TCHAR* PluginFiltered = TEXT("Plugin_Filtered_By_GameSpecificPolicy");
		const TCHAR* PluginDetailsNotFound = TEXT("Plugin_Details_Not_Found");
		const TCHAR* NotGFP = TEXT("Plugin_Not_GFP");
		const TCHAR* PluginURLNotFound = TEXT("Plugin_URL_Not_Found");
		const TCHAR* DependencyFailedRegister = TEXT("Failed_Dependency_Register");
		const TCHAR* BadURL = TEXT("Bad_URL");
		const TCHAR* UnreachableState = TEXT("State_Currently_Unreachable");

		const TCHAR* CancelAddonCode = TEXT("_Cancel");
	}

#define GAME_FEATURE_PLUGIN_PROTOCOL_PREFIX(inEnum, inString) case EGameFeaturePluginProtocol::inEnum: return inString;
	static const TCHAR* GameFeaturePluginProtocolPrefix(EGameFeaturePluginProtocol Protocol)
	{
		switch (Protocol)
		{
			GAME_FEATURE_PLUGIN_PROTOCOL_LIST(GAME_FEATURE_PLUGIN_PROTOCOL_PREFIX)
		}

		check(false);
		return nullptr;
	}
#undef GAME_FEATURE_PLUGIN_PROTOCOL_PREFIX

#if !UE_BUILD_SHIPPING
	static float GBuiltInPluginLoadTimeReportThreshold = 3.0;
	static FAutoConsoleVariableRef CVarBuiltInPluginLoadTimeReportThreshold(
		TEXT("GameFeaturePlugin.BuiltInPluginLoadTimeReportThreshold"),
		GBuiltInPluginLoadTimeReportThreshold,
		TEXT("Built-in plugins that take longer than this amount of time, in seconds, will be reported to the log during startup in non-shipping builds."),
		ECVF_Default);

	static float GBuiltInPluginLoadTimeErrorThreshold = 600;
	static FAutoConsoleVariableRef CVarBuiltInPluginLoadTimeErrorThreshold(
		TEXT("GameFeaturePlugin.BuiltInPluginLoadTimeErrorThreshold"),
		GBuiltInPluginLoadTimeErrorThreshold,
		TEXT("Built-in plugins that take longer than this amount of time, in seconds, will cause an error during startup in non-shipping builds."),
		ECVF_Default);

	static int32 GBuiltInPluginLoadTimeMaxReportCount = 10;
	static FAutoConsoleVariableRef CVarBuiltInPluginLoadTimeMaxReportCount(
		TEXT("GameFeaturePlugin.BuiltInPluginLoadTimeMaxReportCount"),
		GBuiltInPluginLoadTimeMaxReportCount,
		TEXT("When listing worst offenders for built-in plugin load time, show no more than this many plugins, to reduce log spam."),
		ECVF_Default);
#endif // !UE_BUILD_SHIPPING

	static bool bSkipBuiltInGameFeatures = false;
	static FAutoConsoleVariableRef CVarSkipBuiltInGameFeatures(
		TEXT("GameFeaturePlugin.SkipBuiltInGameFeatures"),
		bSkipBuiltInGameFeatures,
		TEXT("Skip loading built-in game feature plugins."),
		ECVF_Default);

	static bool bTrimNonStartupEnabledPlugins = true;
	static FAutoConsoleVariableRef CVarTrimNonBuiltInPlugins(
		TEXT("GameFeaturePlugin.TrimNonStartupEnabledPlugins"),
		bTrimNonStartupEnabledPlugins,
		TEXT("When call LoadBuiltInGameFeaturePlugins should only the plugins that were built by this target exe be considered built in."),
		ECVF_Default);

	static bool bAllowDownloadingAfterFirstDownloadRequest = true;
	static FAutoConsoleVariableRef CVarAllowDownloadingAfterFirstRequest(
		TEXT("GameFeaturePlugin.AllowDownloadingAfterFirstRequest"),
		bAllowDownloadingAfterFirstDownloadRequest,
		TEXT("Allow downloading a GFP when requested through LoadBuiltinGameFeaturePlugins after the first time it's set to be downloaded."),
		ECVF_Default);

	static FAutoConsoleVariable CVarTrackLoadStats(
		TEXT("GameFeaturePlugin.TrackLoadStats"),
		false,
		TEXT("Track all packages loaded by each GFP. If a package is loaded by multiple GFPs, it will only be ")
		TEXT("attributed to the first one that actually causes the load. Stats can be output with ")
		TEXT("GameFeaturePlugin.PrintLoadStats."));

	static FAutoConsoleVariable CVarDisablePluginUnloadBatchingDuringTermination(
		TEXT("GameFeaturePlugin.DisablePluginUnloadBatchingDuringTermination"),
		false,
		TEXT("Development only: Disables the plugin manager unload batchinig, ")
		TEXT("so every unload of GFP plugin will trigger Garbage collection. Happens during batch Termination."));

	static FAutoConsoleVariable CVarDisablePluginUnloadBatchingDuringUnload(
		TEXT("GameFeaturePlugin.DisablePluginUnloadBatchingDuringUnload"),
		false,
		TEXT("Development only: Disables the plugin manager unload batchinig, ")
		TEXT("so every unload of GFP plugin will trigger Garbage collection. Happens during batch Unload."));

	static FAutoConsoleVariable CVarTransitionContextSuspendStreamingManager(
		TEXT("GameFeaturePlugin.TransitionContextSuspendStreamingManager"),
		false,
		TEXT("Suspend streaming manager in transition context to prevent potential GC races."));

	bool bUnloadPrimaryAssetsOnUnregister = false;
	static FAutoConsoleVariableRef CVarUnloadPrimaryAssetsOnUnregister(
		TEXT("GameFeaturePlugin.UnloadPrimaryAssetsOnUnregister"),
		bUnloadPrimaryAssetsOnUnregister,
		TEXT("Unload primary assets from AssetManager when the game feature plugin unregisters. ")
		TEXT("Use this option only when you are certain that unmounting the plugin will not leak primary assets ")
		TEXT("or their referenced secondary assets. ")
		TEXT("This can provide a small performance improvement during plugin unmounting."));

	TOptional<FString> GetPluginUrlForConsoleCommand(const TArray<FString>& Args, FOutputDevice& Ar)
	{
		TOptional<FString> PluginURL;
		if (Args.Num() > 0)
		{
			EGameFeaturePluginProtocol Protocol = UGameFeaturesSubsystem::GetPluginURLProtocol(Args[0]);
			if (Protocol != EGameFeaturePluginProtocol::Unknown)
			{
				PluginURL.Emplace(Args[0]);
			}
			else
			{
				FString PluginURLStr;
				if (UGameFeaturesSubsystem::Get().GetPluginURLByName(Args[0], /*out*/ PluginURLStr))
				{
					PluginURL.Emplace(MoveTemp(PluginURLStr));
				}
			}
		}

		if (PluginURL)
		{
			Ar.Logf(TEXT("Using URL %s for console command"), *PluginURL.GetValue());
		}
		else
		{
			Ar.Logf(TEXT("Expected a game feature plugin URL or name as an argument"));
		}

		return PluginURL;
	}

#if UE_WITH_PACKAGE_ACCESS_TRACKING
	class FPackageLoadTracker final : public FUObjectArray::FUObjectCreateListener
	{
		struct FStat
		{
			FName Package;
			FName Op;
		};

		/**
		 * Store which packages caused other packages to be loaded. Only tracks loads that occur due to a GFP (direct
		 * and indirect, sync and async). Keys are a package name and values are a list of package names directly loaded
		 * by said package. Indirect loads are gathered by walking the list of loaded packages and querying the map
		 * recursively.
		 */
		TArray<FName> Roots;
		TMap<FName, TArray<FStat>> Stats;
		TArray<FName> UntrackedLoads;

	public:

		FPackageLoadTracker()
		{
			GUObjectArray.AddUObjectCreateListener(this);
		}

		~FPackageLoadTracker()
		{
			GUObjectArray.RemoveUObjectCreateListener(this);
		}

		virtual void OnUObjectArrayShutdown() override
		{
			GUObjectArray.RemoveUObjectCreateListener(this);
		}

		virtual void NotifyUObjectCreated(const UObjectBase* Object, int32 Index) override
		{
			if (Object->GetClass() == UPackage::StaticClass())
			{
				using namespace PackageAccessTracking_Private;

				const UPackage* Package = Cast<UPackage>(Object);
				FString Path = Package->GetPathName();

				// Async loading will cache tracking data when a load is requested and restore it when processing the
				// load later. This means we have tracking for async loads, but it only caches the PackageName and
				// OpName from FTrackedData so we can only rely on those two fields.
				const FTrackedData* Data = FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
				if (Data && !Data->PackageName.IsNone())
				{
					FName LoaderName = Data->PackageName;
					if (TArray<FStat>* LoadedPackages = Stats.Find(LoaderName))
					{
						FName LoadeeName = Object->GetFName();
						LoadedPackages->Add({ LoadeeName, Data->OpName });
						Stats.Add(LoadeeName);
					}
				}
				else
				{
					FName LoadeeName = Object->GetFName();
					UntrackedLoads.Add(LoadeeName);
				}
			}
		}

		void AddRoot(FName PackageName)
		{
			Roots.Add(PackageName);
			Stats.Add(PackageName);
		}

		void PrintLoadStats(bool bShowUntracked) const
		{
			struct Local
			{
				static void VisitPackage(FStringBuilderBase& Buf, const TMap<FName, TArray<FStat>>& Stats, FName Package)
				{
					if (const TArray<FStat>* LoadedList = Stats.Find(Package))
					{
						for (const FStat& Loaded : *LoadedList)
						{
							Buf << TEXT("\n\t\t") << Loaded.Op << TEXT(",");
							int32 Pad = FMath::Max(0, int32(23 - Loaded.Op.GetStringLength()));
							Buf.Appendf(TEXT("%*s"), Pad, TEXT(""));
							Buf << Loaded.Package;
							VisitPackage(Buf, Stats, Loaded.Package);
						}
					}
				}
			};

			TStringBuilder<1024> Buf;
			Buf << TEXT("GameFeaturePlugin Package Load Stats");
			
			for (FName Root : Roots)
			{
				Buf << TEXT("\n\t") << Root << TEXT(": {");
				Local::VisitPackage(Buf, Stats, Root);
				Buf << TEXT("\n\t}");
			}

			if (bShowUntracked)
			{
				Buf << TEXT("\n\tUntracked: {");
				for (FName Untracked : UntrackedLoads)
				{
					Buf << TEXT("\n\t\t") << Untracked;
				}
				Buf << TEXT("\n\t}");
			}
			else
			{
				Buf << TEXT("\n\tUntracked: ") << UntrackedLoads.Num();
			}

			UE_LOGF(LogGameFeatures, Display, "%ls", Buf.ToString());
		}
	};
#else
	class FPackageLoadTracker {};
#endif
}

TSet<FString> UGameFeaturesSubsystem::DlcPlugins;

const FString LexToString(const EBuiltInAutoState BuiltInAutoState)
{
	switch (BuiltInAutoState)
	{
		case EBuiltInAutoState::Invalid:
			return TEXT("Invalid");
		case EBuiltInAutoState::Installed:
			return TEXT("Installed");
		case EBuiltInAutoState::Registered:
			return TEXT("Registered");
		case EBuiltInAutoState::Loaded:
			return TEXT("Loaded");
		case EBuiltInAutoState::Active:
			return TEXT("Active");
		default:
			check(false);
			return TEXT("Unknown");
	}
}

const FString LexToString(const EGameFeatureTargetState GameFeatureTargetState)
{
	switch (GameFeatureTargetState)
	{
		case EGameFeatureTargetState::Installed:
			return TEXT("Installed");
		case EGameFeatureTargetState::Registered:
			return TEXT("Registered"); 
		
		case EGameFeatureTargetState::Loaded:
			return TEXT("Loaded"); 
		case EGameFeatureTargetState::Active:
			return TEXT("Active"); 
		default:
			check(false);
			return TEXT("Unknown");
	}

	static_assert((uint8)EGameFeatureTargetState::Count == 4, "Update LexToString to include new EGameFeatureTargetState");
}

void LexFromString(EGameFeatureTargetState& ValueOut, const TCHAR* StringIn)
{
	//Default value if parsing fails
	ValueOut = EGameFeatureTargetState::Count;

	if (!StringIn || StringIn[0] == '\0')
	{
		ensureAlwaysMsgf(false, TEXT("Invalid empty FString used for EGameFeatureTargetState LexFromString."));
		return;
	}

	for (uint8 EnumIndex = 0; EnumIndex < static_cast<uint8>(EGameFeatureTargetState::Count); ++EnumIndex)
	{
		EGameFeatureTargetState StateToCheck = static_cast<EGameFeatureTargetState>(EnumIndex);
		if (LexToString(StateToCheck).Equals(StringIn, ESearchCase::IgnoreCase))
		{
			ValueOut = StateToCheck;
			return;
		}
	}

	ensureAlwaysMsgf(false, TEXT("Invalid FString { %s } used for EGameFeatureTargetState LexFromString. Does not correspond to any EGameFeatureTargetState!"), StringIn);
}

FGameFeaturePluginIdentifier::FGameFeaturePluginIdentifier(FString PluginURL)
{
	FromPluginURL(MoveTemp(PluginURL));
}

FGameFeaturePluginIdentifier::FGameFeaturePluginIdentifier(FGameFeaturePluginIdentifier&& Other)
	: FGameFeaturePluginIdentifier(MoveTemp(Other.PluginURL))
{
	Other.IdentifyingURLSubset.Reset();
	Other.PluginProtocol = EGameFeaturePluginProtocol::Unknown;
}

FGameFeaturePluginIdentifier& FGameFeaturePluginIdentifier::operator=(FGameFeaturePluginIdentifier&& Other)
{
	FromPluginURL(MoveTemp(Other.PluginURL));
	Other.IdentifyingURLSubset.Reset();
	Other.PluginProtocol = EGameFeaturePluginProtocol::Unknown;
	return *this;
}

void FGameFeaturePluginIdentifier::FromPluginURL(FString PluginURLIn)
{
	//Make sure we have no stale data
	IdentifyingURLSubset.Reset();
	PluginURL = MoveTemp(PluginURLIn);

	if (UGameFeaturesSubsystem::ParsePluginURL(PluginURL, &PluginProtocol, &IdentifyingURLSubset))
	{
		if (!IdentifyingURLSubset.IsEmpty())
		{
			// Plugins must be unique so just use the name as the identifier. This avoids issues with normalizing paths.
			IdentifyingURLSubset = FPathViews::GetCleanFilename(IdentifyingURLSubset);
		}
	}
}

bool FGameFeaturePluginIdentifier::operator==(const FGameFeaturePluginIdentifier& Other) const
{
	return ((PluginProtocol == Other.PluginProtocol) &&
			(IdentifyingURLSubset.Equals(Other.IdentifyingURLSubset, ESearchCase::CaseSensitive)));
}

bool FGameFeaturePluginIdentifier::ExactMatchesURL(const FString& PluginURLIn) const
{
	return GetFullPluginURL().Equals(PluginURLIn, ESearchCase::IgnoreCase);
}

FStringView FGameFeaturePluginIdentifier::GetPluginName() const
{ 
	return FPathViews::GetBaseFilename(IdentifyingURLSubset); 
}

void FGameFeatureStateChangeContext::SetRequiredWorldContextHandle(FName Handle)
{
	WorldContextHandle = Handle;
}

bool FGameFeatureStateChangeContext::ShouldApplyToWorldContext(const FWorldContext& WorldContext) const
{
	if (WorldContextHandle.IsNone())
	{
		return true;
	}
	if (WorldContext.ContextHandle == WorldContextHandle)
	{
		return true;
	}
	return false;
}

bool FGameFeatureStateChangeContext::ShouldApplyUsingOtherContext(const FGameFeatureStateChangeContext& OtherContext) const
{
	if (OtherContext == *this)
	{
		return true;
	}

	// If other context is less restrictive, apply
	if (OtherContext.WorldContextHandle.IsNone())
	{
		return true;
	}

	return false;
}

FSimpleDelegate FGameFeatureDeactivatingContext::PauseDeactivationUntilComplete(FString InPauserTag)
{
	UE_LOGF(LogGameFeatures, Display, "Deactivation of %.*ls paused by %ls", PluginName.Len(), PluginName.GetData(), *InPauserTag);

	++NumPausers;
	return FSimpleDelegate::CreateLambda(
		[CompletionCallback = CompletionCallback, PauserTag = MoveTemp(InPauserTag)]() { CompletionCallback(PauserTag); }
	);
}

FSimpleDelegate FGameFeaturePostMountingContext::PauseUntilComplete(FString InPauserTag)
{
	UE_LOGF(LogGameFeatures, Display, "Post-mount of %.*ls paused by %ls", PluginName.Len(), PluginName.GetData(), *InPauserTag);

	++NumPausers;
	return FSimpleDelegate::CreateLambda(
		[CompletionCallback = CompletionCallback, PauserTag = MoveTemp(InPauserTag)]() { CompletionCallback(PauserTag); }
	);
}

FInstallBundlePluginProtocolOptions::FInstallBundlePluginProtocolOptions()
	: InstallBundleFlags(EInstallBundleRequestFlags::Defaults)
	, ReleaseInstallBundleFlags(EInstallBundleReleaseRequestFlags::None)
	, bUninstallBeforeTerminate(false)
	, bUserPauseDownload(false)
	, bAllowIniLoading(false)
	, bDoNotDownload(false)
{}

bool FInstallBundlePluginProtocolOptions::operator==(const FInstallBundlePluginProtocolOptions& Other) const
{
	return
		InstallBundleFlags == Other.InstallBundleFlags &&
		bUninstallBeforeTerminate == Other.bUninstallBeforeTerminate &&
		bUserPauseDownload == Other.bUserPauseDownload && 
		bAllowIniLoading == Other.bAllowIniLoading &&
		bDoNotDownload == Other.bDoNotDownload;
}

FGameFeatureProtocolOptions::FGameFeatureProtocolOptions()
	: bForceSyncLoading(false)
	, bBatchProcess(false)
	, bLogWarningOnForcedDependencyCreation(false)
	, bLogErrorOnForcedDependencyCreation(false)
{ 
	SetSubtype<FNull>(); 
}

FGameFeatureProtocolOptions::FGameFeatureProtocolOptions(const FInstallBundlePluginProtocolOptions& InOptions) 
	: TUnion(InOptions) 
	, bForceSyncLoading(false)
	, bLogWarningOnForcedDependencyCreation(false)
	, bLogErrorOnForcedDependencyCreation(false)
{
}

FGameFeatureProtocolOptions::FGameFeatureProtocolOptions(FNull InOptions)
	: bForceSyncLoading(false)
	, bBatchProcess(false)
	, bLogWarningOnForcedDependencyCreation(false)
	, bLogErrorOnForcedDependencyCreation(false)
{
	SetSubtype<FNull>(InOptions);
}

UGameFeaturesSubsystem::UGameFeaturesSubsystem(const FObjectInitializer& ObjectInitializer)
	: Super()
{
	if (UE::GameFeatures::bUseNewExecutor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		StateMachineExecutor = MakePimpl<UE::GameFeatures::FStateMachineExecutor>(this);
	}
}

void UGameFeaturesSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UE_LOGF(LogGameFeatures, Log, "Initializing game features subsystem");

	// Create the game-specific policy manager
	check(!bInitializedPolicyManager && (GameSpecificPolicies == nullptr));

	const FSoftClassPath& PolicyClassPath = GetDefault<UGameFeaturesSubsystemSettings>()->GameFeaturesManagerClassName;

	UClass* SingletonClass = nullptr;
	if (!PolicyClassPath.IsNull())
	{
		SingletonClass = LoadClass<UGameFeaturesProjectPolicies>(nullptr, *PolicyClassPath.ToString());
	}

	if (SingletonClass == nullptr)
	{
		SingletonClass = UDefaultGameFeaturesProjectPolicies::StaticClass();
	}
		
	GameSpecificPolicies = NewObject<UGameFeaturesProjectPolicies>(this, SingletonClass);
	check(GameSpecificPolicies);

	UAssetManager::CallOrRegister_OnAssetManagerCreated(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &ThisClass::OnAssetManagerCreated));

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("ListGameFeaturePlugins"),
		TEXT("Prints game features plugins and their current state to log. (options: [-activeonly] [-csv])"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateUObject(this, &ThisClass::ListGameFeaturePlugins),
		ECVF_Default);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("LoadGameFeaturePlugin"),
		TEXT("Loads and activates a game feature plugin by PluginName or URL"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
		{
			if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
			{
				UGameFeaturesSubsystem::Get().LoadAndActivateGameFeaturePlugin(PluginURL.GetValue(), FGameFeaturePluginLoadComplete());
			}
		}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("DeactivateGameFeaturePlugin"),
		TEXT("Deactivates a game feature plugin by PluginName or URL"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
		{
			if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
			{
				UGameFeaturesSubsystem::Get().DeactivateGameFeaturePlugin(PluginURL.GetValue(), FGameFeaturePluginLoadComplete());
			}
		}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("UnloadGameFeaturePlugin"),
		TEXT("Unloads a game feature plugin by PluginName or URL"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
		{
			if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
			{
				UGameFeaturesSubsystem::Get().UnloadGameFeaturePlugin(PluginURL.GetValue());
			}
		}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("UnloadAndKeepRegisteredGameFeaturePlugin"),
		TEXT("Unloads a game feature plugin by PluginName or URL but keeps it registered"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
		{
			if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
			{
				UGameFeaturesSubsystem::Get().UnloadGameFeaturePlugin(PluginURL.GetValue(), true);
			}
		}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("ReleaseGameFeaturePlugin"),
		TEXT("Releases a game feature plugin's InstallBundle data by PluginName or URL"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
		{
			if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
			{
				UGameFeaturesSubsystem::Get().ReleaseGameFeaturePlugin(PluginURL.GetValue());
			}
		}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("CancelGameFeaturePlugin"),
		TEXT("Cancel any state changes for a game feature plugin by PluginName or URL"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
		{
			if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
			{
				UGameFeaturesSubsystem::Get().CancelGameFeatureStateChange(PluginURL.GetValue());
			}
		}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("TerminateGameFeaturePlugin"),
		TEXT("Terminates a game feature plugin by PluginName or URL"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
		{
			if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
			{
				UGameFeaturesSubsystem::Get().TerminateGameFeaturePlugin(PluginURL.GetValue());
			}
		}),
		ECVF_Cheat);

#if !UE_BUILD_SHIPPING
	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("EnableDebugGameFeatureState"),
		TEXT("Trigger a debug breakpoint if when the state of the gameplay feature changes"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
			{
				if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
				{
					UGameFeaturesSubsystem::Get().SetPluginDebugStateEnabled(PluginURL.GetValue(), true);
				}
			}),
		ECVF_Cheat);

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("DisableDebugGameFeatureState"),
		TEXT("Turn off triggering a debug breakpoint if when the state of the gameplay feature changes"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
			{
				if (TOptional<FString> PluginURL = UE::GameFeatures::GetPluginUrlForConsoleCommand(Args, Ar))
				{
					UGameFeaturesSubsystem::Get().SetPluginDebugStateEnabled(PluginURL.GetValue(), false);
				}
			}),
		ECVF_Cheat);
#endif

#if UE_WITH_PACKAGE_ACCESS_TRACKING
	if (UE::GameFeatures::CVarTrackLoadStats->GetBool())
	{
		PackageLoadTracker = MakeUnique<UE::GameFeatures::FPackageLoadTracker>();
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GameFeaturePlugin.PrintLoadStats"),
			TEXT("Print a list of all packages loaded by each GFP. If a package is loaded by multiple GFPs, it will only be")
			TEXT("attributed to the first one that actually causes the load")
			TEXT("\nUsage: GameFeaturePlugin.PrintLoadStats [-ShowUntracked]")
			TEXT("\n-ShowUntracked - Print a list of packages that were loaded, but don't have tracking to know why"),
			FConsoleCommandWithArgsDelegate::CreateLambda([&Tracker = PackageLoadTracker](const TArray<FString>& Args)
			{
				bool bShowUntracked = Args.Contains(TEXT("-ShowUntracked"));
				Tracker->PrintLoadStats(bShowUntracked);
			})
		);
	}
#endif

	GetExplanationForUnavailablePackageDelegateHandle = IPluginManager::Get().GetExplanationForUnavailablePackageWithPluginInfoDelegate().AddUObject(this, &UGameFeaturesSubsystem::GetExplanationForUnavailablePackage);
	// TODO: Should this only remove terminal plugins? Depends on editor or not? 
	IPluginManager::Get().OnPluginUnmounted().AddWeakLambda(this, [this] (const IPlugin& Plugin) { PruneCachedGameFeaturePluginDetails(/* doesnt use URL */ FString(""), Plugin.GetDescriptorFileName()); });
	IPluginManager::Get().OnPluginEdited().AddWeakLambda(this, [this](IPlugin& Plugin) { PruneCachedGameFeaturePluginDetails(/* doesnt use URL */ FString(""), Plugin.GetDescriptorFileName()); });

#if WITH_EDITOR
	GameFeatureDataExternalAssetsPathCache = MakeUnique<FGameFeatureDataExternalAssetsPathCache>();
#endif
}

void UGameFeaturesSubsystem::Deinitialize()
{
	UE_LOGF(LogGameFeatures, Log, "Shutting down game features subsystem");
	IPluginManager::Get().GetExplanationForUnavailablePackageWithPluginInfoDelegate().Remove(GetExplanationForUnavailablePackageDelegateHandle);
	GetExplanationForUnavailablePackageDelegateHandle.Reset();

	if ((GameSpecificPolicies != nullptr) && bInitializedPolicyManager)
	{
		GameSpecificPolicies->ShutdownGameFeatureManager();
	}
	GameSpecificPolicies = nullptr;
	bInitializedPolicyManager = false;

	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(MoveTemp(TickHandle));
		TickHandle.Reset();
	}

#if WITH_EDITOR
	GameFeatureDataExternalAssetsPathCache.Reset();
#endif

	// Delete all state objects so that they don't try to run destructor logic during GC/shutdown in a way that could access 
	// already-deleted objects (e.g. GEngine, this subsystem)
	for (TPair<int64, TObjectPtr<UGameFeaturePluginStateMachine>>& Pair : GameFeaturePluginStateMachines)
	{
		if (Pair.Value)
		{
			for (TUniquePtr<FGameFeaturePluginState>& Ptr : Pair.Value->AllStates)
			{
				Ptr.Reset();
			}
		}
	}

	for (TObjectPtr<UGameFeaturePluginStateMachine> Terminal : TerminalGameFeaturePluginStateMachines)
	{
		if (Terminal)
		{
			for (TUniquePtr<FGameFeaturePluginState>& Ptr : Terminal->AllStates)
			{
				Ptr.Reset();
			}
		}
	}
}

void UGameFeaturesSubsystem::OnAssetManagerCreated()
{
	check(!bInitializedPolicyManager && (GameSpecificPolicies != nullptr));

	// Make sure the game has the appropriate asset manager configuration or we won't be able to load game feature data assets
	FPrimaryAssetId DummyGameFeatureDataAssetId(UGameFeatureData::StaticClass()->GetFName(), NAME_None);
	FPrimaryAssetRules GameDataRules = UAssetManager::Get().GetPrimaryAssetRules(DummyGameFeatureDataAssetId);
	if (GameDataRules.IsDefault())
	{
		const bool bHasProject = FApp::HasProjectName(); // Only error when we have a UE project loaded, as otherwise there won't be any GFPs to load
		UE_CLOGF(bHasProject, LogGameFeatures, Error, "Asset manager settings do not include a rule for assets of type %ls, which is required for game feature plugins to function", *UGameFeatureData::StaticClass()->GetName());
	}

	// Give system that want to add an observer before we start to mount plugins a chance to do so.
	OnGameFeaturePolicyPreInit.Broadcast();

	// Create the game-specific policy
	UE_LOGF(LogGameFeatures, Verbose, "Initializing game features policy (type %ls)", *GameSpecificPolicies->GetClass()->GetName());
	bInitializedPolicyManager = true; // Set before calling InitGameFeatureManager() because InitGameFeatureManager may load GFPs
	GameSpecificPolicies->InitGameFeatureManager();

	LoadAllBuiltInGameFeaturePluginDetails();

	// Give systems that want to cache data now that GameSpecificPolicies is initialized a chance
	OnGameFeaturePolicyPostInit.Broadcast();
}

void UGameFeaturesSubsystem::LoadAllBuiltInGameFeaturePluginDetails(bool bForce /*= false */)
{
	// Preload the details for all builtin game features which the policies allow us to load
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UGameFeaturesSubsystem::LoadAllBuiltInGameFeaturePluginDetails");
	if (bLoadedBuiltInDetails && !bForce)
	{
		return;
	}
	bLoadedBuiltInDetails = true;
	TArray<TSharedRef<IPlugin>> BuiltinPlugins = GetInternalEnabledPlugins();
	TArray<UE::Tasks::TTask<TSharedPtr<const FGameFeaturePluginDetails>>> Tasks;
	Tasks.Reserve(BuiltinPlugins.Num());
	Tasks.AddDefaulted(BuiltinPlugins.Num());
	// We can't kick off the tasks in parallel yet because we need to add the thread safety contract to the GameSpecificPolicies 
	EParallelForFlags Flags = EParallelForFlags::ForceSingleThread;
	// Some things constrain us from even beginning the tasks off the game thread
	if (WITH_EDITOR 
		|| FPluginDescriptor::CustomPluginDescriptorReaderDelegate.IsBound())
	{
		Flags = EParallelForFlags::ForceSingleThread;
	}
	ParallelFor(BuiltinPlugins.Num(), [&BuiltinPlugins, &Tasks, this](int32 PluginIndex)
	{
		const FString& DescriptorFilename = BuiltinPlugins[PluginIndex]->GetDescriptorFileName();
		if (DescriptorFilename.IsEmpty() || !GetDefault<UGameFeaturesSubsystemSettings>()->IsValidGameFeaturePlugin(DescriptorFilename))
		{
			Tasks[PluginIndex] = UE::Tasks::MakeCompletedTask<TSharedPtr<const FGameFeaturePluginDetails>>();
		}
		else
		{
			Tasks[PluginIndex] = LoadGameFeaturePluginDetailsAsync(DescriptorFilename);
		}
	}, Flags);
	// Not strictly necessary but block on completion in case other systems want the data after this point and might block or duplicate work
	UE::Tasks::Wait(Tasks); 
}

bool UGameFeaturesSubsystem::IsPluginAllowed(const FString& PluginURL, FString* OutReason) const
{
	ensureMsgf(bInitializedPolicyManager, TEXT("Attemting to load plugin [%s] before GameFeaturesSubsystem is ready!"), *PluginURL);
	return bInitializedPolicyManager && GameSpecificPolicies->IsPluginAllowed(PluginURL, OutReason);
}

TSharedPtr<FStreamableHandle> UGameFeaturesSubsystem::LoadGameFeatureData(const FString& GameFeatureToLoad, bool bStartStalled /*= false*/)
{
	return UAssetManager::Get().GetStreamableManager().RequestAsyncLoad(
		FSoftObjectPath(GameFeatureToLoad), 
		FStreamableDelegate(), 
		FStreamableManager::DefaultAsyncLoadPriority, 
		false, 
		bStartStalled, 
		TEXT("LoadGameFeatureData"));
}

void UGameFeaturesSubsystem::UnloadGameFeatureData(const UGameFeatureData* GameFeatureToUnload)
{
	UAssetManager& LocalAssetManager = UAssetManager::Get();
	LocalAssetManager.UnloadPrimaryAsset(GameFeatureToUnload->GetPrimaryAssetId());
}

void UGameFeaturesSubsystem::AddGameFeatureToAssetManager(const UGameFeatureData* GameFeatureToAdd, const FString& PluginName, TArray<FName>& OutNewPrimaryAssetTypes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GFP_AddToAssetManager);
	check(GameFeatureToAdd);
	FString PluginRootPath = TEXT("/") + PluginName + TEXT("/");
	UAssetManager& LocalAssetManager = UAssetManager::Get();
	IAssetRegistry& LocalAssetRegistry = LocalAssetManager.GetAssetRegistry();

	LocalAssetManager.PushBulkScanning();

	// Add the GameFeatureData itself to the primary asset list
#if WITH_EDITOR
	// In the editor, we may not have scanned the FAssetData yet if during startup, but that is fine because we can gather bundles from the object itself, so just create the FAssetData from the object
	LocalAssetManager.RegisterSpecificPrimaryAsset(GameFeatureToAdd->GetPrimaryAssetId(), FAssetData(GameFeatureToAdd));
#else
	// In non-editor, the asset bundle data is compiled out, so it must be gathered from the asset registry instead
	LocalAssetManager.RegisterSpecificPrimaryAsset(GameFeatureToAdd->GetPrimaryAssetId(), LocalAssetRegistry.GetAssetByObjectPath(FSoftObjectPath(GameFeatureToAdd), true));
#endif // WITH_EDITOR

	// @TODO: HACK - There is no guarantee that the plugin mount point was added before the initial asset scan.
	// If not, ScanPathsForPrimaryAssets will fail to find primary assets without a syncronous scan.
	// A proper fix for this would be to handle all the primary asset discovery internally ins the asset manager 
	// instead of doing it here.
	// We just mounted the folder that contains these primary assets and the editor background scan may not
	// not be finished by the time this is called, but a rescan will happen later in OnAssetRegistryFilesLoaded 
	// as long as LocalAssetRegistry.IsLoadingAssets() is true.
	const bool bForceSynchronousScan = !LocalAssetRegistry.IsGathering();

	for (FPrimaryAssetTypeInfo TypeInfo : GameFeatureToAdd->GetPrimaryAssetTypesToScan())
	{
		for (FDirectoryPath& Path : TypeInfo.GetDirectories())
		{
			// Convert plugin-relative paths to full package paths
			FixPluginPackagePath(Path.Path, PluginRootPath, false);
		}

		// This function also fills out runtime data on the copy
		if (!LocalAssetManager.ShouldScanPrimaryAssetType(TypeInfo))
		{
			continue;
		}

		FPrimaryAssetTypeInfo ExistingAssetTypeInfo;
		const bool bAlreadyExisted = LocalAssetManager.GetPrimaryAssetTypeInfo(FPrimaryAssetType(TypeInfo.PrimaryAssetType), /*out*/ ExistingAssetTypeInfo);
		LocalAssetManager.ScanPathsForPrimaryAssets(TypeInfo.PrimaryAssetType, TypeInfo.AssetScanPaths, TypeInfo.AssetBaseClassLoaded, TypeInfo.bHasBlueprintClasses, TypeInfo.bIsEditorOnly, bForceSynchronousScan);

		if (!bAlreadyExisted)
		{
			OutNewPrimaryAssetTypes.Add(TypeInfo.PrimaryAssetType);

			// If we did not previously scan anything for a primary asset type that is in our config, try to reuse the cook rules from the config instead of the one in the gamefeaturedata, which should not be modifying cook rules
			const FPrimaryAssetTypeInfo* ConfigTypeInfo = LocalAssetManager.GetSettings().PrimaryAssetTypesToScan.FindByPredicate([&TypeInfo](const FPrimaryAssetTypeInfo& PATI) -> bool { return PATI.PrimaryAssetType == TypeInfo.PrimaryAssetType; });
			if (ConfigTypeInfo)
			{
				LocalAssetManager.SetPrimaryAssetTypeRules(TypeInfo.PrimaryAssetType, ConfigTypeInfo->Rules);
			}
			else
			{
				LocalAssetManager.SetPrimaryAssetTypeRules(TypeInfo.PrimaryAssetType, TypeInfo.Rules);
			}
		}
	}
	LocalAssetManager.PopBulkScanning();

	const UAssetManagerSettings& Settings = LocalAssetManager.GetSettings();
	for (const FPrimaryAssetRulesCustomOverride& Override : Settings.CustomPrimaryAssetRules)
	{
		if (Override.FilterDirectory.Path.StartsWith(PluginRootPath))
		{
			LocalAssetManager.ApplyCustomPrimaryAssetRulesOverride(Override);
		}
	}
}

void UGameFeaturesSubsystem::RemoveGameFeatureFromAssetManager(const UGameFeatureData* GameFeatureToRemove, const FString& PluginName, const TArray<FName>& AddedPrimaryAssetTypes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GFP_RemoveFromAssetManager);
	check(GameFeatureToRemove);
	FString PluginRootPath = TEXT("/") + PluginName + TEXT("/");
	UAssetManager& LocalAssetManager = UAssetManager::Get();

	// Remove scan paths from asset manager
	for (FPrimaryAssetTypeInfo TypeInfo : GameFeatureToRemove->GetPrimaryAssetTypesToScan())
	{
		for (FDirectoryPath& Path : TypeInfo.GetDirectories())
		{
			FixPluginPackagePath(Path.Path, PluginRootPath, false);
		}

		// This function also fills out runtime data on the copy
		if (!LocalAssetManager.ShouldScanPrimaryAssetType(TypeInfo))
		{
			continue;
		}

		if (AddedPrimaryAssetTypes.Contains(TypeInfo.PrimaryAssetType))
		{
			LocalAssetManager.RemovePrimaryAssetType(TypeInfo.PrimaryAssetType);
			continue;
		}

		LocalAssetManager.RemoveScanPathsForPrimaryAssets(TypeInfo.PrimaryAssetType, TypeInfo.AssetScanPaths, TypeInfo.AssetBaseClassLoaded, TypeInfo.bHasBlueprintClasses, TypeInfo.bIsEditorOnly);
		if (UE::GameFeatures::bUnloadPrimaryAssetsOnUnregister)
		{
			LocalAssetManager.RemovePrimaryAssetsForTypeInMountPoint(TypeInfo.PrimaryAssetType, PluginRootPath);
		}
	}
}

void UGameFeaturesSubsystem::ForEachGameFeature(TFunctionRef<void(FGameFeatureInfo&&)> Visitor) const
{
	for (auto StateMachineIt = GameFeaturePluginStateMachines.CreateConstIterator(); StateMachineIt; ++StateMachineIt)
	{
		if (UGameFeaturePluginStateMachine* GFSM = StateMachineIt.Value())
		{
			FGameFeatureInfo GameFeatureInfo = { GetPluginName(GFSM), GetPluginURL(GFSM), WasLoadedAsBuiltIn(GFSM), GetCurrentState(GFSM) };
			Visitor(MoveTemp(GameFeatureInfo));
		}
	}
}

void UGameFeaturesSubsystem::AddObserver(UObject* Observer)
{
	//deprecated, previous default behaviour
	AddObserver(Observer, EObserverPluginStateUpdateMode::CurrentAndFuture);
}

void UGameFeaturesSubsystem::AddObserver(UObject* Observer, EObserverPluginStateUpdateMode UpdateMode)
{
	QUICK_SCOPE_CYCLE_COUNTER(UGameFeaturesSubsystem_AddObserver);

	check(Observer);
	IGameFeatureStateChangeObserver* Interface = Cast<IGameFeatureStateChangeObserver>(Observer);
	if (ensureAlwaysMsgf(Interface != nullptr, TEXT("Observers must implement the IGameFeatureStateChangeObserver interface.")))
	{
		Observers.AddUnique(Observer);

		if (UpdateMode == EObserverPluginStateUpdateMode::CurrentAndFuture)
		{
			// Push the current state of all known game features to the new observer
			for (auto StateMachineIt = GameFeaturePluginStateMachines.CreateConstIterator(); StateMachineIt; ++StateMachineIt)
			{
				if (UGameFeaturePluginStateMachine* GFSM = StateMachineIt.Value())
				{
					if (const UGameFeatureData* GameFeatureData = GetGameFeatureDataForRegisteredPlugin(GFSM, false))
					{
						const FString& PluginName = GetPluginName(GFSM);
						const FString& PluginURL = GetPluginURL(GFSM);

						Interface->OnGameFeatureRegistering(GameFeatureData, PluginName, PluginURL);

						if (GetCurrentState(GFSM) >= EGameFeaturePluginState::Loaded)
						{
							Interface->OnGameFeatureLoading(GameFeatureData, PluginURL);
						}

						if (GetCurrentState(GFSM) >= EGameFeaturePluginState::Active)
						{
							Interface->OnGameFeatureActivating(GameFeatureData, PluginURL);
							Interface->OnGameFeatureActivated(GameFeatureData, PluginURL);
						}
					}
				}
			}
		}
	}
}

void UGameFeaturesSubsystem::RemoveObserver(UObject* Observer)
{
	check(Observer);
	Observers.RemoveSingleSwap(Observer);
}

FString UGameFeaturesSubsystem::GetPluginURL_FileProtocol(const FString& PluginDescriptorPath)
{
	return UE::GameFeatures::GameFeaturePluginProtocolPrefix(EGameFeaturePluginProtocol::File) + PluginDescriptorPath;
}

FString UGameFeaturesSubsystem::GetPluginURL_FileProtocol(const FString& PluginDescriptorPath, TArrayView<const TPair<FString, FString>> AdditionalOptions)
{
	FString Path;
	Path += UE::GameFeatures::GameFeaturePluginProtocolPrefix(EGameFeaturePluginProtocol::File);
	Path += PluginDescriptorPath;
	if (AdditionalOptions.Num() > 0)
	{
		Path += UE::GameFeatures::PluginURLStructureInfo::OptionSeperator;
		Path += FString::JoinBy(AdditionalOptions, UE::GameFeatures::PluginURLStructureInfo::OptionSeperator,
			[](const TPair<FString, FString>& OptionPair)
			{
				return OptionPair.Key + UE::GameFeatures::PluginURLStructureInfo::OptionAssignOperator + OptionPair.Value;
			});
	}
	return Path;
}

FString UGameFeaturesSubsystem::GetPluginFilename_FileProtocol(const FString& PluginUrlFileProtocol)
{
	const TCHAR* Prefix = UE::GameFeatures::GameFeaturePluginProtocolPrefix(EGameFeaturePluginProtocol::File);
	FStringView PrefixView(Prefix);
	FString Result;
	if (ensureAlwaysMsgf(PluginUrlFileProtocol.StartsWith(Prefix), TEXT("Unexpected protocol for %s"), *PluginUrlFileProtocol))
	{
		Result = PluginUrlFileProtocol.RightChop(PrefixView.Len());
	}
	return Result;
}

void UGameFeaturesSubsystem::GetExplanationForUnavailablePackage(const FString& UnavailablePackage, IPlugin* PluginIfFound, FStringBuilderBase& InOutExplanation)
{
#if WITH_EDITOR
	if (PluginIfFound)
	{
		if (FString* Explanation = UnmountedPluginNameToExplanation.Find(PluginIfFound->GetName()))
		{
			InOutExplanation.Appendf(TEXT("\nUGameFeaturesSubsystem: Explanation for not mounting plugin %s: %s"), *PluginIfFound->GetFriendlyName(), **Explanation);
		}
	}
	else
	{
		FString ContentDirName = FPackageName::SplitPackageNameRoot(UnavailablePackage, nullptr).GetData();
		if (FString* Explanation = UnmountedPluginNameToExplanation.Find(ContentDirName))
		{
			InOutExplanation.Appendf(TEXT("\nUGameFeaturesSubsystem: Explanation for not mounting plugin %s: %s"), *ContentDirName, **Explanation);
		}
	}
#endif
}

FString GetPluginURL_InstallBundleProtocol(const FString& PluginName, const FInstallBundlePluginProtocolMetaData& ProtocolMetadata, TArrayView<const TPair<FString, FString>> AdditionalOptions = {})
{
	ensure(ProtocolMetadata.InstallBundles.Num() > 0);
	FString Path;
	Path += UE::GameFeatures::GameFeaturePluginProtocolPrefix(EGameFeaturePluginProtocol::InstallBundle);
	Path += PluginName;
	Path += ProtocolMetadata.ToString();
	if (AdditionalOptions.Num() > 0)
	{
		Path += UE::GameFeatures::PluginURLStructureInfo::OptionSeperator;
		Path += FString::JoinBy(AdditionalOptions, UE::GameFeatures::PluginURLStructureInfo::OptionSeperator,
			[](const TPair<FString, FString>& OptionPair)
			{
				return OptionPair.Key + UE::GameFeatures::PluginURLStructureInfo::OptionAssignOperator + OptionPair.Value;
			});
	}

	return Path;
}

FString UGameFeaturesSubsystem::GetPluginURL_InstallBundleProtocol(const FString& PluginName, TArrayView<const FString> BundleNames)
{
	FInstallBundlePluginProtocolMetaData ProtocolMetadata;
	for (const FString& BundleName : BundleNames)
	{
		ProtocolMetadata.InstallBundles.Emplace(BundleName);
	}
	return ::GetPluginURL_InstallBundleProtocol(PluginName, ProtocolMetadata);
}

FString UGameFeaturesSubsystem::GetPluginURL_InstallBundleProtocol(const FString& PluginName, const FString& BundleName)
{
	return GetPluginURL_InstallBundleProtocol(PluginName, MakeArrayView(&BundleName, 1));
}

FString UGameFeaturesSubsystem::GetPluginURL_InstallBundleProtocol(const FString& PluginName, const TArrayView<const FName> BundleNames)
{
	return GetPluginURL_InstallBundleProtocol(PluginName, BundleNames, TArrayView<const TPair<FString, FString>>());
}

FString UGameFeaturesSubsystem::GetPluginURL_InstallBundleProtocol(const FString& PluginName, FName BundleName)
{
	return GetPluginURL_InstallBundleProtocol(PluginName, MakeArrayView(&BundleName, 1));
}

FString UGameFeaturesSubsystem::GetPluginURL_InstallBundleProtocol(const FString& PluginName, TArrayView<const FName> BundleNames, TArrayView<const TPair<FString, FString>> AdditionalOptions)
{
	FInstallBundlePluginProtocolMetaData ProtocolMetadata;
	ProtocolMetadata.InstallBundles.Append(BundleNames.GetData(), BundleNames.Num());
	return ::GetPluginURL_InstallBundleProtocol(PluginName, ProtocolMetadata, AdditionalOptions);
}

EGameFeaturePluginProtocol UGameFeaturesSubsystem::GetPluginURLProtocol(FStringView PluginURL)
{
	for (EGameFeaturePluginProtocol Protocol : TEnumRange<EGameFeaturePluginProtocol>())
	{
		if (UGameFeaturesSubsystem::IsPluginURLProtocol(PluginURL, Protocol))
		{
			return Protocol;
		}
	}
	return EGameFeaturePluginProtocol::Unknown;
}

bool UGameFeaturesSubsystem::IsPluginURLProtocol(FStringView PluginURL, EGameFeaturePluginProtocol PluginProtocol)
{
	return PluginURL.StartsWith(UE::GameFeatures::GameFeaturePluginProtocolPrefix(PluginProtocol));
}

bool UGameFeaturesSubsystem::ParsePluginURL(FStringView PluginURL, EGameFeaturePluginProtocol* OutProtocol /*= nullptr*/, FStringView* OutPath /*= nullptr*/, FStringView* OutOptions /*= nullptr*/)
{
	FStringView Path;
	FStringView Options;
	EGameFeaturePluginProtocol PluginProtocol = UGameFeaturesSubsystem::GetPluginURLProtocol(PluginURL);

	if (ensureAlwaysMsgf(PluginProtocol != EGameFeaturePluginProtocol::Unknown && PluginProtocol != EGameFeaturePluginProtocol::Count,
		TEXT("Invalid PluginProtocol in PluginURL %.*s"), PluginURL.Len(), PluginURL.GetData()))
	{
		int32 PluginProtocolLen = FCString::Strlen(UE::GameFeatures::GameFeaturePluginProtocolPrefix(PluginProtocol));
		int32 FirstOptionIndex = UE::String::FindFirst(PluginURL, UE::GameFeatures::PluginURLStructureInfo::OptionSeperator, ESearchCase::IgnoreCase);

		//If we don't have any options, then the Path is just our entire URL except the protocol string
		if (FirstOptionIndex == INDEX_NONE)
		{
			Path = PluginURL.RightChop(PluginProtocolLen);
		}
		//The Path will be the string between the end of the protocol string and before the first option
		else
		{
			const int32 IdentifierCharCount = (FirstOptionIndex - PluginProtocolLen);
			Path = PluginURL.Mid(PluginProtocolLen, IdentifierCharCount);
			Options = PluginURL.RightChop(FirstOptionIndex);
		}

		if (ensureAlwaysMsgf(Path.EndsWith(TEXTVIEW(".uplugin")), TEXT("Invalid path in PluginURL %.*s"), PluginURL.Len(), PluginURL.GetData()))
		{
			if (OutProtocol)
			{
				*OutProtocol = PluginProtocol;
			}

			if (OutPath)
			{
				*OutPath = Path;
			}

			if (OutOptions)
			{
				*OutOptions = Options;
			}

			return true;
		}
	}

	return false;
}

namespace GameFeaturesSubsystem
{
	static bool SplitOption(FStringView OptionPair, FStringView& OutOptionName, FStringView& OutOptionValue)
	{
		int32 TokenCount = 0;
		FStringView OptionName;
		FStringView OptionValue;
		UE::String::ParseTokens(OptionPair, UE::GameFeatures::PluginURLStructureInfo::OptionAssignOperator, 
		[&TokenCount, &OptionName, &OptionValue](FStringView Token)
		{
			++TokenCount;
			switch (TokenCount)
			{
			case 1:
				OptionName = Token;
				break;
			case 2:
				OptionValue = Token;
				break;
			}
		});

		const bool bSuccess = TokenCount == 2;
		if (bSuccess)
		{
			OutOptionName = OptionName;
			OutOptionValue = OptionValue;
		}

		return bSuccess;
	}

	struct FParsePluginURLOptionsFilter
	{
		EGameFeatureURLOptions OptionsFlags;
		TConstArrayView<FStringView> AdditionalOptions;
	};

	static bool ParsePluginURLOptions(FStringView URLOptionsString, const FParsePluginURLOptionsFilter* OptionsFilter,
		TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output)
	{
		enum class EParseState : uint8
		{
			First,
			OK,
			Error
		};

		//Parse through our URLOptions. The first option won't appear until after the first seperator.
		//We don't care what comes before the first seperator
		EParseState ParseState = EParseState::First;
		UE::String::ParseTokens(URLOptionsString, UE::GameFeatures::PluginURLStructureInfo::OptionSeperator, 
		[OptionsFilter, Output, &ParseState](FStringView Token)
		{
			if (ParseState == EParseState::Error)
			{
				return;
			}

			if (ParseState == EParseState::First)
			{
				ParseState = EParseState::OK;
				return;
			}

			FStringView OptionName;
			FStringView OptionValue;
			if (!GameFeaturesSubsystem::SplitOption(Token, OptionName, OptionValue))
			{
				ParseState = EParseState::Error;
				return;
			}

			EGameFeatureURLOptions OptionEnum = EGameFeatureURLOptions::None;

			if (!OptionsFilter || OptionsFilter->OptionsFlags != EGameFeatureURLOptions::None)
			{
				LexFromString(OptionEnum, OptionName);
			}

			if (!OptionsFilter || EnumHasAnyFlags(OptionsFilter->OptionsFlags, OptionEnum) || OptionsFilter->AdditionalOptions.Contains(OptionName))
			{
				UE::String::ParseTokens(OptionValue, UE::GameFeatures::PluginURLStructureInfo::OptionListSeperator,
				[Output, OptionEnum, OptionName](FStringView ListToken)
				{
					Output(OptionEnum, OptionName, ListToken);
				});
			}
		});

		return ParseState == EParseState::OK;
	}
}

bool UGameFeaturesSubsystem::ParsePluginURLOptions(FStringView URLOptionsString,
	TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output)
{
	return GameFeaturesSubsystem::ParsePluginURLOptions(URLOptionsString, nullptr, Output);
}

bool UGameFeaturesSubsystem::ParsePluginURLOptions(FStringView URLOptionsString, EGameFeatureURLOptions OptionsFlags,
	TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output)
{
	const GameFeaturesSubsystem::FParsePluginURLOptionsFilter OptionsFilter{ OptionsFlags, {} };
	return GameFeaturesSubsystem::ParsePluginURLOptions(URLOptionsString, &OptionsFilter, Output);
}

bool UGameFeaturesSubsystem::ParsePluginURLOptions(FStringView URLOptionsString, TConstArrayView<FStringView> AdditionalOptions,
	TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output)
{
	const GameFeaturesSubsystem::FParsePluginURLOptionsFilter OptionsFilter{ EGameFeatureURLOptions::None, AdditionalOptions };
	return GameFeaturesSubsystem::ParsePluginURLOptions(URLOptionsString, &OptionsFilter, Output);
}

bool UGameFeaturesSubsystem::ParsePluginURLOptions(FStringView URLOptionsString, EGameFeatureURLOptions OptionsFlags, TConstArrayView<FStringView> AdditionalOptions,
	TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output)
{
	const GameFeaturesSubsystem::FParsePluginURLOptionsFilter OptionsFilter{ OptionsFlags, AdditionalOptions };
	return GameFeaturesSubsystem::ParsePluginURLOptions(URLOptionsString, &OptionsFilter, Output);
}

void UGameFeaturesSubsystem::OnGameFeatureTerminating(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Terminating, PluginIdentifier, &PluginName);

	if (!PluginName.IsEmpty())
	{
		UE::TWriteScopeLock WriteLock(GameFeaturePluginNameToPathMapLock);

		// Unmap plugin name to plugin URL
		GameFeaturePluginNameToPathMap.Remove(TO_PLUGIN_STRING(PluginName));
	}
}

void UGameFeaturesSubsystem::OnGameFeatureCheckingStatus(const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::CheckingStatus, PluginIdentifier);
}

void UGameFeaturesSubsystem::OnGameFeatureStatusKnown(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPluginString Name = TO_PLUGIN_STRING(PluginName);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// Map plugin name to plugin URL
	UE::TWriteScopeLock WriteLock(GameFeaturePluginNameToPathMapLock);
	if (ensure(!GameFeaturePluginNameToPathMap.Contains(Name)))
	{
		GameFeaturePluginNameToPathMap.Add(MoveTemp(Name), TO_PLUGIN_STRING(PluginIdentifier.GetFullPluginURL()));
	}
}

void UGameFeaturesSubsystem::OnGameFeaturePredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Predownloading, PluginIdentifier, &PluginName);
}

void UGameFeaturesSubsystem::OnGameFeaturePostPredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::PostPredownloading, PluginIdentifier, &PluginName);
}

void UGameFeaturesSubsystem::OnGameFeatureDownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Downloading, PluginIdentifier, &PluginName);
}

void UGameFeaturesSubsystem::OnGameFeatureReleasing(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Releasing, PluginIdentifier, &PluginName);
}

void UGameFeaturesSubsystem::OnGameFeaturePreMounting(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier, FGameFeaturePreMountingContext& Context)
{
	CallbackObservers(EObserverCallback::PreMounting, PluginIdentifier, &PluginName, /*GameFeatureData=*/nullptr, &Context);
}

void UGameFeaturesSubsystem::OnGameFeaturePostMounting(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier, FGameFeaturePostMountingContext& Context)
{
	CallbackObservers(EObserverCallback::PostMounting, PluginIdentifier, &PluginName, /*GameFeatureData=*/nullptr, &Context);
}

void UGameFeaturesSubsystem::OnGameFeatureRegistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Registering, PluginIdentifier, &PluginName, GameFeatureData);

	for (UGameFeatureAction* Action : GameFeatureData->GetActions())
	{
		if (Action != nullptr)
		{
			Action->OnGameFeatureRegistering();
		}
	}
}

void UGameFeaturesSubsystem::OnGameFeatureUnregistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Unregistering, PluginIdentifier, &PluginName, GameFeatureData);

#if !WITH_EDITOR
	check(GameFeatureData);
#else
	if (GameFeatureData) // In the editor the GameFeatureData asset can be force deleted, otherwise it should exist
#endif
	{
		for (UGameFeatureAction* Action : GameFeatureData->GetActions())
		{
			if (Action != nullptr)
			{
				Action->OnGameFeatureUnregistering();
			}
		}
	}
}

void UGameFeaturesSubsystem::OnGameFeatureLoading(const UGameFeatureData* GameFeatureData, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Loading, PluginIdentifier, nullptr, GameFeatureData);

	for (UGameFeatureAction* Action : GameFeatureData->GetActions())
	{
		if (Action != nullptr)
		{
			Action->OnGameFeatureLoading();
		}
	}
}
void UGameFeaturesSubsystem::OnGameFeatureUnloading(const UGameFeatureData* GameFeatureData, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	CallbackObservers(EObserverCallback::Unloading, PluginIdentifier, nullptr, GameFeatureData);

#if !WITH_EDITOR
	check(GameFeatureData);
#else
	if (GameFeatureData) // In the editor the GameFeatureData asset can be force deleted, otherwise it should exist
#endif
	{
		for (UGameFeatureAction* Action : GameFeatureData->GetActions())
		{
			if (Action != nullptr)
			{
				Action->OnGameFeatureUnloading();
			}
		}
	}
}

void UGameFeaturesSubsystem::OnGameFeatureActivating(const UGameFeatureData* GameFeatureData, const FString& PluginName, FGameFeatureActivatingContext& Context, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_OnActivating_CallbackObservers);
		CallbackObservers(EObserverCallback::Activating, PluginIdentifier, &PluginName, GameFeatureData);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_OnActivating_CallbackActions);
		for (UGameFeatureAction* Action : GameFeatureData->GetActions())
		{
			if (Action != nullptr)
			{
				Action->OnGameFeatureActivating(Context);
			}
		}
	}
}

void UGameFeaturesSubsystem::OnGameFeatureActivated(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_OnActivated_CallbackObservers);
		CallbackObservers(EObserverCallback::Activated, PluginIdentifier, &PluginName, GameFeatureData);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_OnActivated_CallbackActions);
		for (UGameFeatureAction* Action : GameFeatureData->GetActions())
		{
			if (Action != nullptr)
			{
				Action->OnGameFeatureActivated();
			}
		}
	}
}

void UGameFeaturesSubsystem::OnGameFeatureDeactivating(const UGameFeatureData* GameFeatureData, const FString& PluginName, FGameFeatureDeactivatingContext& Context, const FGameFeaturePluginIdentifier& PluginIdentifier)
{
#if !WITH_EDITOR
	check(GameFeatureData);
#else
	if (GameFeatureData) // In the editor the GameFeatureData asset can be force deleted, otherwise it should exist
#endif
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_OnDeactivating_CallbackObservers);
		CallbackObservers(EObserverCallback::Deactivating, PluginIdentifier, &PluginName, GameFeatureData, &Context);
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_OnDeactivating_OnGameFeatureDeactivating);
		for (UGameFeatureAction* Action : GameFeatureData->GetActions())
		{
			if (Action != nullptr)
			{
				Action->OnGameFeatureDeactivating(Context);
			}
		}
	}
}

void UGameFeaturesSubsystem::OnGameFeaturePauseChange(const FGameFeaturePluginIdentifier& PluginIdentifier, const FString& PluginName, FGameFeaturePauseStateChangeContext& Context)
{
	CallbackObservers(EObserverCallback::PauseChanged, PluginIdentifier, &PluginName, nullptr, &Context);
}

const UGameFeatureData* UGameFeaturesSubsystem::GetDataForStateMachine(UGameFeaturePluginStateMachine* GFSM) const
{
	return GetGameFeatureDataForActivePlugin(GFSM);
}

const UGameFeatureData* UGameFeaturesSubsystem::GetRegisteredDataForStateMachine(UGameFeaturePluginStateMachine* GFSM) const
{
	return GetGameFeatureDataForRegisteredPlugin(GFSM);
}

void UGameFeaturesSubsystem::GetGameFeatureDataForActivePlugins(TArray<const UGameFeatureData*>& OutActivePluginFeatureDatas)
{
	for (auto StateMachineIt = GameFeaturePluginStateMachines.CreateConstIterator(); StateMachineIt; ++StateMachineIt)
	{
		if (UGameFeaturePluginStateMachine* GFSM = StateMachineIt.Value())
		{
			if (const UGameFeatureData* GameFeatureData = GetGameFeatureDataForActivePlugin(GFSM))
			{
				OutActivePluginFeatureDatas.Add(GameFeatureData);
			}
		}
	}
}

const UGameFeatureData* UGameFeaturesSubsystem::GetGameFeatureDataForActivePluginByURL(const FString& PluginURL)
{
	if (UGameFeaturePluginStateMachine* GFSM = FindGameFeaturePluginStateMachine(PluginURL))
	{
		return GetGameFeatureDataForActivePlugin(GFSM);
	}

	return nullptr;
}

const UGameFeatureData* UGameFeaturesSubsystem::GetGameFeatureDataForRegisteredPluginByURL(const FString& PluginURL, bool bCheckForRegistering /*= false*/)
{
	if (UGameFeaturePluginStateMachine* GFSM = FindGameFeaturePluginStateMachine(PluginURL))
	{
		return GetGameFeatureDataForRegisteredPlugin(GFSM, bCheckForRegistering);
	}

	return nullptr;
}

bool UGameFeaturesSubsystem::IsGameFeaturePluginInErrorState(const FString& PluginURL) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		return IsInErrorState(StateMachine);
	}
	return false;
}

bool UGameFeaturesSubsystem::IsGameFeaturePluginInstalled(const FString& PluginURL) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		return GetCurrentState(StateMachine) >= EGameFeaturePluginState::Installed;
	}
	return false;
}

bool UGameFeaturesSubsystem::IsGameFeaturePluginMounted(const FString& PluginURL) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		return GetCurrentState(StateMachine) > EGameFeaturePluginState::Mounting;
	}
	return false;
}

bool UGameFeaturesSubsystem::IsGameFeaturePluginRegistered(const FString& PluginURL, bool bCheckForRegistering /*= false*/) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		const EGameFeaturePluginState CurrentState = GetCurrentState(StateMachine);

		return GetCurrentState(StateMachine) >= EGameFeaturePluginState::Registered || (bCheckForRegistering && CurrentState == EGameFeaturePluginState::Registering);
	}
	return false;
}

bool UGameFeaturesSubsystem::IsGameFeaturePluginLoaded(const FString& PluginURL) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		return GetCurrentState(StateMachine) >= EGameFeaturePluginState::Loaded;
	}
	return false;
}

bool UGameFeaturesSubsystem::WasGameFeaturePluginLoadedAsBuiltIn(const FString& PluginURL) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		return WasLoadedAsBuiltIn(StateMachine);
	}
	return false;
}

void UGameFeaturesSubsystem::LoadGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate)
{
	LoadGameFeaturePlugin(PluginURL, FGameFeatureProtocolOptions(), CompleteDelegate);
}

void UGameFeaturesSubsystem::LoadGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginLoadComplete& CompleteDelegate)
{
	const bool bIsPluginAllowed = IsPluginAllowed(PluginURL);
	if (!bIsPluginAllowed)
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginNotAllowed)));
		return;
	}

	UGameFeaturePluginStateMachine* StateMachine = FindOrCreateGameFeaturePluginStateMachine(PluginURL, ProtocolOptions);

	if (!IsRunning(StateMachine) && GetCurrentState(StateMachine) == EGameFeaturePluginState::Active)
	{
		// TODO: Resolve the activated case here, this is needed because in a PIE environment the plugins
		// are not sandboxed, and we need to do simulate a successful activate call in order run GFP systems 
		// on whichever Role runs second between client and server.

		// Refire the observer for Activated and do nothing else.
		CallbackObservers(EObserverCallback::Activating, GetPluginIdentifier(StateMachine), &GetPluginName(StateMachine), GetGameFeatureDataForActivePlugin(StateMachine));
	}

	if (ShouldUpdatePluginProtocolOptions(StateMachine, ProtocolOptions))
	{
		const UE::GameFeatures::FResult Result = UpdateGameFeatureProtocolOptions(StateMachine, ProtocolOptions);
		if (Result.HasError())
		{
			CompleteDelegate.ExecuteIfBound(Result);
			return;
		}
	}

	ChangeGameFeatureDestination(StateMachine, ProtocolOptions, FGameFeaturePluginStateRange(EGameFeaturePluginState::Loaded, EGameFeaturePluginState::Active), CompleteDelegate);
}

void UGameFeaturesSubsystem::LoadGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate)
{
	struct FLoadContext
	{
		TMap<FString, UE::GameFeatures::FResult> Results;
		FMultipleGameFeaturePluginsLoaded CompleteDelegate;

		int32 NumPluginsLoaded = 0;
		bool bPushedTagsBroadcast = false;

		FLoadContext()
		{
			if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
			{
				TagsManager->PushDeferOnGameplayTagTreeChangedBroadcast();
				bPushedTagsBroadcast = true;
			}
		}

		~FLoadContext()
		{
			if (bPushedTagsBroadcast)
			{
				if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
				{
					TagsManager->PopDeferOnGameplayTagTreeChangedBroadcast();
				}
			}

			CompleteDelegate.ExecuteIfBound(Results);
		}
	};
	TSharedRef<FLoadContext> LoadContext = MakeShared<FLoadContext>();
	LoadContext->CompleteDelegate = CompleteDelegate;

	LoadContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		LoadContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	const int32 NumPluginsToLoad = PluginURLs.Num();
	UE_LOGF(LogGameFeatures, Log, "Loading %i GFPs", NumPluginsToLoad);

	for (const FString& PluginURL : PluginURLs)
	{
		LoadGameFeaturePlugin(PluginURL, ProtocolOptions, FGameFeaturePluginChangeStateComplete::CreateLambda([LoadContext, PluginURL](const UE::GameFeatures::FResult& Result)
		{
			LoadContext->Results.Add(PluginURL, Result);
			++LoadContext->NumPluginsLoaded;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Loading %i GFPs", LoadContext->NumPluginsLoaded);
		}));
	}
}

void UGameFeaturesSubsystem::RegisterGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate)
{
	RegisterGameFeaturePlugin(PluginURL, FGameFeatureProtocolOptions(), CompleteDelegate);
}

void UGameFeaturesSubsystem::RegisterGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginLoadComplete& CompleteDelegate)
{
	FString PluginDisallowedReason;
	const bool bIsPluginAllowed = IsPluginAllowed(PluginURL, &PluginDisallowedReason);
	if (!bIsPluginAllowed)
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginNotAllowed),
			FText::Format(NSLOCTEXT("GameFeatures", "PluginDisallowedText", "Plugin disallowed reason: {0}"), FText::FromString(PluginDisallowedReason))));
		return;
	}

	UGameFeaturePluginStateMachine* StateMachine = FindOrCreateGameFeaturePluginStateMachine(PluginURL, ProtocolOptions);

	if (!IsRunning(StateMachine) && GetCurrentState(StateMachine) == EGameFeaturePluginState::Active)
	{
		// TODO: Resolve the activated case here, this is needed because in a PIE environment the plugins
		// are not sandboxed, and we need to do simulate a successful activate call in order run GFP systems 
		// on whichever Role runs second between client and server.

		// Refire the observer for Activated and do nothing else.
		CallbackObservers(EObserverCallback::Activating, GetPluginIdentifier(StateMachine), &GetPluginName(StateMachine), GetGameFeatureDataForActivePlugin(StateMachine));
	}

	if (ShouldUpdatePluginProtocolOptions(StateMachine, ProtocolOptions))
	{
		const UE::GameFeatures::FResult Result = UpdateGameFeatureProtocolOptions(StateMachine, ProtocolOptions);
		if (Result.HasError())
		{
			CompleteDelegate.ExecuteIfBound(Result);
			return;
		}
	}

	ChangeGameFeatureDestination(StateMachine, ProtocolOptions, FGameFeaturePluginStateRange(EGameFeaturePluginState::Registered, EGameFeaturePluginState::Active), CompleteDelegate);
}

void UGameFeaturesSubsystem::RegisterGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate)
{
	struct FLoadContext
	{
		TMap<FString, UE::GameFeatures::FResult> Results;
		FMultipleGameFeaturePluginsLoaded CompleteDelegate;

		int32 NumPluginsLoaded = 0;
		bool bPushedTagsBroadcast = false;

		FLoadContext()
		{
			if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
			{
				TagsManager->PushDeferOnGameplayTagTreeChangedBroadcast();
				bPushedTagsBroadcast = true;
			}
		}

		~FLoadContext()
		{
			if (bPushedTagsBroadcast)
			{
				if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
				{
					TagsManager->PopDeferOnGameplayTagTreeChangedBroadcast();
				}
			}

			CompleteDelegate.ExecuteIfBound(Results);
		}
	};
	TSharedRef<FLoadContext> LoadContext = MakeShared<FLoadContext>();
	LoadContext->CompleteDelegate = CompleteDelegate;

	LoadContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		LoadContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	const int32 NumPluginsToLoad = PluginURLs.Num();
	UE_LOGF(LogGameFeatures, Log, "Registering %i GFPs", NumPluginsToLoad);

	for (const FString& PluginURL : PluginURLs)
	{
		RegisterGameFeaturePlugin(PluginURL, ProtocolOptions, FGameFeaturePluginChangeStateComplete::CreateLambda([LoadContext, PluginURL](const UE::GameFeatures::FResult& Result)
			{
				LoadContext->Results.Add(PluginURL, Result);
				++LoadContext->NumPluginsLoaded;
				UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Registering %i GFPs", LoadContext->NumPluginsLoaded);
			}));
	}
}

void UGameFeaturesSubsystem::LoadAndActivateGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate)
{
	ChangeGameFeatureTargetState(PluginURL, EGameFeatureTargetState::Active, CompleteDelegate);
}

void UGameFeaturesSubsystem::LoadAndActivateGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginLoadComplete& CompleteDelegate)
{
	ChangeGameFeatureTargetState(PluginURL, ProtocolOptions, EGameFeatureTargetState::Active, CompleteDelegate);
}

void UGameFeaturesSubsystem::LoadAndActivateGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate)
{
	ChangeGameFeatureTargetState(PluginURLs, ProtocolOptions, EGameFeatureTargetState::Active, CompleteDelegate);
}

void UGameFeaturesSubsystem::ChangeGameFeatureTargetState(const FString& PluginURL, EGameFeatureTargetState TargetState, const FGameFeaturePluginChangeStateComplete& CompleteDelegate)
{
	ChangeGameFeatureTargetState(PluginURL, FGameFeatureProtocolOptions(), TargetState, CompleteDelegate);
}

void UGameFeaturesSubsystem::ChangeGameFeatureTargetState(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, EGameFeatureTargetState TargetState, const FGameFeaturePluginChangeStateComplete& CompleteDelegate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UGameFeaturesSubsystem::ChangeGameFeatureTargetState");
	EGameFeaturePluginState TargetPluginState = EGameFeaturePluginState::MAX;

	switch (TargetState)
	{
	case EGameFeatureTargetState::Installed:	TargetPluginState = EGameFeaturePluginState::Installed;		break;
	case EGameFeatureTargetState::Registered:	TargetPluginState = EGameFeaturePluginState::Registered;	break;
	case EGameFeatureTargetState::Loaded:		TargetPluginState = EGameFeaturePluginState::Loaded;		break;
	case EGameFeatureTargetState::Active:		TargetPluginState = EGameFeaturePluginState::Active;		break;
	}

	// Make sure we have coverage on all values of EGameFeatureTargetState
	static_assert(std::underlying_type<EGameFeatureTargetState>::type(EGameFeatureTargetState::Count) == 4, "");
	check(TargetPluginState != EGameFeaturePluginState::MAX);


	FString PluginDisallowedReason;
	const bool bIsPluginAllowed = IsPluginAllowed(PluginURL, &PluginDisallowedReason);

	UGameFeaturePluginStateMachine* StateMachine = nullptr;
	if (!bIsPluginAllowed)
	{
		StateMachine = FindGameFeaturePluginStateMachine(PluginURL);
		if (!StateMachine)
		{
			UE_LOGF(LogGameFeatures, Log, "Cannot create GFP State Machine: Plugin not allowed %ls", *PluginURL);

			CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginNotAllowed),
				FText::Format(NSLOCTEXT("GameFeatures", "PluginDisallowedText", "Plugin disallowed reason: {0}"), FText::FromString(PluginDisallowedReason))));
			return;
		}
	}
	else
	{
		StateMachine = FindOrCreateGameFeaturePluginStateMachine(PluginURL, ProtocolOptions);
	}
	
	check(StateMachine);

	if (!bIsPluginAllowed)
	{
		if (TargetPluginState > GetCurrentState(StateMachine) || TargetPluginState > GetDestination(StateMachine))
		{
			UE_LOGF(LogGameFeatures, Log, "Cannot change game feature target state: Plugin not allowed %ls", *PluginURL);

			CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginNotAllowed),
				FText::Format(NSLOCTEXT("GameFeatures", "PluginDisallowedText", "Plugin disallowed reason: {0}"), FText::FromString(PluginDisallowedReason))));
			return;
		}
	}

	if (TargetState == EGameFeatureTargetState::Active &&
		!IsRunning(StateMachine) && 
		GetCurrentState(StateMachine) == TargetPluginState)
	{
		// TODO: Resolve the activated case here, this is needed because in a PIE environment the plugins
		// are not sandboxed, and we need to do simulate a successful activate call in order run GFP systems 
		// on whichever Role runs second between client and server.

		// Refire the observer for Activated and do nothing else.
		CallbackObservers(EObserverCallback::Activating, GetPluginIdentifier(StateMachine), &GetPluginName(StateMachine), GetGameFeatureDataForActivePlugin(StateMachine));
	}
	
	if (ShouldUpdatePluginProtocolOptions(StateMachine, ProtocolOptions))
	{
		const UE::GameFeatures::FResult Result = UpdateGameFeatureProtocolOptions(StateMachine, ProtocolOptions);
		if (Result.HasError())
		{
			CompleteDelegate.ExecuteIfBound(Result);
			return;
		}
	}
	
	ChangeGameFeatureDestination(StateMachine, ProtocolOptions, FGameFeaturePluginStateRange(TargetPluginState), CompleteDelegate);
}

void UGameFeaturesSubsystem::ChangeGameFeatureTargetState(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, EGameFeatureTargetState TargetState, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate)
{
	struct FLoadContext
	{
		TMap<FString, UE::GameFeatures::FResult> Results;
		FMultipleGameFeaturePluginsLoaded CompleteDelegate;

		int32 NumPluginsLoaded = 0;
		bool bPushedTagsBroadcast = false;

		FLoadContext()
		{
			if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
			{
				TagsManager->PushDeferOnGameplayTagTreeChangedBroadcast();
				bPushedTagsBroadcast = true;
			}
		}

		~FLoadContext()
		{
			if (bPushedTagsBroadcast)
			{
				if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
				{
					TagsManager->PopDeferOnGameplayTagTreeChangedBroadcast();
				}
			}

			CompleteDelegate.ExecuteIfBound(Results);
		}
	};
	TSharedRef<FLoadContext> LoadContext = MakeShared<FLoadContext>();
	LoadContext->CompleteDelegate = CompleteDelegate;

	LoadContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		LoadContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	const int32 NumPluginsToLoad = PluginURLs.Num();
	UE_LOGF(LogGameFeatures, Log, "Transitioning (%ls) %i GFPs", *LexToString(TargetState), NumPluginsToLoad);

	for (const FString& PluginURL : PluginURLs)
	{
		ChangeGameFeatureTargetState(PluginURL, ProtocolOptions, TargetState, FGameFeaturePluginChangeStateComplete::CreateLambda([LoadContext, PluginURL, TargetState](const UE::GameFeatures::FResult& Result)
		{
			LoadContext->Results.Add(PluginURL, Result);
			++LoadContext->NumPluginsLoaded;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Transitioning (%ls) %i GFPs", *LexToString(TargetState), LoadContext->NumPluginsLoaded);
		}));
	}
}

UE::GameFeatures::FResult UGameFeaturesSubsystem::UpdateGameFeatureProtocolOptions(const FString& PluginURL, const FGameFeatureProtocolOptions& NewOptions, bool* bOutDidUpdate /*= nullptr*/)
{
	UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL);
	return UpdateGameFeatureProtocolOptions(StateMachine, NewOptions, bOutDidUpdate);
}

UE::GameFeatures::FResult UGameFeaturesSubsystem::UpdateGameFeatureProtocolOptions(UGameFeaturePluginStateMachine* StateMachine, const FGameFeatureProtocolOptions& NewOptions, bool* bOutDidUpdate /*= nullptr*/)
{
	if (bOutDidUpdate)
	{
		*bOutDidUpdate = false;
	}

	if (!StateMachine)
	{
		return MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::BadURL);
	}

	bool bUpdated = false;
	UE::GameFeatures::FResult Result = TryUpdatePluginProtocolOptions(StateMachine, NewOptions, bUpdated);
	if (bOutDidUpdate)
	{
		*bOutDidUpdate = bUpdated;
	}

	return Result;
}

bool UGameFeaturesSubsystem::ShouldUpdatePluginProtocolOptions(const UGameFeaturePluginStateMachine* StateMachine, const FGameFeatureProtocolOptions& NewOptions)
{
	if (NewOptions.HasSubtype<FNull>())
	{
		return false;
	}

	if (!StateMachine)
	{
		return false;
	}
	
	//Make sure our StateMachine isn't in terminal, don't want to update Terminal plugins
	if (TerminalGameFeaturePluginStateMachines.Contains(StateMachine) || GetCurrentState(StateMachine) == EGameFeaturePluginState::Terminal)
	{
		return false;
	}

	if (GetProtocolOptions(StateMachine) == NewOptions)
	{
		return false;
	}
	
	return true;
}

namespace UE::GameFeatures
{
	static float CombineInstallProgress(float InstallProgress, float AssetDependencyProgress)
	{
		// Assumuption that most of the progress will be from asset dependencies in this case
		// For this to be more accurate we'd need to figure out the actual sizes during 
		// EGameFeaturePluginState::CheckingStatus but this is most likely good enough
		return 0.2f * InstallProgress + 0.8f * AssetDependencyProgress;
	}
}

bool UGameFeaturesSubsystem::GetGameFeaturePluginInstallPercent(const FString& PluginURL, float& Install_Percent) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		if (IsStatusKnown(StateMachine) && IsAvailable(StateMachine))
		{
			const FGameFeaturePluginStateInfo& StateInfo = GetCurrentStateInfo(StateMachine);

			float InstallProgress = 0.0f;
			if (StateInfo.State == EGameFeaturePluginState::Downloading)
			{
				InstallProgress = StateInfo.Progress;
			}
			else if (StateInfo.State >= EGameFeaturePluginState::Installed)
			{
				InstallProgress = 1.0f;
			}

			if (!HasAssetStreamingDependencies(StateMachine))
			{
				Install_Percent = InstallProgress;
				return true;
			}

			float AssetDependencyProgress = 0.0f;
			if (StateInfo.State == EGameFeaturePluginState::AssetDependencyStreaming)
			{
				AssetDependencyProgress = StateInfo.Progress;
			}
			else if(StateInfo.State >= EGameFeaturePluginState::Registering)
			{
				AssetDependencyProgress = 1.0f;
			}

			Install_Percent = UE::GameFeatures::CombineInstallProgress(InstallProgress, AssetDependencyProgress);
			return true;
		}
	}
	return false;
}

bool UGameFeaturesSubsystem::GetGameFeaturePluginInstallPercent(TConstArrayView<FString> PluginURLs, float& Install_Percent) const
{
	float TotalInstallPercent = 0;
	int32 NumFound = 0;

	for (const FString& URL : PluginURLs)
	{
		float SingleInstallPercent = 0;
		if (GetGameFeaturePluginInstallPercent(URL, SingleInstallPercent))
		{
			TotalInstallPercent += SingleInstallPercent;
			++NumFound;
		}
	}

	if (NumFound > 0)
	{
		Install_Percent = TotalInstallPercent / NumFound;
		return true;
	}

	return false;
}

bool UGameFeaturesSubsystem::DoesGameFeaturePluginNeedUpdate(const FString& PluginURL) const
{
	TArray<FName> InstallBundles;
	const bool bParseSuccess = UGameFeaturesSubsystem::ParsePluginURLOptions(PluginURL, EGameFeatureURLOptions::Bundles,
		[&InstallBundles](EGameFeatureURLOptions Option, FStringView OptionName, FStringView OptionValue)
		{
			check(Option == EGameFeatureURLOptions::Bundles);
			InstallBundles.Emplace(OptionValue);
		});

	if (InstallBundles.IsEmpty())
	{
		return false;
	}

	TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
	TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> InstallStateResult = BundleManager->GetInstallStateSynchronous(InstallBundles, false);
	if (InstallStateResult.HasError())
	{
		UE_LOGF(LogGameFeatures, Error, "Failed to get install state for PluginURL %ls : Error reason %ls", *PluginURL, LexToString(InstallStateResult.GetError()));
		return false;
	}

	return InstallStateResult.GetValue().GetAnyBundleHasState(EInstallBundleInstallState::NeedsUpdate);
}

bool UGameFeaturesSubsystem::IsGameFeaturePluginActive(const FString& PluginURL, bool bCheckForActivating /*= false*/) const
{
	if (const UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		const EGameFeaturePluginState CurrentState = GetCurrentState(StateMachine);

		return CurrentState == EGameFeaturePluginState::Active || (bCheckForActivating && CurrentState == EGameFeaturePluginState::Activating);
	}

	return false;
}

bool UGameFeaturesSubsystem::IsGameFeaturePluginActiveByName(FStringView PluginName, bool bCheckForActivating /*= false*/) const
{
	FString PluginURLName;
	if (GetPluginURLByName(PluginName, PluginURLName))
	{
		if (!IsGameFeaturePluginActive(PluginURLName, bCheckForActivating))
		{
			return false;
		}
	}
	else
	{
		return false;
	}

	return true;
}

// Internal structs for suspending systems so we can better batch change GFPs from states with out spamming these systems
namespace GameFeaturesSubsystem
{
	struct FNotifyObservers
	{
		UE_NODISCARD_CTOR FNotifyObservers()
		{
			if (GEngine)
			{
				UGameFeaturesSubsystem::Get().BatchCallbackObservers(UGameFeaturesSubsystem::EBatchObserverCallback::BatchOperationStarted);
			}
		}

		~FNotifyObservers()
		{
			if (GEngine)
			{
				UGameFeaturesSubsystem::Get().BatchCallbackObservers(UGameFeaturesSubsystem::EBatchObserverCallback::BatchOperationComplete);
			}
		}
	};

	struct FSuspendRenderAssetStreamer
	{
		UE_NODISCARD_CTOR FSuspendRenderAssetStreamer()
		{
			SuspendRenderAssetStreaming();
		}

		~FSuspendRenderAssetStreamer()
		{
			ResumeRenderAssetStreaming();
		}
	};

	struct FSuspendPluginManager
	{
		UE_NODISCARD_CTOR FSuspendPluginManager()
		{
			IPluginManager::Get().SuppressPluginUnloadGC();
		}

		~FSuspendPluginManager()
		{
			IPluginManager::Get().ResumePluginUnloadGC();
		}
	};

	struct FSuspendGameplayTagTreeChangedBroadcast
	{
		UE_NODISCARD_CTOR FSuspendGameplayTagTreeChangedBroadcast()
		{
			if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
			{
				TagsManager->PushDeferOnGameplayTagTreeChangedBroadcast();
				bWasSuspended = true;
			}
		}

		~FSuspendGameplayTagTreeChangedBroadcast()
		{
			if (bWasSuspended)
			{
				if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
				{
					TagsManager->PopDeferOnGameplayTagTreeChangedBroadcast();
				}
			}
		}

		bool bWasSuspended = false;
	};

	enum class ETransitionOptions
	{
		None							= 0,
		SuspendPluginManager			= 1 << 0,
		SuspendGameplayTagTreeBroadcast	= 1 << 1,
		SuspendAssetRegistry			= 1 << 2,
		NotifyObservers					= 1 << 3,
		SuspendRenderAssetStreamer		= 1 << 4,
		All = SuspendPluginManager | SuspendGameplayTagTreeBroadcast | SuspendAssetRegistry | NotifyObservers | SuspendRenderAssetStreamer,
	};
	ENUM_CLASS_FLAGS(ETransitionOptions)

	struct FTransitionContenxt
	{
		TMap<FString, UE::GameFeatures::FResult> Results;
		FMultipleGameFeaturePluginsReleased CompleteDelegate;
		int32 NumPluginsTransitioned = 0;

		TOptional<FSuspendRenderAssetStreamer> SuspendRenderAssetStreamer; 
		TOptional<FSuspendPluginManager> SuspendPluginManager;
		TOptional<FSuspendGameplayTagTreeChangedBroadcast> SuspendGameplayTagTreeBroadcast;
		TOptional<IAssetRegistry::FSuspendPathsDismounting> SuspendAssetRegistry;
		TOptional<FNotifyObservers> NotifyObservers;

		UE_NODISCARD_CTOR FTransitionContenxt(ETransitionOptions TransitionOptions = ETransitionOptions::All)
		{
			if (EnumHasAnyFlags(TransitionOptions, ETransitionOptions::SuspendRenderAssetStreamer))
			{
				if (!IsEngineExitRequested())
				{
					SuspendRenderAssetStreamer.Emplace();
				}
			}

			if (EnumHasAnyFlags(TransitionOptions, ETransitionOptions::SuspendPluginManager))
			{
				if (!IsEngineExitRequested())
				{
					SuspendPluginManager.Emplace();
				}
			}

			if (EnumHasAnyFlags(TransitionOptions, ETransitionOptions::SuspendGameplayTagTreeBroadcast))
			{
				SuspendGameplayTagTreeBroadcast.Emplace();
			}

			if (EnumHasAnyFlags(TransitionOptions, ETransitionOptions::SuspendAssetRegistry))
			{
				SuspendAssetRegistry.Emplace();
			}

			if (EnumHasAnyFlags(TransitionOptions, ETransitionOptions::NotifyObservers))
			{
				NotifyObservers.Emplace();
			}
		}

		~FTransitionContenxt()
		{
			CompleteDelegate.ExecuteIfBound(Results);
			// Explicit order of destruction is required
			// At the very least we need to execute PluginManger batch at the end and render asset streamer last
			// as it will run GC and does the leaked packages verification.
			// And by that point all other resources need to be properly cleared,
			// i.e. other batches need to finish.
			NotifyObservers.Reset(); // Some observers are responsible for clearing references; must happen before SuspendPluginManager.Reset()
			SuspendAssetRegistry.Reset();
			SuspendGameplayTagTreeBroadcast.Reset();
			SuspendPluginManager.Reset();
			SuspendRenderAssetStreamer.Reset();
		}
	};
}

void UGameFeaturesSubsystem::DeactivateGameFeaturePlugin(const FString& PluginURL)
{
	DeactivateGameFeaturePlugin(PluginURL, FGameFeaturePluginDeactivateComplete());
}

void UGameFeaturesSubsystem::DeactivateGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginDeactivateComplete& CompleteDelegate)
{
	if (UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		ChangeGameFeatureDestination(StateMachine, FGameFeaturePluginStateRange(EGameFeaturePluginState::Terminal, EGameFeaturePluginState::Loaded), CompleteDelegate);
	}
	else
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::BadURL)));
	}
}

void UGameFeaturesSubsystem::DeactivateGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsDeactivate& CompleteDelegate)
{
	// None for now, as the current list are more of an issue going from Registered -> <lower>
	GameFeaturesSubsystem::ETransitionOptions TransitionOptions = GameFeaturesSubsystem::ETransitionOptions::None;
	TSharedRef<GameFeaturesSubsystem::FTransitionContenxt> TransitionContext = MakeShared<GameFeaturesSubsystem::FTransitionContenxt>(TransitionOptions);
	TransitionContext->CompleteDelegate = CompleteDelegate;

	TransitionContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		TransitionContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	UE_LOGF(LogGameFeatures, Log, "Deactivating %i GFPs", PluginURLs.Num());

	for (const FString& PluginURL : PluginURLs)
	{
		DeactivateGameFeaturePlugin(PluginURL, FGameFeaturePluginDeactivateComplete::CreateLambda([TransitionContext, PluginURL](const UE::GameFeatures::FResult& Result)
		{
			TransitionContext->Results[PluginURL] = Result;
			++TransitionContext->NumPluginsTransitioned;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Deactivating %i/%i GFPs", TransitionContext->NumPluginsTransitioned, TransitionContext->Results.Num());
		}));
	}
}

void UGameFeaturesSubsystem::UnloadGameFeaturePlugin(const FString& PluginURL, bool bKeepRegistered /*= false*/)
{
	UnloadGameFeaturePlugin(PluginURL, FGameFeaturePluginUnloadComplete(), bKeepRegistered);
}

void UGameFeaturesSubsystem::UnloadGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginUnloadComplete& CompleteDelegate, bool bKeepRegistered /*= false*/)
{
	if (UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		EGameFeaturePluginState TargetPluginState = bKeepRegistered ? EGameFeaturePluginState::Registered : EGameFeaturePluginState::Installed;
		ChangeGameFeatureDestination(StateMachine, FGameFeaturePluginStateRange(EGameFeaturePluginState::Terminal, TargetPluginState), CompleteDelegate);
	}
	else
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::BadURL)));
	}
}

void UGameFeaturesSubsystem::UnloadGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsUnload& CompleteDelegate, bool bKeepRegistered /*= false*/)
{
	// Only set the downgrade options to suspend things when going below Registered for now
	GameFeaturesSubsystem::ETransitionOptions TransitionOptions = GameFeaturesSubsystem::ETransitionOptions::None;
	if (bKeepRegistered)
	{
		TransitionOptions = GameFeaturesSubsystem::ETransitionOptions::All;
	}
	if (UE::GameFeatures::CVarDisablePluginUnloadBatchingDuringUnload->GetBool())
	{
		EnumRemoveFlags(TransitionOptions, GameFeaturesSubsystem::ETransitionOptions::SuspendPluginManager);
	}
	if (!UE::GameFeatures::CVarTransitionContextSuspendStreamingManager->GetBool())
	{
		EnumRemoveFlags(TransitionOptions, GameFeaturesSubsystem::ETransitionOptions::SuspendRenderAssetStreamer);
	}

	TSharedRef<GameFeaturesSubsystem::FTransitionContenxt> TransitionContext = MakeShared<GameFeaturesSubsystem::FTransitionContenxt>(TransitionOptions);
	TransitionContext->CompleteDelegate = CompleteDelegate;

	TransitionContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		TransitionContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	UE_LOGF(LogGameFeatures, Log, "Unloading %i GFPs bKeepRegistered(%i)", PluginURLs.Num(), bKeepRegistered);

	for (const FString& PluginURL : PluginURLs)
	{
		UnloadGameFeaturePlugin(PluginURL, FGameFeaturePluginUnloadComplete::CreateLambda([TransitionContext, PluginURL](const UE::GameFeatures::FResult& Result)
		{
			TransitionContext->Results[PluginURL] = Result;
			++TransitionContext->NumPluginsTransitioned;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Unloading %i/%i GFPs", TransitionContext->NumPluginsTransitioned, TransitionContext->Results.Num());
		}), bKeepRegistered);
	}
}

void UGameFeaturesSubsystem::ReleaseGameFeaturePlugin(const FString& PluginURL)
{
	ReleaseGameFeaturePlugin(PluginURL, FGameFeaturePluginReleaseComplete());
}

void UGameFeaturesSubsystem::ReleaseGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginReleaseComplete& CompleteDelegate)
{
	if (UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		ChangeGameFeatureDestination(StateMachine, FGameFeaturePluginStateRange(EGameFeaturePluginState::Terminal, EGameFeaturePluginState::StatusKnown), CompleteDelegate);
	}
	else
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::BadURL)));
	}
}

void UGameFeaturesSubsystem::ReleaseGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsReleased& CompleteDelegate)
{
	TSharedRef<GameFeaturesSubsystem::FTransitionContenxt> TransitionContext = MakeShared<GameFeaturesSubsystem::FTransitionContenxt>();
	TransitionContext->CompleteDelegate = CompleteDelegate;

	TransitionContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		TransitionContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	UE_LOGF(LogGameFeatures, Log, "Releasing %i GFPs", PluginURLs.Num());

	for (const FString& PluginURL : PluginURLs)
	{
		ReleaseGameFeaturePlugin(PluginURL, FGameFeaturePluginReleaseComplete::CreateLambda([TransitionContext, PluginURL](const UE::GameFeatures::FResult& Result)
		{
			TransitionContext->Results[PluginURL] = Result;
			++TransitionContext->NumPluginsTransitioned;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Releasing %i/%i GFPs", TransitionContext->NumPluginsTransitioned, TransitionContext->Results.Num());
		}));
	}
}

void UGameFeaturesSubsystem::UninstallGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginUninstallComplete& CompleteDelegate /*= FGameFeaturePluginUninstallComplete()*/)
{
	UninstallGameFeaturePlugin(PluginURL, FGameFeatureProtocolOptions(), CompleteDelegate);
}

void UGameFeaturesSubsystem::UninstallGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& InProtocolOptions, const FGameFeaturePluginUninstallComplete& CompleteDelegate /*= FGameFeaturePluginUninstallComplete()*/)
{
	// FindOrCreate so that we can make sure we uninstall data for plugins that were installed on a previous application run
	// but have not yet been requested on this application run and so are not yet in the plugin list but might have data on disk
	// to uninstall
	UGameFeaturePluginStateMachine* StateMachine = FindOrCreateGameFeaturePluginStateMachine(PluginURL, InProtocolOptions);
	check(StateMachine);

	// We may need to update our ProtocolOptions to force certain metadata changes to facilitate this uninstall
	FGameFeatureProtocolOptions ProtocolOptions = GetProtocolOptions(StateMachine);

	// InstallBundle Protocol GameFeatures may need to change their metadata to force this uninstall
	if (GetPluginIdentifier(StateMachine).GetPluginProtocol() == EGameFeaturePluginProtocol::InstallBundle)
	{
		// It's possible that ParseURL hasn't been called yet so setup options here if needed.
		if (!ProtocolOptions.HasSubtype<FInstallBundlePluginProtocolOptions>())
		{
			ensureMsgf(ProtocolOptions.HasSubtype<FNull>(), TEXT("Protocol options type is incorrect for URL %s"), *PluginURL);
			ProtocolOptions.SetSubtype<FInstallBundlePluginProtocolOptions>();
		}

		// Need to force on bUninstallBeforeTerminate if it wasn't already set to on in our Metadata
		FInstallBundlePluginProtocolOptions& InstallBundleOptions = ProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();
		if (!InstallBundleOptions.bUninstallBeforeTerminate)
		{
			InstallBundleOptions.bUninstallBeforeTerminate = true;
		}
	}

	// Weird flow here because we need to do a few tasks asynchronously
	// 1) Update Protocol Options   -->   2) Call to set destination to Uninstall --> 3) After we get to Uninstall go to Terminate

	// FIRST:
	// If we need to update our ProtocolOptions, do that first before starting the Uninstall. This allows us to update
	// options that might be important on the way to Terminal if they are changed. EX: FInstallBundlePluginProtocolMetaData::bUninstallBeforeTerminate
	if (ShouldUpdatePluginProtocolOptions(StateMachine, ProtocolOptions))
	{
		const UE::GameFeatures::FResult Result = UpdateGameFeatureProtocolOptions(StateMachine, ProtocolOptions);
		if (Result.HasError())
		{
			CompleteDelegate.ExecuteIfBound(Result);
			return;
		}
	}

	// SECOND:
	// Kick off the Uninstall destination after updating our options if necessary
	ChangeGameFeatureDestination(StateMachine, ProtocolOptions, FGameFeaturePluginStateRange(EGameFeaturePluginState::Uninstalled),
		FGameFeaturePluginTerminateComplete::CreateWeakLambda(this, [this, PluginURL, CompleteDelegate](const UE::GameFeatures::FResult& Result)
		{
			// THIRD:
			// Kick off the actual Terminate after we successfully transition to Uninstalled state
			if (Result.HasValue())
			{
				TerminateGameFeaturePlugin(PluginURL, CompleteDelegate);
			}
			//If we failed just bubble error up
			else
			{
				CompleteDelegate.ExecuteIfBound(Result);
			}
		}));
}

void UGameFeaturesSubsystem::TerminateGameFeaturePlugin(const FString& PluginURL)
{
	TerminateGameFeaturePlugin(PluginURL, FGameFeaturePluginTerminateComplete());
}

void UGameFeaturesSubsystem::TerminateGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginTerminateComplete& CompleteDelegate)
{
	if (UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		ChangeGameFeatureDestination(StateMachine, FGameFeaturePluginStateRange(EGameFeaturePluginState::Terminal), CompleteDelegate);
	}
	else
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::BadURL)));
	}
}

void UGameFeaturesSubsystem::TerminateGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsTerminated& CompleteDelegate)
{
	GameFeaturesSubsystem::ETransitionOptions TransitionOptions = GameFeaturesSubsystem::ETransitionOptions::All;
	if (UE::GameFeatures::CVarDisablePluginUnloadBatchingDuringTermination->GetBool())
	{
		EnumRemoveFlags(TransitionOptions, GameFeaturesSubsystem::ETransitionOptions::SuspendPluginManager);
	}
	if (!UE::GameFeatures::CVarTransitionContextSuspendStreamingManager->GetBool())
	{
		EnumRemoveFlags(TransitionOptions, GameFeaturesSubsystem::ETransitionOptions::SuspendRenderAssetStreamer);
	}

	TSharedRef<GameFeaturesSubsystem::FTransitionContenxt> TransitionContext = MakeShared<GameFeaturesSubsystem::FTransitionContenxt>(TransitionOptions);
	TransitionContext->CompleteDelegate = CompleteDelegate;

	TransitionContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		TransitionContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	UE_LOGF(LogGameFeatures, Log, "Terminating %i GFPs", PluginURLs.Num());

	for (const FString& PluginURL : PluginURLs)
	{
		TerminateGameFeaturePlugin(PluginURL, FGameFeaturePluginTerminateComplete::CreateLambda([TransitionContext, PluginURL](const UE::GameFeatures::FResult& Result)
		{
			TransitionContext->Results.Add(PluginURL, Result);
			++TransitionContext->NumPluginsTransitioned;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Terminating %i GFPs", TransitionContext->NumPluginsTransitioned);
		}));
	}
}

bool UGameFeaturesSubsystem::CancellAmortizedLoads(const FString& PluginURL)
{
	bool bDidCancellAnything = false;
	const FGameFeaturePluginIdentifier CancellingPluginNameId(PluginURL);
	const FString CancellingPluginName = FString(CancellingPluginNameId.GetPluginName());
	if (!CancellingPluginName.IsEmpty())
	{
		for (TPair<FGuid, TWeakPtr<FAmmortizedLoadContext>>& AmmortizedLoad : ActiveAmmoritzedLoads)
		{
			if (AmmortizedLoad.Value.IsValid())
			{
				for (const TPair<FString, UE::GameFeatures::FResult>& PluginToResult : AmmortizedLoad.Value.Pin()->Results)
				{
					if (PluginToResult.Key.Equals(CancellingPluginName, ESearchCase::IgnoreCase))
					{
						AmmortizedLoad.Value.Pin()->bCancelled = true;
						bDidCancellAnything = true;
						break;
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogGameFeatures, Error, "Failed to find the PluginName from the PluginURL(%ls) failing to cancel this plugin", *PluginURL);
	}

	return bDidCancellAnything;
}

void UGameFeaturesSubsystem::CancelGameFeatureStateChange(const FString& PluginURL)
{
	CancelGameFeatureStateChange(PluginURL, FGameFeaturePluginChangeStateComplete());
}

void UGameFeaturesSubsystem::CancelGameFeatureStateChange(const FString& PluginURL, const FGameFeaturePluginChangeStateComplete& CompleteDelegate)
{
	CancellAmortizedLoads(PluginURL);

	if (UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		const bool bCancelPending = TryCancel(StateMachine, FGameFeatureStateTransitionCanceled::CreateWeakLambda(this, [CompleteDelegate](UGameFeaturePluginStateMachine* Machine)
		{
			CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeValue()));
		}));

		if (!bCancelPending)
		{
			CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeValue()));
		}
	}
	else
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::BadURL)));
	}
}

void UGameFeaturesSubsystem::CancelGameFeatureStateChange(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginChangeStateComplete& CompleteDelegate)
{
	struct FContext
	{
		TMap<FString, UE::GameFeatures::FResult> Results;
		FMultipleGameFeaturePluginsLoaded CompleteDelegate;

		int32 NumPluginsCanceled = 0;

		~FContext()
		{
			CompleteDelegate.ExecuteIfBound(Results);
		}
	};
	TSharedRef<FContext> CancelContext = MakeShared<FContext>();
	CancelContext->CompleteDelegate = CompleteDelegate;

	CancelContext->Results.Reserve(PluginURLs.Num());
	for (const FString& PluginURL : PluginURLs)
	{
		CancelContext->Results.Add(PluginURL, MakeError("Pending"));
	}

	UE_LOGF(LogGameFeatures, Log, "Canceling %i GFP transitions", PluginURLs.Num());

	for (const FString& PluginURL : PluginURLs)
	{
		CancelGameFeatureStateChange(PluginURL, FGameFeaturePluginChangeStateComplete::CreateLambda([CancelContext, PluginURL](const UE::GameFeatures::FResult& Result)
		{
			CancelContext->Results.Add(PluginURL, Result);
			++CancelContext->NumPluginsCanceled;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished canceling %i GFP transitions", CancelContext->NumPluginsCanceled);
		}));
	}
}

void UGameFeaturesSubsystem::LoadBuiltInGameFeaturePlugin(const TSharedRef<IPlugin>& Plugin, FBuiltInPluginAdditionalFilters AdditionalFilter, const FGameFeaturePluginLoadComplete& CompleteDelegate /*= FGameFeaturePluginLoadComplete()*/)
{
	UE_SCOPED_ENGINE_ACTIVITY(TEXT("Loading GameFeaturePlugin %s"), *Plugin->GetName());

#if WITH_EDITOR
	auto AddUnmountedPluginExplination = [this, &Plugin](const UE::GameFeatures::FResult& Result)
	{
		UnmountedPluginNameToExplanation.FindOrAdd(Plugin->GetName()) = UE::GameFeatures::ToString(Result);
		return Result;
	};
#else
	auto AddUnmountedPluginExplination = [](const UE::GameFeatures::FResult& Result)
	{
		return Result;
	};
#endif // WITH_EDITOR

	UAssetManager::Get().PushBulkScanning();
	ON_SCOPE_EXIT
	{
		UAssetManager::Get().PopBulkScanning();
	};

	const FString& PluginDescriptorFilename = Plugin->GetDescriptorFileName();

	FString PluginURL;

	if (PluginDescriptorFilename.IsEmpty())
	{
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginDetailsNotFound)));
		return;
	}
	else if (!GetDefault<UGameFeaturesSubsystemSettings>()->IsValidGameFeaturePlugin(FPaths::ConvertRelativePathToFull(PluginDescriptorFilename)))
	{
		// Not a GFP, trivial success
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeValue()));
		return;
	}
	else if (!FPaths::FileExists(PluginDescriptorFilename))
	{
		CompleteDelegate.ExecuteIfBound(AddUnmountedPluginExplination(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginDetailsNotFound)));
		return;
	}

	TSharedPtr<const FGameFeaturePluginDetails> PluginDetails = GetGameFeaturePluginDetailsInternal(PluginDescriptorFilename);
	if (PluginDetails == nullptr)
	{
		CompleteDelegate.ExecuteIfBound(AddUnmountedPluginExplination(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginDetailsNotFound)));
	}
	else if (!GetBuiltInGameFeaturePluginURL(Plugin, PluginURL))
	{
		CompleteDelegate.ExecuteIfBound(AddUnmountedPluginExplination(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginURLNotFound)));
	}
	else if (FString PluginDisallowedReason; !IsPluginAllowed(PluginURL, &PluginDisallowedReason))
	{
		CompleteDelegate.ExecuteIfBound(
			AddUnmountedPluginExplination(
				UE::GameFeatures::FResult(
					MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginNotAllowed),
					FText::Format(NSLOCTEXT("GameFeatures", "PluginDisallowedText", "Plugin disallowed reason: {0}"), FText::FromString(PluginDisallowedReason))
				)
			)
		);
	}
	else
	{
		FBuiltInGameFeaturePluginBehaviorOptions BehaviorOptions;
		const bool bShouldProcess = AdditionalFilter(Plugin->GetDescriptorFileName(), *PluginDetails, BehaviorOptions);
		if (!bShouldProcess)
		{
			CompleteDelegate.ExecuteIfBound(AddUnmountedPluginExplination(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::PluginFiltered)));
		}
		else
		{
			if (UE::GameFeatures::bAllowDownloadingAfterFirstDownloadRequest)
			{
				if (const UGameFeaturePluginStateMachine* GFSM = FindGameFeaturePluginStateMachine(PluginURL))
				{
					FGameFeatureProtocolOptions CurrentProtocolOptions = GetProtocolOptions(GFSM);
					if (CurrentProtocolOptions.HasSubtype<FInstallBundlePluginProtocolOptions>())
					{
						const FInstallBundlePluginProtocolOptions& CurrentInstallBundleOptions = CurrentProtocolOptions.GetSubtype<FInstallBundlePluginProtocolOptions>();
						BehaviorOptions.bDoNotDownload &= CurrentInstallBundleOptions.bDoNotDownload; // already allowed to be downloaded.
					}
				}
			}
			FGameFeatureProtocolOptions ProtocolOptions;
			if (UGameFeaturesSubsystem::GetPluginURLProtocol(PluginURL) == EGameFeaturePluginProtocol::InstallBundle)
			{
				FInstallBundlePluginProtocolOptions InstallBundleOptions;
				InstallBundleOptions.bAllowIniLoading = true;
				InstallBundleOptions.bDoNotDownload = BehaviorOptions.bDoNotDownload;
				ProtocolOptions = FGameFeatureProtocolOptions(InstallBundleOptions);
			}
			ProtocolOptions.bForceSyncLoading = BehaviorOptions.bForceSyncLoading;
			ProtocolOptions.bLogWarningOnForcedDependencyCreation = BehaviorOptions.bLogWarningOnForcedDependencyCreation;
			ProtocolOptions.bLogErrorOnForcedDependencyCreation = BehaviorOptions.bLogErrorOnForcedDependencyCreation;
			ProtocolOptions.bBatchProcess = BehaviorOptions.bBatchProcess;

			bool bFoundExisting = false;
			UGameFeaturePluginStateMachine* StateMachine = FindOrCreateGameFeaturePluginStateMachine(PluginURL, ProtocolOptions, &bFoundExisting);
			if (bFoundExisting && ShouldUpdatePluginProtocolOptions(StateMachine, ProtocolOptions))
			{
				const UE::GameFeatures::FResult Result = UpdateGameFeatureProtocolOptions(StateMachine, ProtocolOptions);
				if (Result.HasError())
				{
					CompleteDelegate.ExecuteIfBound(Result);
					return;
				}
			}

			EGameFeaturePluginState MaximumState = EGameFeaturePluginState::Active;
			switch (BehaviorOptions.MaximumState)
			{
			case EGameFeatureTargetState::Installed:	MaximumState = EGameFeaturePluginState::Installed;	break;
			case EGameFeatureTargetState::Registered:	MaximumState = EGameFeaturePluginState::Registered;	break;
			case EGameFeatureTargetState::Loaded:		MaximumState = EGameFeaturePluginState::Loaded;		break;
			case EGameFeatureTargetState::Active:		MaximumState = EGameFeaturePluginState::Active;		break;
			}
			// If this state machine already existed and was in a state the behavior options disallow, log an error
			UE_CLOGFMT(bFoundExisting && StateMachine->StateProperties.CurrentState > MaximumState,
				LogGameFeatures, Error, "Attempted to limit builtin GFP {PluginName} to state {MaximumState} but it was already in a higher state {CurrentState}",
				Plugin->GetName(), MaximumState, StateMachine->StateProperties.CurrentState);
			StateMachine->StateProperties.MaximumState = MaximumState;

			EBuiltInAutoState InitialAutoState = (BehaviorOptions.AutoStateOverride != EBuiltInAutoState::Invalid) ? 
				BehaviorOptions.AutoStateOverride : PluginDetails->BuiltInAutoState;
			if (InitialAutoState < EBuiltInAutoState::Registered && IsRunningCookCommandlet())
			{
				UE_LOGF(LogGameFeatures, Display, "%ls will be set to Registered for cooking", *Plugin->GetName());
				InitialAutoState = EBuiltInAutoState::Registered;
			}
			const EGameFeaturePluginState DestinationState = ConvertInitialFeatureStateToTargetState(InitialAutoState);

			SetWasLoadedAsBuiltIn(StateMachine);

			// If we're already at the destination or beyond, don't transition back
			FGameFeaturePluginStateRange Destination(DestinationState, EGameFeaturePluginState::Active);
			ChangeGameFeatureDestination(StateMachine, ProtocolOptions, Destination,
				FGameFeaturePluginChangeStateComplete::CreateWeakLambda(this, [this, StateMachine, Destination, CompleteDelegate](const UE::GameFeatures::FResult& Result)
				{
					LoadBuiltInGameFeaturePluginComplete(Result, StateMachine, Destination);
					CompleteDelegate.ExecuteIfBound(Result);
				}));
		}
	}
}

#if UE_BUILD_SHIPPING
class FBuiltInPluginLoadTimeTracker {};
class FBuiltInPluginLoadTimeTrackerScope
{
public:
	FORCEINLINE FBuiltInPluginLoadTimeTrackerScope(FBuiltInPluginLoadTimeTracker& InTracker, const TSharedRef<IPlugin>& Plugin) {};
};
#else // !UE_BUILD_SHIPPING
class FBuiltInPluginLoadTimeTracker
{
public:
	FBuiltInPluginLoadTimeTracker()
	{
		StartTime = FPlatformTime::Seconds();
	}

	~FBuiltInPluginLoadTimeTracker()
	{
		double TotalLoadTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogGameFeatures, Display, "Total built in plugin load time %.4fs", TotalLoadTime);

		if (PluginLoadTimes.Num() > 0)
		{
			UE_LOGF(LogGameFeatures, Display, "There were %d built in plugins that took longer than %.4fs to load. Listing worst offenders.", PluginLoadTimes.Num(), UE::GameFeatures::GBuiltInPluginLoadTimeReportThreshold);
			const int32 NumToReport = FMath::Min(PluginLoadTimes.Num(), UE::GameFeatures::GBuiltInPluginLoadTimeMaxReportCount);
			Algo::Sort(PluginLoadTimes, [](const TPair<FString, double>& A, TPair<FString, double>& B) { return A.Value > B.Value; });
			for (int32 PluginIdx = 0; PluginIdx < NumToReport; ++PluginIdx)
			{
				const TPair<FString, double>& Plugin = PluginLoadTimes[PluginIdx];
				double LoadTime = Plugin.Value;
				if (LoadTime >= UE::GameFeatures::GBuiltInPluginLoadTimeErrorThreshold)
				{
					UE_LOGF(LogGameFeatures, Warning, "%ls took %.4f seconds to load. Something was done to significantly increase the load time of this plugin and it is now well outside what is acceptable. Reduce the load time to much less than %.4f seconds. Ideally, reduce the load time to less than %.4f seconds.",
						*Plugin.Key, Plugin.Value, UE::GameFeatures::GBuiltInPluginLoadTimeErrorThreshold, UE::GameFeatures::GBuiltInPluginLoadTimeReportThreshold);
				}
				else
				{
					UE_LOGF(LogGameFeatures, Display, "  %.4fs\t%ls", Plugin.Value, *Plugin.Key);
				}
			}
		}
	}

	void ReportPlugin(const FString& PluginName, double LoadTime)
	{
		PluginLoadTimes.Emplace(PluginName, LoadTime);
	}

private:
	TArray<TPair<FString, double>> PluginLoadTimes;
	double StartTime;
};

class FBuiltInPluginLoadTimeTrackerScope
{
public:
	FBuiltInPluginLoadTimeTrackerScope(FBuiltInPluginLoadTimeTracker& InTracker, const TSharedRef<IPlugin>& Plugin)
		: Tracker(InTracker), PluginName(Plugin->GetName()), StartTime(FPlatformTime::Seconds())
	{}

	~FBuiltInPluginLoadTimeTrackerScope()
	{
		double LoadTime = FPlatformTime::Seconds() - StartTime;
		if (LoadTime >= UE::GameFeatures::GBuiltInPluginLoadTimeReportThreshold)
		{
			Tracker.ReportPlugin(PluginName, LoadTime);
		}
	}

private:
	FBuiltInPluginLoadTimeTracker& Tracker;
	FString PluginName;
	double StartTime;
};

#endif // !UE_BUILD_SHIPPING

TArray<TSharedRef<IPlugin>> UGameFeaturesSubsystem::GetInternalEnabledPlugins() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UGameFeaturesSubsystem::GetInternalEnabledPlugins");
	TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPlugins();

	if (UE::GameFeatures::bTrimNonStartupEnabledPlugins)
	{
		const TSet<FString>& CompiledInPlugins = IPluginManager::Get().GetPluginsEnabledAtStartup();
		EnabledPlugins.RemoveAllSwap([&CompiledInPlugins](const TSharedRef<IPlugin>& Plugin)
			{
				const bool bRemove = !CompiledInPlugins.Contains(Plugin->GetName());
				return bRemove;
			});
	}

	return EnabledPlugins;
}

void UGameFeaturesSubsystem::LoadBuiltInGameFeaturePlugins(FBuiltInPluginAdditionalFilters AdditionalFilter, const FBuiltInGameFeaturePluginsLoaded& InCompleteDelegate /*= FBuiltInGameFeaturePluginsLoaded()*/)
{
	FBuiltInPluginAdditionalFilters_Copyable NullCopyable;
	LoadBuiltInGameFeaturePluginsInternal(GetInternalEnabledPlugins(), AdditionalFilter, NullCopyable, 0, InCompleteDelegate);
}

void UGameFeaturesSubsystem::LoadBuiltInGameFeaturePlugins_Amortized(const FBuiltInPluginAdditionalFilters_Copyable& AdditionalFilter_Copyable, int32 AmortizeRate, const FBuiltInGameFeaturePluginsLoaded& InCompleteDelegate /*= FBuiltInGameFeaturePluginsLoaded()*/)
{
	LoadBuiltInGameFeaturePluginsInternal(GetInternalEnabledPlugins(), AdditionalFilter_Copyable, AdditionalFilter_Copyable, AmortizeRate, InCompleteDelegate);
}

void UGameFeaturesSubsystem::LoadBuiltInGameFeaturePlugins_Amortized(const TArray<TSharedRef<IPlugin>>& Plugins, const FBuiltInPluginAdditionalFilters_Copyable& AdditionalFilter_Copyable, int32 AmortizeRate, const FBuiltInGameFeaturePluginsLoaded& InCompleteDelegate)
{
	LoadBuiltInGameFeaturePluginsInternal(Plugins, AdditionalFilter_Copyable, AdditionalFilter_Copyable, AmortizeRate, InCompleteDelegate);
}

void UGameFeaturesSubsystem::LoadBuiltInGameFeaturePluginsInternal(TArray<TSharedRef<IPlugin>> Plugins, FBuiltInPluginAdditionalFilters AdditionalFilter, const FBuiltInPluginAdditionalFilters_Copyable& AdditionalFilter_Copyable, int32 AmortizeRate, const FBuiltInGameFeaturePluginsLoaded& InCompleteDelegate /*= FBuiltInGameFeaturePluginsLoaded()*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UGameFeaturesSubsystem::LoadBuiltInGameFeaturePlugins");

	if (UE::GameFeatures::bSkipBuiltInGameFeatures)
	{
		UE_LOGF(LogGameFeatures, Log, "Skipping built-in GFP loading (GameFeaturePlugin.SkipBuiltInGameFeatures)");
		InCompleteDelegate.ExecuteIfBound(TMap<FString, UE::GameFeatures::FResult>());
		return;
	}

	TSharedRef<FAmmortizedLoadContext> LoadContext = MakeShared<FAmmortizedLoadContext>();
	FGuid ReferenceGuid = FGuid::NewGuid();
	LoadContext->GUID = ReferenceGuid;
	ActiveAmmoritzedLoads.Add(ReferenceGuid, LoadContext);

	LoadContext->CompleteDelegate = InCompleteDelegate;

	FBuiltInPluginLoadTimeTracker PluginLoadTimeTracker;

	LoadContext->Results.Reserve(Plugins.Num());
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		LoadContext->Results.Add(Plugin->GetName(), MakeError("Pending"));
	}

	const int32 NumPluginsToLoad = Plugins.Num();
	UE_LOGFMT(LogGameFeatures, Log, "Loading {Num} BuiltIn GameFeaturePlugins. {Amortized}. {Plugins}.",
		NumPluginsToLoad,
		AmortizeRate > 0 ? TEXT("Amortized") : TEXT("Not Amortized"),
		[&Plugins]()
		{
			TArray<FString> Names;
			for (const TSharedRef<IPlugin>& Plugin : Plugins)
			{
				Names.Add(Plugin->GetName());
			}
			return Names;
		}());

	// Sort the plugins so we can more accurately track how long it takes to load rather than have inconsistent dependency timings.
	TArray<TSharedRef<IPlugin>> Dependencies;
	auto GetPluginDependencies =
		[&Dependencies](TSharedRef<IPlugin> CurrentPlugin)
	{
		IPluginManager& PluginManager = IPluginManager::Get();
		Dependencies.Reset();

		const FPluginDescriptor& Desc = CurrentPlugin->GetDescriptor();
		for (const FPluginReferenceDescriptor& Dependency : Desc.Plugins)
		{
			if (Dependency.bEnabled)
			{
				if (TSharedPtr<IPlugin> FoundPlugin = PluginManager.FindEnabledPlugin(Dependency.Name))
				{
					Dependencies.Add(FoundPlugin.ToSharedRef());
				}
			}
		}
		return Dependencies;
	};

	Algo::TopologicalSort(Plugins, GetPluginDependencies);

	auto ProcessGameFeaturePlugin = [this](const TSharedRef<IPlugin>& PluginToProcess, const TSharedRef<FAmmortizedLoadContext>& LoadContextToProcess, const FBuiltInPluginAdditionalFilters& AdditionalFilterForProcess)
	{
		LoadBuiltInGameFeaturePlugin(PluginToProcess, AdditionalFilterForProcess, FGameFeaturePluginLoadComplete::CreateLambda([LoadContextToProcess, PluginToProcess](const UE::GameFeatures::FResult& Result)
		{
			LoadContextToProcess->Results.Add(PluginToProcess->GetName(), Result);
			++LoadContextToProcess->NumPluginsLoaded;
			UE_LOGF(LogGameFeatures, VeryVerbose, "Finished Loading %i builtins", LoadContextToProcess->NumPluginsLoaded);
		}));
	};

	if (AmortizeRate > 0 && NumPluginsToLoad > 0)
	{
		LoadContext->PluginAmortizeRate = AmortizeRate;
		LoadContext->AmortizedPluginsRemaining = Plugins;
		// Note that we are using AdditionalFilter_Copyable here. We need to make a copy of thus function so that the captured properties in lambdas are not destroyed
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [this, LoadContext, AdditionalFilter_Copyable, ProcessGameFeaturePlugin](float)
		{
			if (GameSpecificPolicies && !LoadContext->bCancelled)
			{
				// This PluginAmortizeRate use to be how many GFPs per tick, but we have moved this rate to mean milliseconds now
				double MaxMillisecondsPerFrame = LoadContext->PluginAmortizeRate / 1000.0;
				double StartTime = FPlatformTime::Seconds();
				int32 NumPluginsThisFrame = 0;
				int32 NumPluginsLeft = LoadContext->AmortizedPluginsRemaining.Num();

				do
				{
					const TSharedRef<IPlugin>& Plugin = LoadContext->AmortizedPluginsRemaining[NumPluginsThisFrame++];
					ProcessGameFeaturePlugin(Plugin, LoadContext, AdditionalFilter_Copyable);
				} while (FPlatformTime::Seconds() - StartTime < MaxMillisecondsPerFrame && NumPluginsThisFrame < NumPluginsLeft);

				LoadContext->AmortizedPluginsRemaining.RemoveAt(0, NumPluginsThisFrame);
			}
			return GameSpecificPolicies && LoadContext->AmortizedPluginsRemaining.Num() > 0 && !LoadContext->bCancelled;
		}));
	}
	else
	{
		// Conditionally defer execution if the new executor is being used and deferred execution is enabled 
		UE::GameFeatures::FScopedPauseStateMachineExecution PauseExecution(UE::GameFeatures::bLoadBuiltin_DeferExecution ? StateMachineExecutor.Get() : nullptr);
		for (const TSharedRef<IPlugin>& Plugin : Plugins)
		{
			FBuiltInPluginLoadTimeTrackerScope TrackerScope(PluginLoadTimeTracker, Plugin);
			ProcessGameFeaturePlugin(Plugin, LoadContext, AdditionalFilter);
		}
	}
}

bool UGameFeaturesSubsystem::GetPluginURLByName(FStringView PluginName, FString& OutPluginURL) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE::TReadScopeLock ReadLock(GameFeaturePluginNameToPathMapLock);

	if (const FPluginString* PluginURL = GameFeaturePluginNameToPathMap.FindByHash(GetTypeHash(PluginName), TO_PLUGIN_STRING(PluginName)))	//TODO: GetTypeHash for utf8 strings only that doesn't match Tchar strings
	{
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OutPluginURL = FString(*PluginURL);
		return true;
	}

	return false;
}

bool UGameFeaturesSubsystem::GetPluginURLByName(FUtf8StringView PluginName, FString& OutPluginURL) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE::TReadScopeLock ReadLock(GameFeaturePluginNameToPathMapLock);
	if (const FPluginString* PluginURL = GameFeaturePluginNameToPathMap.FindByHash(GetTypeHash(PluginName), FPluginString(PluginName)))
	{
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OutPluginURL = FString(*PluginURL);
		return true;
	}

	return false;
}

FString UGameFeaturesSubsystem::GetPluginFilenameFromPluginURL(const FString& PluginURL) const
{
	FString PluginFilename;
	const UGameFeaturePluginStateMachine* GFSM = FindGameFeaturePluginStateMachine(PluginURL);
	if (GFSM == nullptr || !GetPluginFilename(GFSM, PluginFilename))
	{
		UE_LOGF(LogGameFeatures, Error, "UGameFeaturesSubsystem could not get the plugin path from the plugin URL. URL:%ls ", *PluginURL);
	}
	return PluginFilename;
}

FString UGameFeaturesSubsystem::GetPluginNameFromPluginURL(const FString& PluginURL) const
{
	const UGameFeaturePluginStateMachine* GFSM = FindGameFeaturePluginStateMachine(PluginURL);
	if (GFSM == nullptr)
	{
		UE_LOGF(LogGameFeatures, Error, "UGameFeaturesSubsystem could not get the plugin name from the plugin URL. URL:%ls ", *PluginURL);
		return FString();
	}
	return GetPluginName(GFSM);
}

void UGameFeaturesSubsystem::FixPluginPackagePath(FString& PathToFix, const FString& PluginRootPath, bool bMakeRelativeToPluginRoot)
{
	if (bMakeRelativeToPluginRoot)
	{
		// This only modifies paths starting with the root
		PathToFix.RemoveFromStart(PluginRootPath);
	}
	else
	{
		if (!FPackageName::IsValidLongPackageName(PathToFix))
		{
			PathToFix = PluginRootPath / PathToFix;
		}
	}
}

void UGameFeaturesSubsystem::GetLoadedGameFeaturePluginFilenamesForCooking(TArray<FString>& OutLoadedPluginFilenames) const
{
	for (auto StateMachineIt = GameFeaturePluginStateMachines.CreateConstIterator(); StateMachineIt; ++StateMachineIt)
	{
		UGameFeaturePluginStateMachine* GFSM = StateMachineIt.Value();
		if (GFSM && GetCurrentState(GFSM) > EGameFeaturePluginState::Installed)
		{
			FString PluginFilename;
			if (GetPluginFilename(GFSM, PluginFilename))
			{
				OutLoadedPluginFilenames.Add(PluginFilename);
			}
		}
	}
}

EGameFeaturePluginState UGameFeaturesSubsystem::GetPluginState(const FString& PluginURL) const
{
	FGameFeaturePluginIdentifier PluginIdentifier(PluginURL);
	return GetPluginState(PluginIdentifier);
}

EGameFeaturePluginState UGameFeaturesSubsystem::GetPluginState(FGameFeaturePluginIdentifier PluginIdentifier) const
{
	if (UGameFeaturePluginStateMachine* StateMachine = FindGameFeaturePluginStateMachine(PluginIdentifier))
	{
		return GetCurrentState(StateMachine);
	}
	else
	{
		return EGameFeaturePluginState::UnknownStatus;
	}
}

bool UGameFeaturesSubsystem::GetBuiltInGameFeaturePluginDetails(const TSharedRef<IPlugin>& Plugin, struct FGameFeaturePluginDetails& OutPluginDetails) const
{
	TSharedPtr<const FGameFeaturePluginDetails> Ptr = GetBuiltInGameFeaturePluginDetails(Plugin);
	if (Ptr)
	{
		OutPluginDetails = *Ptr;
		return true;
	}
	return false;
}

TSharedPtr<const FGameFeaturePluginDetails> UGameFeaturesSubsystem::GetBuiltInGameFeaturePluginDetails(const TSharedRef<IPlugin>& Plugin) const
{
	const FString& PluginDescriptorFilename = Plugin->GetDescriptorFileName();
	// Make sure you are in a game feature plugins folder. All GameFeaturePlugins are rooted in a GameFeatures folder.
	if (!PluginDescriptorFilename.IsEmpty() && GetDefault<UGameFeaturesSubsystemSettings>()->IsValidGameFeaturePlugin(PluginDescriptorFilename))
	{
		return GetGameFeaturePluginDetailsInternal(PluginDescriptorFilename);
	}

	return {};
}


bool UGameFeaturesSubsystem::GetBuiltInGameFeaturePluginURL(const TSharedRef<IPlugin>& Plugin, FString& OutPluginURL) const
{
	// @TODO: this problematic because it assumes file protocol.
	// Ideally this would work with any protocol, but for current uses cases the exact protocol doesn't seem to matter.

	const FString& PluginDescriptorFilename = Plugin->GetDescriptorFileName();
	// Make sure you are in a game feature plugins folder. All GameFeaturePlugins are rooted in a GameFeatures folder.
	if (!PluginDescriptorFilename.IsEmpty() && GetDefault<UGameFeaturesSubsystemSettings>()->IsValidGameFeaturePlugin(FPaths::ConvertRelativePathToFull(PluginDescriptorFilename)))
	{
		const FString& PluginName = Plugin->GetName();
		bool bFoundPluginURL = GetPluginURLByName(PluginName, OutPluginURL);
		if (!bFoundPluginURL)
		{
			bFoundPluginURL = GameSpecificPolicies->GetGameFeaturePluginURL(Plugin, OutPluginURL);
		}
		return bFoundPluginURL;
	}

	return false;
}

bool UGameFeaturesSubsystem::GetGameFeaturePluginDetails(const FString& PluginURL, FGameFeaturePluginDetails& OutPluginDetails) const
{
	TSharedPtr<const FGameFeaturePluginDetails> PluginDetails = GetGameFeaturePluginDetails(PluginURL);
	if (PluginDetails)
	{
		OutPluginDetails = *PluginDetails;
		return true;
	}

	return false;
}

TSharedPtr<const FGameFeaturePluginDetails> UGameFeaturesSubsystem::GetGameFeaturePluginDetails(const FString& PluginURL) const
{
	FStringView PluginPath;
	if (UGameFeaturesSubsystem::ParsePluginURL(PluginURL, nullptr, &PluginPath))
	{
		return GetGameFeaturePluginDetailsInternal(FString(PluginPath));
	}

	return {};
}

bool UGameFeaturesSubsystem::GetGameFeatureControlsUPlugin(const FString& PluginURL, bool& OutGameFeatureControlsUPlugin) const
{
	if (UGameFeaturePluginStateMachine* Machine = FindGameFeaturePluginStateMachine(PluginURL))
	{
		OutGameFeatureControlsUPlugin = GetProperties(Machine).bAddedPluginToManager;
		return true;
	}

	return false;
}

UE::Tasks::TTask<TSharedPtr<const FGameFeaturePluginDetails>> UGameFeaturesSubsystem::LoadGameFeaturePluginDetailsAsync(const FString& PluginDescriptorFilename) const
{
	using namespace UE::Tasks;

	if (PluginDescriptorFilename.IsEmpty()) 
	{
		return MakeCompletedTask<TSharedPtr<const FGameFeaturePluginDetails>>();
	}

	if (!GameSpecificPolicies->ShouldReadPluginDetails(PluginDescriptorFilename))
	{
		return MakeCompletedTask<TSharedPtr<const FGameFeaturePluginDetails>>();
	}

	{
		UE::TReadScopeLock ReadLock(CachedGameFeaturePluginDetailsLock);
		if (TSharedRef<const FGameFeaturePluginDetails>* PluginDetails = CachedPluginDetailsByFilename.Find(PluginDescriptorFilename))
		{
			return MakeCompletedTask<TSharedPtr<const FGameFeaturePluginDetails>>(*PluginDetails);
		}
	}

	// Under some conditions we must fetch the plugin details text or json object on the main thread
	if (WITH_EDITOR || FPluginDescriptor::CustomPluginDescriptorReaderDelegate.IsBound())
	{
		// There is existing code which calls this in ParallelFor
		// check (IsInGameThread());

		auto CompleteAndCache = [this, PluginDescriptorFilename](TSharedPtr<const FGameFeaturePluginDetails> Result)
		{
			if (Result)
			{ 
				UE::TWriteScopeLock Lock(CachedGameFeaturePluginDetailsLock);
				CachedPluginDetailsByFilename.Add(PluginDescriptorFilename, Result.ToSharedRef());
			}
			return MakeCompletedTask<TSharedPtr<const FGameFeaturePluginDetails>>(Result);
		};

#if WITH_EDITOR
		// if we failed to find the plugin and add it to the plugin manager, we may leak this when closing a project
		// so make sure to clean up the plugin when leaving this function
		// In the editor we already have the plugin JSON cached
		bool bPluginWasAdded = false;
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(FPathViews::GetBaseFilename(PluginDescriptorFilename));
		if (!Plugin.IsValid())
		{
			if (IPluginManager::Get().AddToPluginsList(PluginDescriptorFilename))
			{
				bPluginWasAdded = true;
				Plugin = IPluginManager::Get().FindPlugin(FPathViews::GetBaseFilename(PluginDescriptorFilename));
			}
		}

		if (Plugin)
		{
			if (TSharedPtr<FJsonObject> ObjectPtr = Plugin->GetDescriptorJson())
			{ 
				TSharedPtr<const FGameFeaturePluginDetails> OutPluginDetails = ParseGameFeaturePluginDetails(PluginDescriptorFilename, ObjectPtr.ToSharedRef());
				if (bPluginWasAdded)
				{
					verify(IPluginManager::Get().RemoveFromPluginsList(PluginDescriptorFilename));
				}
				return CompleteAndCache(OutPluginDetails);
			}
		}
#endif
	
		if (FPluginDescriptor::CustomPluginDescriptorReaderDelegate.IsBound())
		{
			bool TryWithNative = false;
			FText FailReason; 
			TSharedPtr<FJsonObject> ObjectPtr;
			bool Success = FPluginDescriptor::CustomPluginDescriptorReaderDelegate.Execute(*PluginDescriptorFilename, &FailReason, ObjectPtr, TryWithNative);
			if (!TryWithNative)
			{
				if (Success)
				{
					TSharedPtr<const FGameFeaturePluginDetails> OutPluginDetails = ParseGameFeaturePluginDetails(PluginDescriptorFilename, ObjectPtr.ToSharedRef());
					return CompleteAndCache(OutPluginDetails);
				}
				else
				{
					UE_LOGFMT(LogGameFeatures, Error, "Failed to read plugin descriptor {Filename}. Reason: {FailReason}", PluginDescriptorFilename, FailReason.ToString());
					return CompleteAndCache({});
				}
			}
		}

		// Read the file to a string
		FString FileContents;
		if (!FFileHelper::LoadFileToString(FileContents, *PluginDescriptorFilename))
		{
			UE_LOGF(LogGameFeatures, Error, "UGameFeaturesSubsystem could not load plugin descriptor. Failed to read file. File:%ls Error:%d", *PluginDescriptorFilename, FPlatformMisc::GetLastError());
			return CompleteAndCache({});
		}

		// Deserialize a JSON object from the string	
		TSharedPtr<const FGameFeaturePluginDetails> OutPluginDetails = ParseGameFeaturePluginDetails(PluginDescriptorFilename, FileContents);
		return CompleteAndCache(OutPluginDetails);
	}
	else
	{
		// If there are no special places to read the json from we just need to deal with loading the file async and parsing on completion
		TTask<TValueOrError<FString, FFileSystemError>> StringTask = FFileHelper::LoadFileToStringAsync(*PluginDescriptorFilename, {});
		return UE::Tasks::Launch(TEXT("ParsePluginDesc"), [StringTask, this, PluginDescriptorFilename]() mutable -> TSharedPtr<const FGameFeaturePluginDetails>
		{
			TValueOrError<FString, FFileSystemError>& Result = StringTask.GetResult();
			if (Result.HasValue())
			{ 
				TSharedPtr<const FGameFeaturePluginDetails> Details = ParseGameFeaturePluginDetails(PluginDescriptorFilename, Result.GetValue());
				if (Details)
				{
					UE::TWriteScopeLock Lock(CachedGameFeaturePluginDetailsLock);
					CachedPluginDetailsByFilename.Add(PluginDescriptorFilename, Details.ToSharedRef());
				}
				return Details; 
			}
			return {}; 
		}, UE::Tasks::Prerequisites(StringTask));
	}
}

TSharedPtr<const FGameFeaturePluginDetails> UGameFeaturesSubsystem::GetGameFeaturePluginDetailsInternal(const FString& PluginDescriptorFilename) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GFP_GetPluginDetails);
	UE::Tasks::TTask<TSharedPtr<const FGameFeaturePluginDetails>> Task = LoadGameFeaturePluginDetailsAsync(PluginDescriptorFilename);
	Task.Wait();
	return Task.GetResult();
}

/** Parse FGameFeaturePluginDetails from a string */
TSharedPtr<const FGameFeaturePluginDetails> UGameFeaturesSubsystem::ParseGameFeaturePluginDetails(const FString& ErrorContext, TArrayView<TCHAR> JsonTxt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UGameFeaturesSubsystem::ParseGameFeaturePluginDetails");
	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> Result = UE::Json::ParseInPlace(JsonTxt);
	if (Result.HasValue())
	{
		if (TOptional<UE::Json::FConstObject> Root = UE::Json::GetRootObject(Result.GetValue()); Root.IsSet())
		{
			return ParseGameFeaturePluginDetails(ErrorContext, Root.GetValue());
		}
	}

	if (Result.HasError())
	{
		UE_LOGFMT(LogGameFeatures, Error, "Failed to parse plugin descriptor from {ErrorContext}. Error: {Error}", ErrorContext, Result.GetError().CreateMessage(FStringView(JsonTxt)));
	}
	else
	{
		UE_LOGFMT(LogGameFeatures, Error, "Failed to parse plugin descriptor from {ErrorContext}. No root object element?", ErrorContext);
	}
	return {};
}

/** Parse FGameFeaturePluginDetails from a RapidJSON object */
TSharedPtr<const FGameFeaturePluginDetails> UGameFeaturesSubsystem::ParseGameFeaturePluginDetails(const FString& ErrorContext, UE::Json::FConstObject InObject) const
{
	FGameFeaturePluginDetails PluginDetails;
	// Read the properties
	// Hotfixable. If it is not specified, then we assume it is
	PluginDetails.bHotfixable = UE::Json::GetBoolField(InObject, TEXT("Hotfixable")).Get(true);

	// Determine the initial plugin state
	PluginDetails.BuiltInAutoState = DetermineBuiltInInitialFeatureState(InObject, ErrorContext);

	// Read any additional metadata the policy might want to consume (e.g., a release version number)
	for (const FString& ExtraKey : GetDefault<UGameFeaturesSubsystemSettings>()->AdditionalPluginMetadataKeys)
	{
		UE::Json::FValue::ConstMemberIterator Field = InObject.FindMember(*ExtraKey);
		if (Field != InObject.MemberEnd())
		{
			PluginDetails.AdditionalMetadata.Add(TO_PLUGIN_STRING(ExtraKey), UE::Json::ConvertRapidJsonToSharedJsonValue(Field->value));
		}
		else
		{
			PluginDetails.AdditionalMetadata.Add(TO_PLUGIN_STRING(ExtraKey), MakeShared<FJsonValueString>(TEXT("")));
		}
	}

	// Parse plugin dependencies
	if (TOptional<UE::Json::FConstArray> PluginsArray = UE::Json::GetArrayField(InObject, TEXT("Plugins")); PluginsArray.IsSet())
	{
		constexpr const TCHAR* NameField = TEXT("Name");
		constexpr const TCHAR* EnabledField = TEXT("Enabled");
		constexpr const TCHAR* ActivateField = TEXT("Activate");
		for (const UE::Json::FValue& PluginElement : *PluginsArray)
		{
			if (!PluginElement.IsObject())
			{
				continue;
			}
			UE::Json::FConstObject ElementObject = PluginElement.GetObject();

			FStringView DependencyName = UE::Json::GetStringField(ElementObject, NameField).Get({});
			if (DependencyName.IsEmpty())
			{
				UE_LOGFMT(LogGameFeatures, Error, "Error parsing dependency name in {PluginDescriptorFilename}! Invalid JSON data!", ErrorContext);
				continue;
			}

			const bool bElementEnabled = UE::Json::GetBoolField(ElementObject, EnabledField).Get(false);
			if (!bElementEnabled)
			{
				UE_LOGFMT(LogGameFeatures, VeryVerbose, "Skipping adding dependency {DependencyName} in {PluginDescriptorFileName}. Plugin is disabled.", 
					DependencyName, ErrorContext);
				continue;
			}

			//Have to get Activate from JSON as it's unique to GFP and not in the PluginManager
			bool bElementActivate = UE::Json::GetBoolField(ElementObject, ActivateField).Get(false);

			FGameFeaturePluginReferenceDetails& RefDetails = PluginDetails.PluginDependencies.Emplace_GetRef();
			RefDetails.PluginName = TO_PLUGIN_STRING(DependencyName);
			RefDetails.bShouldActivate = bElementActivate;
		}
	}

	return MakeShared<FGameFeaturePluginDetails>(MoveTemp(PluginDetails));
}

/** Parse FGameFeaturePluginDetails from a JSON object */
TSharedPtr<const FGameFeaturePluginDetails> UGameFeaturesSubsystem::ParseGameFeaturePluginDetails(const FString& ErrorContext, TSharedRef<FJsonObject> ObjectPtr) const
{
	FGameFeaturePluginDetails PluginDetails;
	// Read the properties
	// Hotfixable. If it is not specified, then we assume it is
	PluginDetails.bHotfixable = true;
	ObjectPtr->TryGetBoolField(TEXTVIEW("Hotfixable"), PluginDetails.bHotfixable);

	// Determine the initial plugin state
	PluginDetails.BuiltInAutoState = DetermineBuiltInInitialFeatureState(ObjectPtr, ErrorContext);

	// Read any additional metadata the policy might want to consume (e.g., a release version number)
	for (const FString& ExtraKey : GetDefault<UGameFeaturesSubsystemSettings>()->AdditionalPluginMetadataKeys)
	{
		TSharedPtr<FJsonValue> Field = ObjectPtr->TryGetField(ExtraKey);
		if (Field.IsValid())
		{
			PluginDetails.AdditionalMetadata.Add(TO_PLUGIN_STRING(ExtraKey), Field);
		}
		else
		{
			PluginDetails.AdditionalMetadata.Add(TO_PLUGIN_STRING(ExtraKey), MakeShared<FJsonValueString>(TEXT("")));
		}
	}

	// Parse plugin dependencies
	const TArray<TSharedPtr<FJsonValue>>* PluginsArray = nullptr;
	ObjectPtr->TryGetArrayField(TEXTVIEW("Plugins"), PluginsArray);
	if (PluginsArray)
	{
		const FStringView NameField = TEXTVIEW("Name");
		const FStringView EnabledField = TEXTVIEW("Enabled");
		const FStringView ActivateField = TEXTVIEW("Activate");
		for (const TSharedPtr<FJsonValue>& PluginElement : *PluginsArray)
		{
			if (!PluginElement)
			{
				continue;
			}

			const TSharedPtr<FJsonObject>* ElementObjectPtr = nullptr;
			PluginElement->TryGetObject(ElementObjectPtr);
			if (!ElementObjectPtr || !ElementObjectPtr->IsValid())
			{
				continue;
			}
			const TSharedPtr<FJsonObject>& ElementObject = *ElementObjectPtr;

			FString DependencyName;
			ElementObject->TryGetStringField(NameField, DependencyName);
			if (DependencyName.IsEmpty())
			{
				UE_LOGFMT(LogGameFeatures, Error, "Error parsing dependency name in {ErrorContext}! Invalid JSON data!", ErrorContext);
				continue;
			}

			bool bElementEnabled = false;
			ElementObject->TryGetBoolField(EnabledField, bElementEnabled);
			if (!bElementEnabled)
			{
				UE_LOGFMT(LogGameFeatures, VeryVerbose, "Skipping adding dependency {DependencyName} in {ErrorContext}. Plugin is disabled.", DependencyName, ErrorContext);
				continue;
			}

			//Have to get Activate from JSON as it's unique to GFP and not in the PluginManager
			bool bElementActivate = false;
			ElementObject->TryGetBoolField(ActivateField, bElementActivate);

			FGameFeaturePluginReferenceDetails& RefDetails = PluginDetails.PluginDependencies.Emplace_GetRef();
			RefDetails.PluginName = TO_PLUGIN_STRING(DependencyName);
			RefDetails.bShouldActivate = bElementActivate;
		}
	}

	return MakeShared<FGameFeaturePluginDetails>(MoveTemp(PluginDetails));
}

void UGameFeaturesSubsystem::PruneCachedGameFeaturePluginDetails(const FString& PluginURL, const FString& PluginDescriptorFilename) const
{
	UE::TWriteScopeLock WriteLock(CachedGameFeaturePluginDetailsLock);
	CachedPluginDetailsByFilename.Remove(PluginDescriptorFilename);
}

struct FGameFeaturePluginPredownloadContext : public FGameFeaturePluginPredownloadHandle
{
	struct FGFPData
	{
		FInstallBundlePluginProtocolMetaData ProtocolMetadata;
		TArray<EStreamingAssetInstallMode> BundleInstallModes;
	};

	static constexpr const FStringView PredownloadErrorNamespace = TEXTVIEW("GameFeaturePlugin.Predownload.");

	UE::GameFeatures::FResult Result = MakeValue();

	FTSTicker::FDelegateHandle TickHandle;

	TMap<FGameFeaturePluginIdentifier, FGFPData> GFPs;

	TArray<FName> PendingBundleDownloads;
	TArray<FName> PendingBundleReleases;

	FInstallBundleCombinedProgressTracker ProgressTracker{ false /*bAutoTick*/};

	TUniqueFunction<void(const UE::GameFeatures::FResult&)> OnComplete;
	TUniqueFunction<void(float)> OnProgress;

	TArray<UE::IoStore::FOnDemandContentHandle> IoStoreContentHandles;
	TArray<UE::IoStore::FOnDemandInstallRequest> IoStoreInstallRequests;
	TArray<UE::IoStore::FOnDemandInstallProgress> IoStoreProgress;
	TOptional<UE::UnifiedError::FError> IoStoreError;
	int32 IoStorePendingInstalls = 0;

	float Progress = 0.0f;

	UE::FMutex IoStoreMutex;

	bool bHasAssetDependencies = false;
	bool bWaitForBundlesToRelease = false;
	bool bIsComplete = false;
	bool bCanceled = false;

	virtual ~FGameFeaturePluginPredownloadContext() override
	{
		Cleanup();
	}

	virtual bool IsComplete() const override
	{
		return bIsComplete;
	}

	virtual const UE::GameFeatures::FResult& GetResult() const override
	{
		return Result;
	}

	virtual float GetProgress() const override
	{
		return Progress;
	}

	virtual void Cancel() override
	{
		bCanceled = true;

		if (PendingBundleDownloads.Num() > 0)
		{
			TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
			if (BundleManager)
			{
				BundleManager->CancelUpdateContent(PendingBundleDownloads);
			}
		}

		UE::TUniqueLock Lock(IoStoreMutex);
		CancelIoStoreRequests();
	}

	bool Tick(float /*dt*/)
	{
		UpdateProgress();

		return true;
	}

	void Cleanup()
	{
		if (TickHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickHandle);
		}		
		IInstallBundleManager::InstallBundleCompleteDelegate.RemoveAll(this);
		IInstallBundleManager::PausedBundleDelegate.RemoveAll(this);
		if (bWaitForBundlesToRelease && PendingBundleReleases.Num() == 0)
		{
			// If we have pending releases, this will be cleaned up once the last one is released
			IInstallBundleManager::ReleasedDelegate.RemoveAll(this);
		}
	}

	void FinishComplete()
	{
		bIsComplete = true;
		if (OnComplete)
		{
			OnComplete(Result);
		}
	}

	void SetComplete()
	{
		ReleaseBundlesIfPossible();

		if (PendingBundleReleases.Num() == 0)
		{
			FinishComplete();
		}
	}

	void SetComplete(UE::GameFeatures::FResult&& InResult)
	{
		ReleaseBundlesIfPossible();

		Result = MoveTemp(InResult);
		if (PendingBundleReleases.Num() == 0)
		{
			FinishComplete();
		}
	}

	void SetCompleteCanceled()
	{
		ReleaseBundlesIfPossible();

		Result = MakeError(FString::Printf(TEXT("%.*s%s"),
			PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
			TEXT("Canceled")));
		if (PendingBundleReleases.Num() == 0)
		{
			FinishComplete();
		}
	}

	void Start(TConstArrayView<FString> PluginURLs)
	{
		if (bCanceled)
		{
			SetCompleteCanceled();
			return;
		}

		for (const FString& URL : PluginURLs)
		{
			if (UGameFeaturesSubsystem::GetPluginURLProtocol(URL) != EGameFeaturePluginProtocol::InstallBundle)
			{
				// Only support install bundle protocol for downloading right now
				continue;
			}

			TValueOrError<FInstallBundlePluginProtocolMetaData, FString> MaybeInstallBundleOptions = FInstallBundlePluginProtocolMetaData::FromString(URL);
			if (MaybeInstallBundleOptions.HasError())
			{
				UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed to parse URL {URL}", ("URL", URL));
				UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
					PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
					*MaybeInstallBundleOptions.GetError()));
				SetComplete(MoveTemp(ErrorResult));
				return;
			}

			GFPs.Emplace(URL, MaybeInstallBundleOptions.StealValue());
		}

		if (GFPs.Num() == 0)
		{
			SetComplete(MakeValue());
			return;
		}

		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
		if (!BundleManager)
		{
			UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed, no Install Bundle Manager found.");
			UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
				PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
				TEXT("BundleManager_Null")));
			SetComplete(MoveTemp(ErrorResult));
			return;
		}

		UGameFeaturesSubsystem& GFPSubSys = UGameFeaturesSubsystem::Get();

		bool bAllBundlesUpToDate = true; 
		TArray<FName> BundlesToInstall;
		for (TPair<FGameFeaturePluginIdentifier, FGFPData>& Pair : GFPs)
		{
			const FGameFeaturePluginIdentifier& GFPIdentifier = Pair.Key;
			FGFPData& GFPData = Pair.Value;

			UGameFeaturePluginStateMachine* Machine = GFPSubSys.FindGameFeaturePluginStateMachine(GFPIdentifier);
			if (Machine && GFPSubSys.GetDestination(Machine) < EGameFeaturePluginState::Installed)
			{
				// Existing machine exists and wants to be uninstalled, can't precache
				UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed because a GFP is unloading, GFP: {GFP}", ("GFP", GFPSubSys.GetPluginName(Machine)));
				UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
					PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
					TEXT("GFPUnloading")));
				SetComplete(MoveTemp(ErrorResult));
				return;
			}

			GFPSubSys.OnGameFeaturePredownloading(FString(GFPIdentifier.GetPluginName()), GFPIdentifier);

			const bool bAddDependencies = true;
			TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> MaybeInstallState = 
				BundleManager->GetInstallStateSynchronous(GFPData.ProtocolMetadata.InstallBundles, bAddDependencies);
			if (MaybeInstallState.HasError())
			{
				UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed, failed to get install state for {GFP}", ("GFP", GFPIdentifier.GetPluginName()));
				UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
					PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
					LexToString(MaybeInstallState.GetError())));
				SetComplete(MoveTemp(ErrorResult));
				return;
			}

			const FInstallBundleCombinedInstallState& InstallState = MaybeInstallState.GetValue();
			const bool bIsAvailable = Algo::AllOf(GFPData.ProtocolMetadata.InstallBundles,
				[&InstallState](FName BundleName) { return InstallState.IndividualBundleStates.Contains(BundleName); });

			if (!bIsAvailable)
			{
				UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed, unvailable {GFP}", ("GFP", GFPIdentifier.GetPluginName()));
				UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
					PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
					TEXT("Plugin_Unavailable")));
				SetComplete(MoveTemp(ErrorResult));
				return;
			}

			GFPData.ProtocolMetadata.InstallBundlesWithAssetDependencies = InstallState.BundlesWithIoStoreOnDemand.Array();
			if (!GFPData.ProtocolMetadata.InstallBundlesWithAssetDependencies.IsEmpty())
			{
				TValueOrError<TArray<EStreamingAssetInstallMode>, FString> MaybeInstallModes =
					UGameFeaturesSubsystem::Get().GetPolicy().GetStreamingAssetInstallModes(
						GFPIdentifier.GetFullPluginURL(), GFPData.ProtocolMetadata.InstallBundlesWithAssetDependencies);

				if (MaybeInstallModes.HasError())
				{
					UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed, no install modes for {GFP}", ("GFP", GFPIdentifier.GetPluginName()));
					UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
						PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
						*MaybeInstallModes.GetError()));
					SetComplete(MoveTemp(ErrorResult));
					return;
				}

				GFPData.BundleInstallModes = MaybeInstallModes.StealValue();
				check(GFPData.BundleInstallModes.Num() == GFPData.ProtocolMetadata.InstallBundlesWithAssetDependencies.Num());
			}

			bHasAssetDependencies = bHasAssetDependencies || !InstallState.BundlesWithIoStoreOnDemand.IsEmpty();

			bAllBundlesUpToDate = bAllBundlesUpToDate && 
				InstallState.BundlesWithIoStoreOnDemand.IsEmpty() && // For now, just assume that any IAD data is not up to date
				InstallState.GetAllBundlesHaveState(EInstallBundleInstallState::UpToDate);

			// Update metadata with fully expanded dependency list. This can only be done after all bundles are known to be available,
			// otherwise unavailable bundles in the URL could be stripped from the list.
			GFPData.ProtocolMetadata.InstallBundles.Empty(InstallState.IndividualBundleStates.Num());
			InstallState.IndividualBundleStates.GetKeys(GFPData.ProtocolMetadata.InstallBundles);

			// Its ok to have duplicates in this list
			BundlesToInstall.Append(GFPData.ProtocolMetadata.InstallBundles);
		}

		// Early out if everything is up to date already. This helps avoid enquing UI dialogs for content that doesn't actually need to be downloaded
		if (bAllBundlesUpToDate)
		{
			SetComplete(MakeValue());
			return;
		}

		BundleManager->GetContentState(BundlesToInstall, EInstallBundleGetContentStateFlags::None, false,
			FInstallBundleGetContentStateDelegate::CreateLambda(
				[Context = SharedThis(this)](FInstallBundleCombinedContentState BundleContentState)
				{ Context->OnGotContentState(MoveTemp(BundleContentState)); }
			)
		);
	}

	void OnGotContentState(FInstallBundleCombinedContentState BundleContentState)
	{
		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();

		if (BundleContentState.GetAllBundlesHaveState(EInstallBundleInstallState::UpToDate))
		{
			OnAllInstallBundlesCompleted();
			return;
		}

		if (bCanceled)
		{
			SetCompleteCanceled();
			return;
		}

		TArray<FName> BundlesToInstall;
		for (const TPair<FGameFeaturePluginIdentifier, FGFPData>& Pair : GFPs)
		{
			const FGFPData& GFPData = Pair.Value;
			BundlesToInstall.Append(GFPData.ProtocolMetadata.InstallBundles);
		}

		EInstallBundleRequestFlags InstallFlags = EInstallBundleRequestFlags::Defaults | EInstallBundleRequestFlags::SkipMount;
		TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> MaybeRequestInfo = BundleManager->RequestUpdateContent(BundlesToInstall, InstallFlags);

		if (MaybeRequestInfo.HasError())
		{
			UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed to request content, Error: {Error}", ("Error", LexToString(MaybeRequestInfo.GetError())));
			UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
				PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
				LexToString(MaybeRequestInfo.GetError())));
			SetComplete(MoveTemp(ErrorResult));
			return;
		}

		FInstallBundleRequestInfo RequestInfo = MaybeRequestInfo.StealValue();
		if (RequestInfo.BundlesEnqueued.Num() == 0)
		{
			OnAllInstallBundlesCompleted();
			return;
		}

		PendingBundleDownloads = MoveTemp(RequestInfo.BundlesEnqueued);

		ProgressTracker.SetBundlesToTrackFromContentState(BundleContentState, PendingBundleDownloads);

		// Start ticking
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FGameFeaturePluginPredownloadContext::Tick));

		IInstallBundleManager::InstallBundleCompleteDelegate.AddSPLambda(this,
			[Context = SharedThis(this)](FInstallBundleRequestResultInfo BundleResult)
			{ Context->OnInstallBundleCompleted(MoveTemp(BundleResult)); });
		// TODO: handle pause?  Just cancel? This should only be relevent for cell connections
		// IInstallBundleManager::PausedBundleDelegate.AddRaw(this, &FGameFeaturePluginState_Downloading::OnInstallBundlePaused);
	}

	void OnInstallBundleCompleted(FInstallBundleRequestResultInfo BundleResult)
	{
		if (!PendingBundleDownloads.Contains(BundleResult.BundleName))
		{
			return;
		}

		PendingBundleDownloads.RemoveSwap(BundleResult.BundleName);

		if (Result.HasValue() && BundleResult.Result != EInstallBundleResult::OK)
		{
			if (BundleResult.OptionalErrorCode.IsEmpty())
			{
				UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed to install {Bundle}, Error: {Error}",
					("Bundle", BundleResult.BundleName), ("Error", LexToString(BundleResult.Result)));
			}
			else
			{
				UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed to install {Bundle}, Error: {Error}",
					("Bundle", BundleResult.BundleName), ("Error", BundleResult.OptionalErrorCode));
			}

			//Use OptionalErrorCode and/or OptionalErrorText if available
			const FString ErrorCodeEnding = (BundleResult.OptionalErrorCode.IsEmpty()) ? LexToString(BundleResult.Result) : BundleResult.OptionalErrorCode;
			FText ErrorText = BundleResult.OptionalErrorCode.IsEmpty() ? UE::GameFeatures::CommonErrorCodes::GetErrorTextForBundleResult(BundleResult.Result) : BundleResult.OptionalErrorText;
			Result = UE::GameFeatures::FResult(
				MakeError(FString::Printf(TEXT("%.*s%s"), PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(), *ErrorCodeEnding)),
				MoveTemp(ErrorText)
			);

			// Cancel remaining downloads
			TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
			BundleManager->CancelUpdateContent(PendingBundleDownloads);
		}

		if (PendingBundleDownloads.Num() > 0)
		{
			return;
		}

		// Delay call. We don't want to possibly release bundles from within the complete callback.
		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda([Context = SharedThis(this)](float)
			{
				Context->OnAllInstallBundlesCompleted();
				return false;
			})
		);
	}

	void OnBundleReleaseCompleted(FInstallBundleReleaseRequestResultInfo ResultInfo)
	{
		// Note: ReleaseBundlesIfPossible is what populates PendingBundleReleases and it is only called as a result of SetComplete.
		// For this reason, we will assume that once we are done releasing then we are complete.
		if (PendingBundleReleases.Num() > 0)
		{
			PendingBundleReleases.Remove(ResultInfo.BundleName);
			
			if (PendingBundleReleases.Num() == 0)
			{
				UE_LOGFMT(LogGameFeatures, Display, "GFP Predownload has finished releasing all bundles. Completed.");
				FinishComplete();

				// Since we did not remove the delegate handle earlier when we called SetComplete, remove it now
				IInstallBundleManager::ReleasedDelegate.RemoveAll(this);

				// NOTE: After we remove ourselves from ReleasedDelegate, we will likely be deleted.
				// Do not add any code after this line. Explicitly returning here to protect against future code changes
				return;
			}
		}
	}

	void OnAllInstallBundlesCompleted()
	{
		if (Result.HasError() || !bHasAssetDependencies)
		{
			SetComplete();
			Cleanup();
			return;
		}

		UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
		if (!IoStore)
		{
			UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed to get IoStoreOnDemand module!");
			UE::GameFeatures::FResult ErrorResult = MakeError(FString::Printf(TEXT("%.*s%s"),
				PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(),
				TEXT("IoStoreOnDemand.ModuleNotFound")));
			SetComplete(MoveTemp(ErrorResult));
			Cleanup();
			return;
		}

		// Create a single list of all IAD assets to install
		TMap<FName, EStreamingAssetInstallMode> InstallBundlesWithAssetDependencies;
		for (TPair<FGameFeaturePluginIdentifier, FGFPData>& Pair : GFPs)
		{
			FGFPData& GFPData = Pair.Value;

			for (int i = 0; const FName BundleName : GFPData.ProtocolMetadata.InstallBundlesWithAssetDependencies)
			{
				const EStreamingAssetInstallMode GfpInstallMode = GFPData.BundleInstallModes[i++];
			
				// Merge install modes from all GFPs that references the bundle
				EStreamingAssetInstallMode& BundleInstallMode = 
					InstallBundlesWithAssetDependencies.FindOrAdd(BundleName, GfpInstallMode);
				if (BundleInstallMode == EStreamingAssetInstallMode::GfpRequiredOnly && GfpInstallMode == EStreamingAssetInstallMode::Full)
				{
					BundleInstallMode = EStreamingAssetInstallMode::Full;
				}
			}
		}

		IoStorePendingInstalls = InstallBundlesWithAssetDependencies.Num();
		IoStoreContentHandles.SetNum(IoStorePendingInstalls);
		IoStoreInstallRequests.SetNum(IoStorePendingInstalls);
		IoStoreProgress.SetNum(IoStorePendingInstalls);
		for (int i = 0; const TPair<FName, EStreamingAssetInstallMode>& Pair : InstallBundlesWithAssetDependencies)
		{
			const FName InstallBundle = Pair.Key;
			const EStreamingAssetInstallMode InstallMode = Pair.Value;

			FNameBuilder Debugname(InstallBundle);
			Debugname.Append(TEXTVIEW("/deps"));
			IoStoreContentHandles[i] = UE::IoStore::FOnDemandContentHandle::Create(Debugname);

			UE::IoStore::FOnDemandInstallArgs InstallArgs;
			InstallArgs.MountId = InstallBundle.ToString();
			if (InstallMode == EStreamingAssetInstallMode::GfpRequiredOnly)
			{
				InstallArgs.TagSets.Emplace(TEXTVIEW("required"));
			}
			InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::InstallSoftReferences;
			if (UE::GameFeatures::CVarAllowMissingOnDemandDependencies.GetValueOnGameThread())
			{
				InstallArgs.Options |= UE::IoStore::EOnDemandInstallOptions::AllowMissingDependencies;
			}

			InstallArgs.ContentHandle = IoStoreContentHandles[i];

			UE::TUniqueLock Lock(IoStoreMutex);

			IoStoreInstallRequests[i] = IoStore->Install(MoveTemp(InstallArgs),
				// On Complete
				[this, Context = SharedThis(this), i](UE::IoStore::FOnDemandInstallResult&& OnDemandInstallResult)
				{
					UE::TUniqueLock Lock(IoStoreMutex);

					if (!OnDemandInstallResult.IsOk() && !IoStoreError.IsSet())
					{
						IoStoreError = MoveTemp(OnDemandInstallResult.Error);
						CancelIoStoreRequests();
					}

					--IoStorePendingInstalls;
					if (IoStorePendingInstalls == 0)
					{
						ExecuteOnGameThread(UE_SOURCE_LOCATION, [Context]
						{
							Context->OnAllIoStoreRequestsCompleted();
						});
					}
				},
				// On Progress
				[this, Context = SharedThis(this), i](const UE::IoStore::FOnDemandInstallProgress& InProgress)
				{
					UE::TUniqueLock Lock(IoStoreMutex);
					IoStoreProgress[i] = InProgress;
				}
			);

			++i;
		}
		
	}

	void OnAllIoStoreRequestsCompleted()
	{
		// Don't need to take the mutex here, all the requests should be done

		if (bCanceled)
		{
			SetCompleteCanceled();
		}
		else if (IoStoreError.IsSet())
		{
			UE_LOGFMT(LogGameFeatures, Error, "GFP Predownload failed to install iostore on demand assets, Error: {Error}", IoStoreError.GetValue());

			const FText ErrorMessage	= IoStoreError.GetValue().GetErrorMessage();
			FString ErrorCode			= FString(IoStoreError.GetValue().GetModuleIdAndErrorCodeString());

			ErrorCode.ReplaceCharInline(TEXT(' '), TEXT('_'), ESearchCase::CaseSensitive);
			UE::GameFeatures::FResult ErrorResult(MakeError(
				FString::Printf(TEXT("%.*s%s"), PredownloadErrorNamespace.Len(), PredownloadErrorNamespace.GetData(), *ErrorCode)),
				ErrorMessage);	
			SetComplete(MoveTemp(ErrorResult));
		}
		else
		{
			SetComplete();
		}

		Cleanup();
	}

	void UpdateProgress()
	{
		ProgressTracker.ForceTick();
		const float InstallBundleProgress = ProgressTracker.GetCurrentCombinedProgress().ProgressPercent;

		if (!bHasAssetDependencies)
		{
			Progress = InstallBundleProgress;
			if (OnProgress)
			{
				OnProgress(Progress);
			}
			return;
		}

		float IoStoreRelativeProgress;
		{
			UE::TUniqueLock Lock(IoStoreMutex);

#if 0
			// This approach doesn't work becuse we don't get initial progress for a request until the request is started
			// so the total size changes as requests are processed which causes progress backsliding
			const UE::IoStore::FOnDemandInstallProgress SumIoStoreProgress = Algo::Accumulate(
				IoStoreProgress,
				UE::IoStore::FOnDemandInstallProgress(),
				&UE::IoStore::FOnDemandInstallProgress::Combine);

			IoStoreRelativeProgress = SumIoStoreProgress.GetRelativeProgress();
#else
			// TODO: Just use flat weighting for now, but this could be improved by weighting by install size
			const float SumIoStoreProgress = Algo::TransformAccumulate(
				IoStoreProgress, 
				&UE::IoStore::FOnDemandInstallProgress::GetRelativeProgress, 
				0.0f);

			IoStoreRelativeProgress = IoStoreProgress.IsEmpty() ? 0.0f : (SumIoStoreProgress / IoStoreProgress.Num());
#endif
		}
		
		Progress = UE::GameFeatures::CombineInstallProgress(InstallBundleProgress, IoStoreRelativeProgress);

		if (OnProgress)
		{
			OnProgress(Progress);
		}
	}

	void CancelIoStoreRequests()
	{
		for (UE::IoStore::FOnDemandInstallRequest& InstallRequest : IoStoreInstallRequests)
		{
			InstallRequest.Cancel();
		}
	}

	// Predownload shouldn't pin any cached bundles so release them now
	void ReleaseBundlesIfPossible()
	{
		// Don't need to do anything special with content handles. GFPs will keep their own handles internally.
		IoStoreContentHandles.Empty();

		UGameFeaturesSubsystem& GFPSubSys = UGameFeaturesSubsystem::Get();
		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();

		TArray<FName> ReleaseList;
		TSet<FName> KeepList;
		for (const TPair<int64, TObjectPtr<UGameFeaturePluginStateMachine>>& Pair : GFPSubSys.GameFeaturePluginStateMachines)
		{
			UGameFeaturePluginStateMachine* Machine = Pair.Value;
			if (Machine &&
				GFPSubSys.GetCurrentState(Machine) > EGameFeaturePluginState::StatusKnown &&
				GFPSubSys.GetCurrentState(Machine) != EGameFeaturePluginState::Releasing)
			{
				const FGameFeatureProtocolMetadata& ProtocolMetaData = GFPSubSys.GetProtocolMetadata(Machine);
				if (ProtocolMetaData.HasSubtype<FInstallBundlePluginProtocolMetaData>())
				{
					const FInstallBundlePluginProtocolMetaData& ProtocolData = ProtocolMetaData.GetSubtype<FInstallBundlePluginProtocolMetaData>();
					KeepList.Append(ProtocolData.InstallBundles);
				}
			}
		}

		for (const TPair<FGameFeaturePluginIdentifier, FGFPData>& Pair : GFPs)
		{
			const FGameFeaturePluginIdentifier& GFPIdentifier = Pair.Key;
			const FGFPData& GFPData = Pair.Value;

			UGameFeaturePluginStateMachine* Machine = GFPSubSys.FindGameFeaturePluginStateMachine(GFPIdentifier);
			if (Machine &&
				GFPSubSys.GetCurrentState(Machine) > EGameFeaturePluginState::StatusKnown &&
				GFPSubSys.GetCurrentState(Machine) != EGameFeaturePluginState::Releasing)
			{
				// A machine is using the bundles, don't release
				KeepList.Append(GFPData.ProtocolMetadata.InstallBundles);
				continue;
			}

			// Only send this if a GFP is not using the bundle
			GFPSubSys.OnGameFeaturePostPredownloading(FString(GFPIdentifier.GetPluginName()), GFPIdentifier);

			ReleaseList.Append(GFPData.ProtocolMetadata.InstallBundles);
		}

		if (bWaitForBundlesToRelease && ReleaseList.Num() > 0)
		{
			FString BundlesStr = ReleaseList[0].ToString();
			for (int32 BundleIdx = 1; BundleIdx < ReleaseList.Num(); ++BundleIdx)
			{
				BundlesStr += TEXT(",") + ReleaseList[BundleIdx].ToString();
			}
			UE_LOGFMT(LogGameFeatures, Display, "GFP Predownload is waiting to release bundles before completing. Bundles: {Bundles}", ("Bundles", BundlesStr));
			PendingBundleReleases = ReleaseList;

			IInstallBundleManager::ReleasedDelegate.AddSPLambda(this,
				[Context = SharedThis(this)](FInstallBundleReleaseRequestResultInfo ResultInfo)
				{ Context->OnBundleReleaseCompleted(MoveTemp(ResultInfo)); });
		}

		BundleManager->RequestReleaseContent(ReleaseList, EInstallBundleReleaseRequestFlags::None, KeepList.Array());
	}
};

TSharedRef<FGameFeaturePluginPredownloadHandle> UGameFeaturesSubsystem::PredownloadGameFeaturePlugins(TConstArrayView<FString> PluginURLs, TUniqueFunction<void(const UE::GameFeatures::FResult&)> OnComplete /*= nullptr*/, TUniqueFunction<void(float)> OnProgress /*= nullptr*/, bool bWaitForBundlesToRelease /*= false*/)
{
	TSharedRef<FGameFeaturePluginPredownloadContext> Context = MakeShared<FGameFeaturePluginPredownloadContext>();
	Context->OnComplete = MoveTemp(OnComplete);
	Context->OnProgress = MoveTemp(OnProgress);
	Context->bWaitForBundlesToRelease = bWaitForBundlesToRelease;
	Context->Start(PluginURLs);

	return Context;
}

UGameFeaturePluginStateMachine* UGameFeaturesSubsystem::FindGameFeaturePluginStateMachine(const FString& PluginURL) const
{
	FGameFeaturePluginIdentifier FindPluginIdentifier(PluginURL);
	return FindGameFeaturePluginStateMachine(FindPluginIdentifier);
}

UGameFeaturePluginStateMachine* UGameFeaturesSubsystem::FindGameFeaturePluginStateMachine(const FGameFeaturePluginIdentifier& PluginIdentifier) const
{
	const FStringView ShortUrl = PluginIdentifier.GetIdentifyingString();

	TObjectPtr<UGameFeaturePluginStateMachine> const* ExistingStateMachine = 
		GameFeaturesByIdentifier.FindByHash(GetTypeHash(PluginIdentifier.GetIdentifyingString()), PluginIdentifier.GetIdentifyingString());
	if (ExistingStateMachine)
	{
		EGameFeaturePluginProtocol ExpectedProtocol = GetPluginIdentifier(*ExistingStateMachine).GetPluginProtocol();
		if (ensureMsgf(ExpectedProtocol == PluginIdentifier.GetPluginProtocol(), TEXT("Expected protocol %s for %.*s"), UE::GameFeatures::GameFeaturePluginProtocolPrefix(ExpectedProtocol), ShortUrl.Len(), ShortUrl.GetData()))
		{
			UE_LOGF(LogGameFeatures, VeryVerbose, "FOUND GameFeaturePlugin using PluginIdentifier:%.*ls for PluginURL:%ls", ShortUrl.Len(), ShortUrl.GetData(), *PluginIdentifier.GetFullPluginURL());
			return *ExistingStateMachine;
		}
	}
	UE_LOGF(LogGameFeatures, VeryVerbose, "NOT FOUND GameFeaturePlugin using PluginIdentifier:%.*ls for PluginURL:%ls", ShortUrl.Len(), ShortUrl.GetData(), *PluginIdentifier.GetFullPluginURL());

	return nullptr;
}

// Note: ProtocolOptions is not defaulted here. Any API call that could create a state machine should allow the user to pass ProtocolOptions to initialize the machine.
// It is acceptable that user passes null options. 
UGameFeaturePluginStateMachine* UGameFeaturesSubsystem::FindOrCreateGameFeaturePluginStateMachine(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, bool* bOutFoundExisting /*= nullptr*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GFP_FindOrCreateStateMachine);
	FGameFeaturePluginIdentifier PluginIdentifier(PluginURL);
	TObjectPtr<UGameFeaturePluginStateMachine> const* ExistingStateMachine =
		GameFeaturesByIdentifier.FindByHash(GetTypeHash(PluginIdentifier.GetIdentifyingString()), PluginIdentifier.GetIdentifyingString());

	if (bOutFoundExisting)
	{
		*bOutFoundExisting = !!ExistingStateMachine;
	}

	if (ExistingStateMachine)
	{
		// In this case, still return the existing machine, even if the protocol doesn't match. This function should never return null.
		// There can only be one active instance of any machine.
		EGameFeaturePluginProtocol ExpectedProtocol = GetPluginIdentifier(*ExistingStateMachine).GetPluginProtocol();
		ensureMsgf(ExpectedProtocol == PluginIdentifier.GetPluginProtocol(), TEXT("Expected protocol %s for %.*s"), UE::GameFeatures::GameFeaturePluginProtocolPrefix(ExpectedProtocol), PluginIdentifier.GetIdentifyingString().Len(), PluginIdentifier.GetIdentifyingString().GetData());

		UE_LOGF(LogGameFeatures, VeryVerbose, "Found GameFeaturePlugin StateMachine using Identifier:%.*ls from PluginURL:%ls", PluginIdentifier.GetIdentifyingString().Len(), PluginIdentifier.GetIdentifyingString().GetData(), *PluginURL);
		return *ExistingStateMachine;
	}

	UE_LOGF(LogGameFeatures, Verbose, "Creating GameFeaturePlugin StateMachine using Identifier:%.*ls from PluginURL:%ls", PluginIdentifier.GetIdentifyingString().Len(), PluginIdentifier.GetIdentifyingString().GetData(), *PluginURL);

#if UE_WITH_PACKAGE_ACCESS_TRACKING
	if (PackageLoadTracker)
	{
		FName PluginName = FName(PluginIdentifier.GetPluginName());
		PackageLoadTracker->AddRoot(PluginName);
	}
#endif

	UGameFeaturePluginStateMachine* NewStateMachine = NewObject<UGameFeaturePluginStateMachine>(this);
	const int64 Id = NextInstanceId++;
	GameFeaturePluginStateMachines.Add(Id, NewStateMachine);
	GameFeaturesByIdentifier.Add(FString(PluginIdentifier.GetIdentifyingString()), NewStateMachine);
	InitStateMachine(NewStateMachine, Id, MoveTemp(PluginIdentifier), ProtocolOptions);

	return NewStateMachine;
}

void UGameFeaturesSubsystem::LoadBuiltInGameFeaturePluginComplete(const UE::GameFeatures::FResult& Result, UGameFeaturePluginStateMachine* Machine, FGameFeaturePluginStateRange RequestedDestination)
{
	check(Machine);
	if (Result.HasValue())
	{
		checkf(RequestedDestination.Contains(GetCurrentState(Machine)), TEXT("Game feature '%s': Ending state %s is not in expected range [%s, %s]"), 
			*GetGameFeatureName(Machine), 
			*UE::GameFeatures::ToString(GetCurrentState(Machine)), 
			*UE::GameFeatures::ToString(RequestedDestination.MinState), 
			*UE::GameFeatures::ToString(RequestedDestination.MaxState));
	}
	else
	{
		SetExplanationForNotMountingPlugin(GetPluginURL(Machine), UE::GameFeatures::ToString(Result));
	}
}

void UGameFeaturesSubsystem::ChangeGameFeatureDestination(UGameFeaturePluginStateMachine* Machine, const FGameFeaturePluginStateRange& StateRange, FGameFeaturePluginChangeStateComplete CompleteDelegate)
{
	ChangeGameFeatureDestination(Machine, FGameFeatureProtocolOptions(), StateRange, CompleteDelegate);
}

void UGameFeaturesSubsystem::ChangeGameFeatureDestination(UGameFeaturePluginStateMachine* Machine, const FGameFeatureProtocolOptions& InProtocolOptions, const FGameFeaturePluginStateRange& InStateRange, FGameFeaturePluginChangeStateComplete CompleteDelegate)
{
	// Limit destination to allowed maximum. If requested minimum is beyond that we fail, if requested maximum is beyond it we simply clamp.
	if (InStateRange.MinState > Machine->StateProperties.MaximumState)
	{
		UE_LOGFMT(LogGameFeatures, Warning, "Ignoring request to change game feature {PluginName} destination to {DestinationRange} because maximum state is {MaximumState}",
			GetGameFeatureName(Machine), InStateRange, Machine->StateProperties.MaximumState);
		CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::UnreachableState)));
		return;
	}

	FGameFeaturePluginStateRange StateRange{ InStateRange.MinState, FMath::Min(Machine->StateProperties.MaximumState, InStateRange.MaxState)};
	UE_CLOGFMT(StateRange != InStateRange, LogGameFeatures, Log, "Limited state transition for GFP {Name} to maximum state {MaximumState}", GetGameFeatureName(Machine), Machine->StateProperties.MaximumState);

	const bool bSetDestination = SetDestination(Machine, StateRange,
		FGameFeatureStateTransitionComplete::CreateUObject(this, &ThisClass::ChangeGameFeatureTargetStateComplete, CompleteDelegate));

	if (bSetDestination)
	{
		UE_LOGF(LogGameFeatures, Verbose, "ChangeGameFeatureDestination: Set Game Feature %ls Destination State to [%ls, %ls]", *GetGameFeatureName(Machine), *UE::GameFeatures::ToString(StateRange.MinState), *UE::GameFeatures::ToString(StateRange.MaxState));
	}
	else
	{
		FGameFeaturePluginStateRange CurrDestination = GetDestination(Machine);
		UE_LOGF(LogGameFeatures, Display, "ChangeGameFeatureDestination: Attempting to cancel transition for Game Feature %ls. Desired [%ls, %ls]. Current [%ls, %ls]", 
			*GetGameFeatureName(Machine), 
			*UE::GameFeatures::ToString(StateRange.MinState), *UE::GameFeatures::ToString(StateRange.MaxState),
			*UE::GameFeatures::ToString(CurrDestination.MinState), *UE::GameFeatures::ToString(CurrDestination.MaxState));

		// Try canceling any current transition, then retry
		auto OnCanceled = [this, InProtocolOptions, StateRange, CompleteDelegate](UGameFeaturePluginStateMachine* Machine) mutable
		{
			// Special case for terminal state since it cannot be exited, we need to make a new machine
			if (GetCurrentState(Machine) == EGameFeaturePluginState::Terminal)
			{
				UGameFeaturePluginStateMachine* NewMachine = FindOrCreateGameFeaturePluginStateMachine(GetPluginURL(Machine), InProtocolOptions);
				checkf(NewMachine != Machine, TEXT("Game Feature Plugin %s should have already been removed from subsystem!"), *GetPluginURL(Machine));
				Machine = NewMachine;
			}

			// Now that the transition has been canceled, retry reaching the desired destination
			const bool bSetDestination = SetDestination(Machine, StateRange,
				FGameFeatureStateTransitionComplete::CreateUObject(this, &ThisClass::ChangeGameFeatureTargetStateComplete, CompleteDelegate));

			if (!ensure(bSetDestination))
			{
				UE_LOGF(LogGameFeatures, Warning, "ChangeGameFeatureDestination: Failed to set Game Feature %ls Destination State to [%ls, %ls]", *GetGameFeatureName(Machine), *UE::GameFeatures::ToString(StateRange.MinState), *UE::GameFeatures::ToString(StateRange.MaxState));

				CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::UnreachableState)));
			}
			else
			{
				UE_LOGF(LogGameFeatures, Display, "ChangeGameFeatureDestination: OnCanceled, set Game Feature %ls Destination State to [%ls, %ls]", *GetGameFeatureName(Machine), *UE::GameFeatures::ToString(StateRange.MinState), *UE::GameFeatures::ToString(StateRange.MaxState));
			}
		};

		const bool bCancelPending = TryCancel(Machine, FGameFeatureStateTransitionCanceled::CreateWeakLambda(this, MoveTemp(OnCanceled)));
		if (!ensure(bCancelPending))
		{
			UE_LOGF(LogGameFeatures, Warning, "ChangeGameFeatureDestination: Failed to cancel Game Feature %ls", *GetGameFeatureName(Machine));

			CompleteDelegate.ExecuteIfBound(UE::GameFeatures::FResult(MakeError(UE::GameFeatures::SubsystemErrorNamespace + UE::GameFeatures::CommonErrorCodes::UnreachableState + UE::GameFeatures::CommonErrorCodes::CancelAddonCode)));
		}
	}
}

void UGameFeaturesSubsystem::ChangeGameFeatureTargetStateComplete(UGameFeaturePluginStateMachine* Machine, const UE::GameFeatures::FResult& Result, FGameFeaturePluginChangeStateComplete CompleteDelegate)
{
#if WITH_EDITOR
	if (!Result.HasError() && GetCurrentState(Machine) > EGameFeaturePluginState::Mounting)
	{
		UnmountedPluginNameToExplanation.Remove(GetPluginName(Machine));
	}
#endif
	CompleteDelegate.ExecuteIfBound(Result);
}

void UGameFeaturesSubsystem::BeginTermination(UGameFeaturePluginStateMachine* Machine)
{
	check(IsValid(Machine));
	check(GetCurrentState(Machine) == EGameFeaturePluginState::Terminal);

	FStringView Identifer = GetPluginIdentifier(Machine).GetIdentifyingString();

	UE_LOGF(LogGameFeatures, Verbose, "BeginTermination of GameFeaturePlugin. Identifier:%.*ls URL:%ls", Identifer.Len(), Identifer.GetData(), *GetPluginURL(Machine));
	GameFeaturePluginStateMachines.Remove(Machine->StateProperties.Id);
	GameFeaturesByIdentifier.RemoveByHash(GetTypeHash(Identifer), Identifer);
	TerminalGameFeaturePluginStateMachines.Add(Machine);
}

void UGameFeaturesSubsystem::FinishTermination(UGameFeaturePluginStateMachine* Machine)
{
	UE_LOGF(LogGameFeatures, Verbose, "FinishTermination of GameFeaturePlugin. Identifier:%.*ls URL:%ls", GetPluginIdentifier(Machine).GetIdentifyingString().Len(), GetPluginIdentifier(Machine).GetIdentifyingString().GetData(), *GetPluginURL(Machine));
	TerminalGameFeaturePluginStateMachines.RemoveSwap(Machine);
	Machine->Executor = nullptr;
}

bool UGameFeaturesSubsystem::FindOrCreatePluginDependencyStateMachines(const FString& PluginURL, const FGameFeaturePluginStateMachineProperties& InStateProperties, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines)
{
	const FString& PluginFilename = InStateProperties.PluginInstalledFilename;
	const FGameFeatureProtocolOptions InDepProtocolOptions = InStateProperties.RecycleProtocolOptions();
    const EGameFeaturePluginProtocol InProtocol = UGameFeaturesSubsystem::GetPluginURLProtocol(PluginURL);
	const bool bWarnOnDepCreation = InStateProperties.ProtocolOptions.bLogWarningOnForcedDependencyCreation;
	const bool bErrorOnDepCreation = InStateProperties.ProtocolOptions.bLogErrorOnForcedDependencyCreation;

	TSharedPtr<const FGameFeaturePluginDetails> Details = GetGameFeaturePluginDetailsInternal(PluginFilename);
	if (Details)
	{
		FString DependencyName;
		for (const FGameFeaturePluginReferenceDetails& PluginDependency : Details->PluginDependencies)
		{
			DependencyName = FString(PluginDependency.PluginName);
			FPluginDependencyDetails DependencyDetails;
			TValueOrError<FString, FString> DependencyURLInfo = GameSpecificPolicies->ResolvePluginDependency(PluginURL, DependencyName, DependencyDetails);
			if (DependencyURLInfo.HasError())
			{
				FString ErrorMessage = FString::Printf(TEXT("Game feature plugin '%s' has unknown dependency '%s' [%s]."), *PluginFilename, *DependencyName, *DependencyURLInfo.GetError());

				SetExplanationForNotMountingPlugin(PluginURL, ErrorMessage);

				if (DependencyDetails.bFailIfNotFound)
				{
					// This plugin is known to be a hard dependency so fail early before something harder to debug happens.
					UE_LOGF(LogGameFeatures, Error, "%ls", *ErrorMessage);
					return false;
				}
				else
				{
					// Don't actually return false here as we want to still be able to progress in the case of 
					// things like an editor plugin being included as a dependency in the client. We can't tell from just 
					// the reference if a plugin is not enabled for this build target.
					UE_LOGF(LogGameFeatures, Log, "%ls", *ErrorMessage);
					continue;
				}
			}

			const FString& DependencyURL = DependencyURLInfo.GetValue();

			// Dependency may not be a GFP and so will have an empty URL but not have an error
			if (DependencyURL.IsEmpty())
			{
				UE_LOGFMT(LogGameFeatures, Verbose, "Skipping dependency {DependencyName} of plugin {Plugin} as it is not a GFP/has no URL", PluginDependency.PluginName, InStateProperties.PluginName);
				continue;
			}

			// Inherit dep protocol options if possible
			FGameFeatureProtocolOptions DepProtocolOptions;
			EGameFeaturePluginProtocol DepProtocol = UGameFeaturesSubsystem::GetPluginURLProtocol(DependencyURL);
			if (DepProtocol == EGameFeaturePluginProtocol::InstallBundle)
            {
                if (InDepProtocolOptions.HasSubtype<FInstallBundlePluginProtocolOptions>())
                {
                    DepProtocolOptions = InDepProtocolOptions;
                }
                else if (InProtocol == EGameFeaturePluginProtocol::File)
                {
                    FInstallBundlePluginProtocolOptions InstallBundleOptions;
                    InstallBundleOptions.bAllowIniLoading = true;
                    DepProtocolOptions = FGameFeatureProtocolOptions(InstallBundleOptions);
                }
            }
            // Always propogate non-protocol specific flags
            DepProtocolOptions.bForceSyncLoading = InDepProtocolOptions.bForceSyncLoading;
            DepProtocolOptions.bLogWarningOnForcedDependencyCreation = InDepProtocolOptions.bLogWarningOnForcedDependencyCreation;
            DepProtocolOptions.bLogErrorOnForcedDependencyCreation = InDepProtocolOptions.bLogErrorOnForcedDependencyCreation;

			bool bFoundExisting = false;
			UGameFeaturePluginStateMachine* ResolvedDependency = FindOrCreateGameFeaturePluginStateMachine(DependencyURL, DepProtocolOptions, &bFoundExisting);
			check(ResolvedDependency);

			if (!bFoundExisting)
			{
				// Propogate bWasLoadedAsBuiltInGameFeaturePlugin
				if (InStateProperties.bWasLoadedAsBuiltInGameFeaturePlugin)
				{
					SetWasLoadedAsBuiltIn(ResolvedDependency);
				}

				// Note: Given that LoadBuiltInGameFeaturePlugins does a topological sort, we don't expect to hit this path for built-ins
				if (bWarnOnDepCreation)
				{
					if (InStateProperties.bWasLoadedAsBuiltInGameFeaturePlugin)
					{
						UE_LOGFMT(LogGameFeatures, Warning, "GFP dependency {Dep} was forcibly created by {Parent}, Game specific policies may be incorrectly filtering this dependency.",
							("Dep", GetPluginIdentifier(ResolvedDependency).GetIdentifyingString()), ("Parent", InStateProperties.PluginIdentifier.GetIdentifyingString()));
					}
					else
					{
						UE_LOGFMT(LogGameFeatures, Warning, "GFP dependency {Dep} was unexpectedly forcibly created by {Parent}",
							("Dep", GetPluginIdentifier(ResolvedDependency).GetIdentifyingString()), ("Parent", InStateProperties.PluginIdentifier.GetIdentifyingString()));
					}
				}
				else if (bErrorOnDepCreation)
				{
					if (InStateProperties.bWasLoadedAsBuiltInGameFeaturePlugin)
					{
						UE_LOGFMT(LogGameFeatures, Error, "GFP dependency {Dep} was forcibly created by {Parent}, Game specific policies may be incorrectly filtering this dependency.",
							("Dep", GetPluginIdentifier(ResolvedDependency).GetIdentifyingString()), ("Parent", InStateProperties.PluginIdentifier.GetIdentifyingString()));
					}
					else
					{
						UE_LOGFMT(LogGameFeatures, Error, "GFP dependency {Dep} was unexpectedly forcibly created by {Parent}",
							("Dep", GetPluginIdentifier(ResolvedDependency).GetIdentifyingString()), ("Parent", InStateProperties.PluginIdentifier.GetIdentifyingString()));
					}
				}
			}

			OutDependencyMachines.Add(ResolvedDependency);
		}

		Algo::Sort(OutDependencyMachines);
		OutDependencyMachines.SetNum(Algo::Unique(OutDependencyMachines));
		return true;
	}

	return false;
}

bool UGameFeaturesSubsystem::FindPluginDependencyStateMachinesToActivate(const FString& PluginURL, const FString& PluginFilename, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines) const
{
	const bool bResult = EnumeratePluginDependenciesWithShouldActivate(PluginURL, PluginFilename, [this, &OutDependencyMachines](const FString& DependencyName, const FString& DependencyURL) {
		UGameFeaturePluginStateMachine* Dependency = FindGameFeaturePluginStateMachine(DependencyURL);
		if (Dependency)
		{
			OutDependencyMachines.Add(Dependency);
			return true;
		}
		//Expect to find all valid dependencies and activate them, so error if not found
		else
		{
			UE_LOGF(LogGameFeatures, Error, "FindPluginDependencyStateMachinesToActivate failed to find plugin state machine for %ls using URL %ls", *DependencyName, *DependencyURL);
			return false;
		}
	});
	Algo::Sort(OutDependencyMachines);
	OutDependencyMachines.SetNum(Algo::Unique(OutDependencyMachines));

	return bResult;
}

bool UGameFeaturesSubsystem::FindPluginDependencyStateMachinesToDeactivate(const FString& PluginURL, const FString& PluginFilename, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines) const
{
	const bool bResult = EnumeratePluginDependenciesWithShouldActivate(PluginURL, PluginFilename, [this, &OutDependencyMachines](const FString& DependencyName, const FString& DependencyURL) {
		FStringView PluginPath;
		if (UGameFeaturesSubsystem::ParsePluginURL(DependencyURL, nullptr, &PluginPath))
		{
			if (TSharedPtr<const FGameFeaturePluginDetails> Details = GetGameFeaturePluginDetailsInternal(FString(PluginPath)))
			{
				if (Details->BuiltInAutoState >= EBuiltInAutoState::Active) // if our builtin state is active dont deactivate these
				{
					return true;
				}
			}
		}

		UGameFeaturePluginStateMachine* Dependency = FindGameFeaturePluginStateMachine(DependencyURL);
		if (Dependency)
		{
			// deactivate by default will mean loaded, so make that our default state as we check our min required from the StateHandle, if there is a ref count
			EGameFeaturePluginState MinRequiredState = EGameFeaturePluginState::Loaded;
			FGameFeatureStateHandleReferenceController::Get().GetMinimumRequiredStateForGameFeaturePlugin(DependencyURL, MinRequiredState);

			// only if our min required state is less then active should allow these to deactivate, otherwise something still has a ref count on this depends being active
			if (MinRequiredState <= EGameFeaturePluginState::Loaded)
			{
				OutDependencyMachines.Add(Dependency);
			}
			else
			{
				UE_LOGF(LogGameFeatures, Verbose, "FindPluginDependencyStateMachinesToDeactivate skipped '%ls' as it still has an 'Active' minimum reuqired state", *DependencyName);
			}
		}
		else
		{
			// Depenedency may have been fully terminated which is considered deactivated already.
			UE_LOGF(LogGameFeatures, Log, "FindPluginDependencyStateMachinesToDeactivate unable to find plugin state machine for %ls using URL %ls", *DependencyName, *DependencyURL);
		}
		return true;
	});
	Algo::Sort(OutDependencyMachines);
	OutDependencyMachines.SetNum(Algo::Unique(OutDependencyMachines));

	return bResult;
}

template <typename CallableT>
bool UGameFeaturesSubsystem::EnumeratePluginDependenciesWithShouldActivate(const FString& PluginURL, const FString& PluginFilename, CallableT Callable) const
{
	TSharedPtr<const FGameFeaturePluginDetails> Details = GetGameFeaturePluginDetailsInternal(PluginFilename);
	if (Details)
	{
		FString DependencyName;
		for (const FGameFeaturePluginReferenceDetails& PluginDependency : Details->PluginDependencies)
		{
			if (PluginDependency.bShouldActivate)
			{
				DependencyName = FString(PluginDependency.PluginName);
				TValueOrError<FString, FString> DependencyURLInfo = GameSpecificPolicies->ResolvePluginDependency(PluginURL, DependencyName);
				if (DependencyURLInfo.HasError())
				{
					UE_LOGF(LogGameFeatures, Error, "Failure to resolve dependency %ls [%ls] for parent plugin url: %ls", *DependencyName, *DependencyURLInfo.GetError(), *PluginURL);
					return false;
				}

				const FString& DependencyURL = DependencyURLInfo.GetValue();

				// Dependency may not be a GFP and so will have an empty URL but not have an error
				if (DependencyURL.IsEmpty())
				{
					continue;
				}

				if (!Callable(DependencyName, DependencyURL))
				{
					return false;
				}
			}
		}
		return true;
	}
	else
	{
		return false;
	}
}

void UGameFeaturesSubsystem::ListGameFeaturePlugins(const TArray<FString>& Args, UWorld* InWorld, FOutputDevice& Ar)
{
	const bool bActiveOnly = Args.ContainsByPredicate([](const FString& Arg) { return Arg.Compare(TEXT("-ACTIVEONLY"), ESearchCase::IgnoreCase) == 0; });
	const bool bCsv = Args.ContainsByPredicate([](const FString& Arg) { return Arg.Compare(TEXT("-CSV"), ESearchCase::IgnoreCase) == 0; });

	FString PlatformName = FPlatformMisc::GetCPUBrand().TrimStartAndEnd();
	Ar.Logf(TEXT("Listing Game Feature Plugins...(%s)"), *PlatformName);
	if (bCsv)
	{
		Ar.Logf(TEXT(",Plugin,State"));
	}

	// create a copy for sorting
	TArray<typename decltype(GameFeaturePluginStateMachines)::ValueType> StateMachines;
	GameFeaturePluginStateMachines.GenerateValueArray(StateMachines);

	// Alphasort
	StateMachines.Sort([this](const UGameFeaturePluginStateMachine& A, const UGameFeaturePluginStateMachine& B) { return GetGameFeatureName(&A).Compare(GetGameFeatureName(&B)) < 0; });

	int32 PluginCount = 0;
	for (UGameFeaturePluginStateMachine* GFSM : StateMachines)
	{
		if (!GFSM)
		{
			continue;
		}

		if (bActiveOnly && GetCurrentState(GFSM) != EGameFeaturePluginState::Active)
		{
			continue;
		}

		if (bCsv)
		{
			Ar.Logf(TEXT(",%s,%s"), *GetGameFeatureName(GFSM), *UE::GameFeatures::ToString(GetCurrentState(GFSM)));
		}
		else
		{
			Ar.Logf(TEXT("%s (%s)"), *GetGameFeatureName(GFSM), *UE::GameFeatures::ToString(GetCurrentState(GFSM)));
		}
		++PluginCount;
	}

	Ar.Logf(TEXT("Total Game Feature Plugins: %d"), PluginCount);
}

void UGameFeaturesSubsystem::CallbackObservers(EObserverCallback CallbackType, const FGameFeaturePluginIdentifier& PluginIdentifier,
	const FString* PluginName /*= nullptr*/, 
	const UGameFeatureData* GameFeatureData /*= nullptr*/, 
	FGameFeatureStateChangeContext* StateChangeContext /*= nullptr*/)
{
	static_assert(std::underlying_type<EObserverCallback>::type(EObserverCallback::Count) == 16, "Update UGameFeaturesSubsystem::CallbackObservers to handle added EObserverCallback");

	// Protect against modifying the observer list during iteration
	TArray<UObject*> LocalObservers(Observers);

	switch (CallbackType)
	{
	case EObserverCallback::CheckingStatus:
	{
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureCheckingStatus(PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Terminating:
	{
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureTerminating(PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Predownloading:
	{
		check(PluginName);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeaturePredownloading(*PluginName, PluginIdentifier);
		}
		break;
	}
	case EObserverCallback::PostPredownloading:
	{
		check(PluginName);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeaturePostPredownloading(*PluginName, PluginIdentifier);
		}
		break;
	}
	case EObserverCallback::Downloading:
	{
		check(PluginName);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureDownloading(*PluginName, PluginIdentifier);
		}
		break;
	}
	case EObserverCallback::Releasing:
	{
		check(PluginName);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureReleasing(*PluginName, PluginIdentifier);
		}
		break;
	}
	case EObserverCallback::PreMounting:
	{
		check(PluginName);
		check(StateChangeContext);
		FGameFeaturePreMountingContext* PreMountingContext = static_cast<FGameFeaturePreMountingContext*>(StateChangeContext);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeaturePreMounting(*PluginName, PluginIdentifier, *PreMountingContext);
		}
		break;
	}
	case EObserverCallback::PostMounting:
	{
		check(PluginName);
		check(StateChangeContext);
		FGameFeaturePostMountingContext* PostMountingContext = static_cast<FGameFeaturePostMountingContext*>(StateChangeContext);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeaturePostMounting(*PluginName, PluginIdentifier, *PostMountingContext);
		}
		break;
	}
	case EObserverCallback::Registering:
	{
		check(PluginName);
		check(GameFeatureData);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureRegistering(GameFeatureData, *PluginName, PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Unregistering:
	{
		check(PluginName);
#if !WITH_EDITOR
		// In the editor the GameFeatureData asset can be force deleted, otherwise it should exist
		check(GameFeatureData);
#endif
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureUnregistering(GameFeatureData, *PluginName, PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Loading:
	{
		check(GameFeatureData);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureLoading(GameFeatureData, PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Unloading:
	{
#if !WITH_EDITOR
		// In the editor the GameFeatureData asset can be force deleted, otherwise it should exist
		check(GameFeatureData);
#endif
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureUnloading(GameFeatureData, PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Activating:
	{
		check(GameFeatureData);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureActivating(GameFeatureData, PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Activated:
	{
		check(GameFeatureData);
		for (UObject* Observer : LocalObservers)
		{
			CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureActivated(GameFeatureData, PluginIdentifier.GetFullPluginURL());
		}
		break;
	}
	case EObserverCallback::Deactivating:
	{
#if !WITH_EDITOR
		// In the editor the GameFeatureData asset can be force deleted, otherwise it should exist
		check(GameFeatureData);
#endif
		check(StateChangeContext);
		FGameFeatureDeactivatingContext* DeactivatingContext = static_cast<FGameFeatureDeactivatingContext*>(StateChangeContext);
		if (ensureAlwaysMsgf(DeactivatingContext, TEXT("Invalid StateChangeContext supplied! Could not cast to FGameFeaturePauseStateChangeContext*!")))
		{
			for (UObject* Observer : LocalObservers)
			{
				CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureDeactivating(GameFeatureData, *DeactivatingContext, PluginIdentifier.GetFullPluginURL());
			}
		}
		break;
	}
	case EObserverCallback::PauseChanged:
	{
		check(PluginName);
		check(StateChangeContext);
		FGameFeaturePauseStateChangeContext* PauseChangeContext = static_cast<FGameFeaturePauseStateChangeContext*>(StateChangeContext);
		if (ensureAlwaysMsgf(PauseChangeContext, TEXT("Invalid StateChangeContext supplied! Could not cast to FGameFeaturePauseStateChangeContext*!")))
		{
			for (UObject* Observer : LocalObservers)
			{
				CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeaturePauseChange(PluginIdentifier.GetFullPluginURL(), *PluginName, *PauseChangeContext);
			}
		}
		break;
	}
	default:
		UE_LOGF(LogGameFeatures, Fatal, "Unkown EObserverCallback!");
	}
}

void UGameFeaturesSubsystem::BatchCallbackObservers(EBatchObserverCallback BatchCallbackType)
{
	// Protect against modifying the observer list during iteration
	TArray<UObject*> LocalObservers(Observers);

	switch (BatchCallbackType)
	{
		case EBatchObserverCallback::BatchOperationStarted:
		{
			for (UObject* Observer : LocalObservers)
			{
				CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureBatchOperationStarted();
			}
			break;
		}
		case EBatchObserverCallback::BatchOperationComplete:
		{
			for (UObject* Observer : LocalObservers)
			{
				CastChecked<IGameFeatureStateChangeObserver>(Observer)->OnGameFeatureBatchOperationComplete();
			}
			break;
		}
	}
}

void UGameFeaturesSubsystem::RegisterRunningStateMachine(UGameFeaturePluginStateMachine* GFPSM)
{
	check(!RunningStateMachines.Contains(GFPSM));
	RunningStateMachines.Add(GFPSM);
}

void UGameFeaturesSubsystem::UnregisterRunningStateMachine(UGameFeaturePluginStateMachine* GFPSM)
{
	verify(RunningStateMachines.Remove(GFPSM) == 1);
}

FDelegateHandle UGameFeaturesSubsystem::AddBatchingRequest(EGameFeaturePluginState State, FGameFeaturePluginRequestUpdateStateMachine UpdateDelegate)
{
	// Adding first?
	if (BatchProcessingFences.IsEmpty())
	{
		EnableTick();
	}

	FGameFeatureBatchProcessingFence& Fence = BatchProcessingFences.FindOrAdd(State);
	return Fence.NotifyUpdateStateMachines.Add(UpdateDelegate);
}

void UGameFeaturesSubsystem::CancelBatchingRequest(EGameFeaturePluginState State, FDelegateHandle DelegateHandle)
{
	FGameFeatureBatchProcessingFence& Fence = BatchProcessingFences[State];
	Fence.NotifyUpdateStateMachines.Remove(DelegateHandle);
}

void UGameFeaturesSubsystem::EnableTick()
{
	if (!TickHandle.IsValid())
	{
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UGameFeaturesSubsystem::Tick));
		UE_LOGF(LogGameFeatures, Verbose, "UGameFeaturesSubsystem enabled tick [InFlightTransitions:%d]", RunningStateMachines.Num());
	}
}

void UGameFeaturesSubsystem::DisableTick()
{
	if (TickHandle.IsValid())
	{
		TickHandle.Reset();
		UE_LOGF(LogGameFeatures, Verbose, "UGameFeaturesSubsystem disabled tick [InFlightTransitions:%d]", RunningStateMachines.Num());
	}
}

bool UGameFeaturesSubsystem::Tick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GFP_GameFeaturesSubsystem_Tick);

	bool bContinueTicking = TickBatchProcessing();
	if (StateMachineExecutor)
	{
		bContinueTicking = StateMachineExecutor->UpdateStateMachines(DeltaTime) || bContinueTicking;
	}
	
	if (!bContinueTicking)
	{
		DisableTick();
		return false;
	}

	return true;
}

bool UGameFeaturesSubsystem::TickBatchProcessing()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GFP_GameFeaturesSubsystem_TickFences);

	// No fences?
	if (BatchProcessingFences.IsEmpty())
	{
		return false;
	}

	// Find the closest fence to process.
	EGameFeaturePluginState CurrentFence = EGameFeaturePluginState::MAX;
	for (std::underlying_type<EGameFeaturePluginState>::type iState = 0;
		iState < EGameFeaturePluginState::MAX;
		++iState)
	{
		if (BatchProcessingFences.Contains((EGameFeaturePluginState)iState))
		{
			CurrentFence = (EGameFeaturePluginState)(iState);
			break;
		}
	}

	check(CurrentFence != EGameFeaturePluginState::MAX);

	// Check for in flight transitions on state machines to be included in fence
	TArray<UGameFeaturePluginStateMachine*> AtFence;
	AtFence.Reserve(RunningStateMachines.Num());
	bool bAreAnyGFPSMsPendingAtFence = false;
	for (UGameFeaturePluginStateMachine* GFPSM : RunningStateMachines)
	{
		if (GetProperties(GFPSM).IsWaitingForBatchProcessing() && GetCurrentState(GFPSM) == CurrentFence)
		{
			AtFence.Add(GFPSM);
		}
		if (GetDestination(GFPSM).MinState >= CurrentFence && GetCurrentState(GFPSM) < CurrentFence)
		{
			bAreAnyGFPSMsPendingAtFence = true;
			break;
		}
	}

	if (!bAreAnyGFPSMsPendingAtFence)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GFP_GameFeaturesSubsystem_FenceComplete);

		FGameFeatureBatchProcessingFence& Fence = BatchProcessingFences[CurrentFence];

		BatchProcess(CurrentFence, AtFence);

		Fence.NotifyUpdateStateMachines.Broadcast();

		BatchProcessingFences.Remove(CurrentFence);

		return BatchProcessingFences.IsEmpty();
	}

	return true;
}

TSet<FString> UGameFeaturesSubsystem::GetActivePluginNames() const
{
	TSet<FString> ActivePluginNames;

	for (const TPair<int64, TObjectPtr<UGameFeaturePluginStateMachine>>& Pair : GameFeaturePluginStateMachines)
	{
		UGameFeaturePluginStateMachine* StateMachine = Pair.Value;
		if (GetCurrentState(StateMachine) == EGameFeaturePluginState::Active &&
			GetDestination(StateMachine).Contains(EGameFeaturePluginState::Active))
		{
			ActivePluginNames.Add(GetPluginName(StateMachine));
		}
	}

	return ActivePluginNames;
}

namespace GameFeaturesSubsystem
{ 
	static bool IsContentWithinActivePlugin(const FString& InObjectOrPackagePath, const TSet<FString>& ActivePluginNames)
	{
		// Look for the first slash beyond the first one we start with.
		const int32 RootEndIndex = InObjectOrPackagePath.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromStart, 1);

		const FString ObjectPathRootName = InObjectOrPackagePath.Mid(1, RootEndIndex - 1);

		if (ActivePluginNames.Contains(ObjectPathRootName))
		{
			return true;
		}

		return false;
	}
}

void UGameFeaturesSubsystem::FilterInactivePluginAssets(TArray<FAssetIdentifier>& AssetsToFilter) const
{
	AssetsToFilter.RemoveAllSwap([ActivePluginNames = GetActivePluginNames()](const FAssetIdentifier& Asset) 
	{
		return !GameFeaturesSubsystem::IsContentWithinActivePlugin(Asset.PackageName.ToString(), ActivePluginNames);
	});
}

void UGameFeaturesSubsystem::FilterInactivePluginAssets(TArray<FAssetData>& AssetsToFilter) const
{
	AssetsToFilter.RemoveAllSwap([ActivePluginNames = GetActivePluginNames()](const FAssetData& Asset) 
	{
		return !GameFeaturesSubsystem::IsContentWithinActivePlugin(Asset.GetObjectPathString(), ActivePluginNames);
	});
}

EBuiltInAutoState UGameFeaturesSubsystem::DetermineBuiltInInitialFeatureState(TSharedPtr<FJsonObject> Descriptor, const FString& ErrorContext)
{
	EBuiltInAutoState InitialState = EBuiltInAutoState::Invalid;

	FString InitialFeatureStateStr;
	if (Descriptor->TryGetStringField(TEXT("BuiltInInitialFeatureState"), InitialFeatureStateStr))
	{
		if (InitialFeatureStateStr == TEXT("Installed"))
		{
			InitialState = EBuiltInAutoState::Installed;
		}
		else if (InitialFeatureStateStr == TEXT("Registered"))
		{
			InitialState = EBuiltInAutoState::Registered;
		}
		else if (InitialFeatureStateStr == TEXT("Loaded"))
		{
			InitialState = EBuiltInAutoState::Loaded;
		}
		else if (InitialFeatureStateStr == TEXT("Active"))
		{
			InitialState = EBuiltInAutoState::Active;
		}
		else
		{
			if (!ErrorContext.IsEmpty())
			{
				UE_LOGF(LogGameFeatures, Error, "Game feature '%ls' has an unknown value '%ls' for BuiltInInitialFeatureState (expected Installed, Registered, Loaded, or Active); defaulting to Active.", *ErrorContext, *InitialFeatureStateStr);
			}
			InitialState = EBuiltInAutoState::Active;
		}
	}
	else
	{
		// BuiltInAutoRegister. Default to true. If this is a built in plugin, should it be registered automatically (set to false if you intent to load late with LoadAndActivateGameFeaturePlugin)
		bool bBuiltInAutoRegister = true;
		Descriptor->TryGetBoolField(TEXT("BuiltInAutoRegister"), bBuiltInAutoRegister);

		// BuiltInAutoLoad. Default to true. If this is a built in plugin, should it be loaded automatically (set to false if you intent to load late with LoadAndActivateGameFeaturePlugin)
		bool bBuiltInAutoLoad = true;
		Descriptor->TryGetBoolField(TEXT("BuiltInAutoLoad"), bBuiltInAutoLoad);

		// The cooker will need to activate the plugin so that assets can be scanned properly
		bool bBuiltInAutoActivate = true;
		Descriptor->TryGetBoolField(TEXT("BuiltInAutoActivate"), bBuiltInAutoActivate);

		InitialState = EBuiltInAutoState::Installed;
		if (bBuiltInAutoRegister)
		{
			InitialState = EBuiltInAutoState::Registered;
			if (bBuiltInAutoLoad)
			{
				InitialState = EBuiltInAutoState::Loaded;
				if (bBuiltInAutoActivate)
				{
					InitialState = EBuiltInAutoState::Active;
				}
			}
		}

		if (!ErrorContext.IsEmpty())
		{
			//@TODO: Increase severity to a warning after changing existing features
			UE_LOGF(LogGameFeatures, Log, "Game feature '%ls' has no BuiltInInitialFeatureState key, using legacy BuiltInAutoRegister(%d)/BuiltInAutoLoad(%d)/BuiltInAutoActivate(%d) values to arrive at initial state.",
				*ErrorContext,
				bBuiltInAutoRegister ? 1 : 0,
				bBuiltInAutoLoad ? 1 : 0,
				bBuiltInAutoActivate ? 1 : 0);
		}
	}

	return InitialState;
}

EBuiltInAutoState UGameFeaturesSubsystem::DetermineBuiltInInitialFeatureState(UE::Json::FConstObject Descriptor, const FString& ErrorContext)
{
	EBuiltInAutoState InitialState = EBuiltInAutoState::Invalid;

	if (TOptional<FStringView> InitialFeatureStateStr = UE::Json::GetStringField(Descriptor, TEXT("BuiltInInitialFeatureState")).Get({}))
	{
		if (*InitialFeatureStateStr == TEXT("Installed"))
		{
			InitialState = EBuiltInAutoState::Installed;
		}
		else if (*InitialFeatureStateStr == TEXT("Registered"))
		{
			InitialState = EBuiltInAutoState::Registered;
		}
		else if (*InitialFeatureStateStr == TEXT("Loaded"))
		{
			InitialState = EBuiltInAutoState::Loaded;
		}
		else if (*InitialFeatureStateStr == TEXT("Active"))
		{
			InitialState = EBuiltInAutoState::Active;
		}
		else
		{
			if (!ErrorContext.IsEmpty())
			{
				UE_LOGFMT(LogGameFeatures, Error, "Game feature '{ErrorContext}' has an unknown value '{ValueString}' for BuiltInInitialFeatureState"
					" (expected Installed, Registered, Loaded, or Active); defaulting to Active.",
					ErrorContext, *InitialFeatureStateStr);
			}
			InitialState = EBuiltInAutoState::Active;
		}
	}
	else
	{
		// BuiltInAutoRegister. Default to true. If this is a built in plugin, should it be registered automatically (set to false if you intent to load late with LoadAndActivateGameFeaturePlugin)
		bool bBuiltInAutoRegister = UE::Json::GetBoolField(Descriptor, TEXT("BuiltInAutoRegister")).Get(true);

		// BuiltInAutoLoad. Default to true. If this is a built in plugin, should it be loaded automatically (set to false if you intent to load late with LoadAndActivateGameFeaturePlugin)
		bool bBuiltInAutoLoad = UE::Json::GetBoolField(Descriptor, TEXT("BuiltInAutoLoad")).Get(true);

		// The cooker will need to activate the plugin so that assets can be scanned properly
		bool bBuiltInAutoActivate = UE::Json::GetBoolField(Descriptor, TEXT("BuiltInAutoActivate")).Get(true);

		InitialState = EBuiltInAutoState::Installed;
		if (bBuiltInAutoRegister)
		{
			InitialState = EBuiltInAutoState::Registered;
			if (bBuiltInAutoLoad)
			{
				InitialState = EBuiltInAutoState::Loaded;
				if (bBuiltInAutoActivate)
				{
					InitialState = EBuiltInAutoState::Active;
				}
			}
		}

		if (!ErrorContext.IsEmpty())
		{
			//@TODO: Increase severity to a warning after changing existing features
			UE_LOGFMT(LogGameFeatures, Log, "Game feature '{ErrorContext}' has no BuiltInInitialFeatureState key,"
				" using legacy BuiltInAutoRegister({BuiltInAutoRegister})/BuiltInAutoLoad({BuiltInAutoLoad})/BuiltInAutoActivate({BuiltInAutoActivate})"
				" values to arrive at initial state.",
				ErrorContext,
				bBuiltInAutoRegister,
				bBuiltInAutoLoad,
				bBuiltInAutoActivate);
		}
	}

	return InitialState;
}

EGameFeaturePluginState UGameFeaturesSubsystem::ConvertInitialFeatureStateToTargetState(EBuiltInAutoState AutoState)
{
	EGameFeaturePluginState InitialState;
	switch (AutoState)
	{
	default:
	case EBuiltInAutoState::Invalid:
		InitialState = EGameFeaturePluginState::UnknownStatus;
		break;
	case EBuiltInAutoState::Installed:
		InitialState = EGameFeaturePluginState::Installed;
		break;
	case EBuiltInAutoState::Registered:
		InitialState = EGameFeaturePluginState::Registered;
		break;
	case EBuiltInAutoState::Loaded:
		InitialState = EGameFeaturePluginState::Loaded;
		break;
	case EBuiltInAutoState::Active:
		InitialState = EGameFeaturePluginState::Active;
		break;
	}
	return InitialState;
}

void UGameFeaturesSubsystem::GetPluginsToCook(TSet<FString>& OutPlugins)
{
	// Command line parameter -CookPlugins.
	static TArray<FString> PluginsList = []()
	{
		TArray<FString> ReturnList;
		FString CookPluginsStr;
		if (FParse::Value(FCommandLine::Get(), TEXT("CookPlugins="), CookPluginsStr, false))
		{
			// check if it's a filename
			if (CookPluginsStr.EndsWith(".txt"))
			{
				const bool bSuccess = FFileHelper::LoadFileToStringWithLineVisitor(*CookPluginsStr, [&ReturnList](FStringView Line)
				{
					ReturnList.Add(FString(Line));
				});
				ensureMsgf(bSuccess, TEXT("Failed to read - %s"), *CookPluginsStr);
			}
			else
			{
				CookPluginsStr.ParseIntoArray(ReturnList, TEXT(","));
			}
		}

		if (IsRunningDLCCookCommandlet())
		{
			DlcPlugins.Append(ReturnList);
		}

		return ReturnList;
	}();
	
	OutPlugins.Append(PluginsList);	
}

const TSet<FString>& UGameFeaturesSubsystem::GetDlcPlugins()
{
	return DlcPlugins;
}

bool UGameFeaturesSubsystem::IsGameFeaturePlugin(const TSharedPtr<IPlugin>& InPlugin)
{
	const FString PluginsFolderRoot = (InPlugin->GetLoadedFrom() == EPluginLoadedFrom::Project) ? FPaths::ProjectPluginsDir() : FPaths::EnginePluginsDir();
	FString PluginPathRelativeToDomain = InPlugin->GetBaseDir();
	if (FPaths::MakePathRelativeTo(PluginPathRelativeToDomain, *PluginsFolderRoot))
	{
		PluginPathRelativeToDomain = TEXT("/") + PluginPathRelativeToDomain + TEXT("/");
	}
	else
	{
		PluginPathRelativeToDomain = FString();
	}

	return PluginPathRelativeToDomain.StartsWith(TEXT("/GameFeatures/"));
}

bool UGameFeaturesSubsystem::GetPluginDebugStateEnabled(const FString& PluginUrl) const
{
#if !UE_BUILD_SHIPPING
	return DebugStateChangedForPlugins.Contains(PluginUrl);
#else
	return false;
#endif
}
void UGameFeaturesSubsystem::SetPluginDebugStateEnabled(const FString& PluginUrl, bool bEnabled)
{
#if !UE_BUILD_SHIPPING
	if (bEnabled)
	{
		DebugStateChangedForPlugins.Add(PluginUrl);
	}
	else
	{
		DebugStateChangedForPlugins.Remove(PluginUrl);
	}
#endif
}

FString UGameFeaturesSubsystem::GetInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist /*= false*/)
{
	if (GameSpecificPolicies)
	{
		return GameSpecificPolicies->GetInstallBundleName(PluginName, bEvenIfDoesntExist);
	}
	return UGameFeatureData::GetInstallBundleName(PluginName, bEvenIfDoesntExist);
}

FString UGameFeaturesSubsystem::GetOptionalInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist /*= false*/)
{
	if (GameSpecificPolicies)
	{
		return GameSpecificPolicies->GetOptionalInstallBundleName(PluginName, bEvenIfDoesntExist);
	}
	return UGameFeatureData::GetOptionalInstallBundleName(PluginName, bEvenIfDoesntExist);
}

bool UGameFeaturesSubsystem::GetBundlesForPlugin(FStringView PluginName, TArray<FName>& OutBundles)
{
	FString PluginURL;
	if (UGameFeaturesSubsystem::Get().GetPluginURLByName(PluginName, PluginURL))
	{
		return GetBundlesForPluginURL(PluginURL, OutBundles);
	}
	return false;
}

bool UGameFeaturesSubsystem::GetBundlesForPluginURL(FStringView PluginURL, TArray<FName>& OutBundles)
{
	if (const UGameFeaturePluginStateMachine* GFSM = FindGameFeaturePluginStateMachine(FString(PluginURL)))
	{
		const FGameFeatureProtocolMetadata& ProtocolMetadata = GetProtocolMetadata(GFSM);
		if (ProtocolMetadata.HasSubtype<FInstallBundlePluginProtocolMetaData>())
		{
			const FInstallBundlePluginProtocolMetaData& ProtocolData = ProtocolMetadata.GetSubtype<FInstallBundlePluginProtocolMetaData>();
			OutBundles.Append(ProtocolData.InstallBundles);
			return true;
		}
	}
	return false;
}

void UGameFeaturesSubsystem::SetExplanationForNotMountingPlugin(const FString& PluginURL, const FString& Explanation)
{
#if WITH_EDITOR
	FGameFeaturePluginIdentifier Identifier(PluginURL);
	FStringView PluginName = Identifier.GetPluginName();
	if (!PluginName.IsEmpty())
	{
		UnmountedPluginNameToExplanation.FindOrAdd(FString(PluginName)) = Explanation;
	}
#endif
}

UGameFeaturesSubsystem::FAmmortizedLoadContext::FAmmortizedLoadContext()
{
	if (UAssetManager* AssetManager = UAssetManager::GetIfInitialized())
	{
		AssetManager->PushBulkScanning();
		bPushedAssetBulkScanning = true;
	}
	if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
	{
		TagsManager->PushDeferOnGameplayTagTreeChangedBroadcast();
		bPushedTagsBroadcast = true;
	}
}

UGameFeaturesSubsystem::FAmmortizedLoadContext::~FAmmortizedLoadContext()
{
	if (bPushedTagsBroadcast)
	{
		if (UGameplayTagsManager* TagsManager = UGameplayTagsManager::GetIfAllocated())
		{
			TagsManager->PopDeferOnGameplayTagTreeChangedBroadcast();
		}
	}
	if (bPushedAssetBulkScanning)
	{
		if (UAssetManager* AssetManager = UAssetManager::GetIfInitialized())
		{
			AssetManager->PopBulkScanning();
		}
	}

	// if we are exiting we may have lost some of these singletones we are calling into, just avoid calling into to prevent random crashes on exit
	if (!IsEngineExitRequested())
	{
		CompleteDelegate.ExecuteIfBound(Results);
		UGameFeaturesSubsystem::Get().ActiveAmmoritzedLoads.Remove(GUID);
	}
}
