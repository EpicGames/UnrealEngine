// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturesSubsystemSettings.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "ProjectDescriptor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeaturesSubsystemSettings)

const FName UGameFeaturesSubsystemSettings::LoadStateClient(TEXT("Client"));
const FName UGameFeaturesSubsystemSettings::LoadStateServer(TEXT("Server"));

UGameFeaturesSubsystemSettings::UGameFeaturesSubsystemSettings()
{
}

bool UGameFeaturesSubsystemSettings::IsValidGameFeaturePlugin(const FString& PluginDescriptorFilename) const
{
	return IsValidGameFeaturePlugin(FStringView(PluginDescriptorFilename));
}

bool UGameFeaturesSubsystemSettings::IsValidGameFeaturePlugin(FStringView PluginDescriptorFilename) const
{
	// Build the cache of game feature plugin folders the first time this is called
	static struct FBuiltInGameFeaturePluginsFolders
	{
		FBuiltInGameFeaturePluginsFolders()
		{
			const FPaths::EGetExtensionDirsFlags ExtensionFlags =
				FPaths::EGetExtensionDirsFlags::WithBase |
				FPaths::EGetExtensionDirsFlags::WithRestricted;

			// Get all the existing game feature paths
			TArray<FString> PluginPaths = FPaths::GetExtensionDirs(
				FPaths::ProjectDir(), FPaths::Combine(TEXT("Plugins"), TEXT("GameFeatures")), ExtensionFlags);

			// The base directory may not exist yet, add it if empty
			if (PluginPaths.IsEmpty())
			{
				PluginPaths.Add(FPaths::Combine(FPaths::ProjectDir(), TEXT("Plugins"), TEXT("GameFeatures")));
			}

			// Add any additional plugin directories from the project
			const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
			if (Project != nullptr)
			{
				for (const FString& Dir : Project->GetAdditionalPluginDirectories())
				{
					PluginPaths.Add(FPaths::Combine(Dir, TEXT("GameFeatures")));
				}
			}

			BuiltInGameFeaturePluginsFolders.Reserve(2 * PluginPaths.Num());
			for (FString& BuiltInFolder : PluginPaths)
			{
				BuiltInFolder /= TEXT(""); // Add trailing slash if needed
				// if the path was absolute, no need to do this
				if (FPaths::IsRelative(BuiltInFolder))
				{
					BuiltInGameFeaturePluginsFolders.Add(FPaths::ConvertRelativePathToFull(BuiltInFolder));
				}
				BuiltInGameFeaturePluginsFolders.Add(MoveTemp(BuiltInFolder));
			}
			BuiltInGameFeaturePluginsFolders.Shrink();
		}

		TArray<FString> BuiltInGameFeaturePluginsFolders;
	} Lazy;

	// Check to see if the filename is rooted in a game feature plugin folder
	for (const FString& BuiltInFolder : Lazy.BuiltInGameFeaturePluginsFolders)
	{
		if (FPathViews::IsParentPathOf(BuiltInFolder, PluginDescriptorFilename))
		{
			return true;
		}
	}

	return false;
}

