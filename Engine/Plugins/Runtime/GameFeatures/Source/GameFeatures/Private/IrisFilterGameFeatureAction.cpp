// Copyright Epic Games, Inc. All Rights Reserved.

#include "IrisFilterGameFeatureAction.h"

#include "GameFeaturesSubsystem.h"
#include "Iris/ReplicationSystem/ObjectReplicationBridge.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Misc/ScopeExit.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IrisFilterGameFeatureAction)

DEFINE_LOG_CATEGORY_STATIC(LogIrisFilterGameFeatureAction, Log, Verbose)

void UIrisFilterGameFeatureAction::OnGameFeatureActivating(FGameFeatureActivatingContext& ChangeContext)
{
	Super::OnGameFeatureActivating(ChangeContext);

#if WITH_SERVER_CODE
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (ShouldApplyToWorld(ChangeContext, WorldContext))
		{
			AddFilterToWorld(WorldContext.World());
		}
	}
#endif // WITH_SERVER_CODE
}

void UIrisFilterGameFeatureAction::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& ChangeContext)
{
	ON_SCOPE_EXIT{ Super::OnGameFeatureDeactivating(ChangeContext); };

#if WITH_SERVER_CODE
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (ShouldApplyToWorld(ChangeContext, WorldContext))
		{
			RemoveFilterFromWorld(WorldContext.World());
		}
	}
#endif // WITH_SERVER_CODE
}

bool UIrisFilterGameFeatureAction::ShouldApplyToWorld(const FGameFeatureStateChangeContext& ChangeContext, const FWorldContext& WorldContext) const
{
#if WITH_SERVER_CODE
	if (!ChangeContext.ShouldApplyToWorldContext(WorldContext))
	{
		return false;
	}

	UWorld* World = WorldContext.World();
	if (!World)
	{
		return false;
	}

	return World->GetNetMode() < NM_Client;
#else // !WITH_SERVER_CODE
	return false;
#endif // !WITH_SERVER_CODE
}

void UIrisFilterGameFeatureAction::AddFilterToWorld(UWorld* World)
{
#if WITH_SERVER_CODE
	if (!World)
	{
		return;
	}

	UReplicationSystem* ReplicationSystem = UE::Net::FReplicationSystemUtil::GetReplicationSystem(World);
	if (!ReplicationSystem)
	{
		return;
	}

	if (!ReplicationSystem->CreateFilter(FilterDefinition).IsValid())
	{
		UE_LOGF(LogIrisFilterGameFeatureAction, Warning, "Failed to create filter %ls", *FilterDefinition.FilterName.ToString());
		return;
	}

	UE_LOGF(
		LogIrisFilterGameFeatureAction,
		Verbose,
		"Created filter %ls in world %ls",
		*FilterDefinition.FilterName.ToString(),
		*World->GetName());

	AddedToWorlds.Emplace(World);

	if (UObjectReplicationBridge* ReplicationBridge = ReplicationSystem->GetReplicationBridge())
	{
		// Reload config to apply filter
		ReplicationBridge->ReloadConfig();
	}
#endif // WITH_SERVER_CODE
}

void UIrisFilterGameFeatureAction::RemoveFilterFromWorld(UWorld* World)
{
#if WITH_SERVER_CODE
	if (AddedToWorlds.Contains(World))
	{
		if (UReplicationSystem* ReplicationSystem = UE::Net::FReplicationSystemUtil::GetReplicationSystem(World))
		{
			ReplicationSystem->DestroyFilter(FilterDefinition.FilterName);

			if (UObjectReplicationBridge* ReplicationBridge = ReplicationSystem->GetReplicationBridge())
			{
				ReplicationBridge->ReloadConfig();
			}
		}

		AddedToWorlds.Remove(World);
	}
#endif // WITH_SERVER_CODE
}
