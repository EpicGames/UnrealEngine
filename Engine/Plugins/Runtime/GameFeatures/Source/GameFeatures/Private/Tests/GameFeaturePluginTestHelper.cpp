// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "GameFeaturePluginTestsHelper.h"
#include "GameFeaturesSubsystemSettings.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

FString GeneratePluginString(const TArray<FGameFeatureDependsProperties>& Depends)
{
	TStringBuilder<512> Out;
	for (const FGameFeatureDependsProperties& Depend : Depends)
	{
		Out += FString::Printf(TEXT(R"(
		{
			"Name": "%s",
			"Enabled": true,
			"Activate": %s,
			"Optional": true
		},

		)"), *Depend.PluginName, Depend.ShouldActivate == EShouldActivate::No ? TEXT("false") : TEXT("true"));
	}

	return *Out;
}

bool CreateGameFeaturePlugin(FGameFeatureProperties Properties, FString& OutPluginURL)
{
	FString PluginPath = FPaths::ProjectPluginsDir() / TEXT("GameFeatures") / Properties.PluginName;
	FString UPluginPath = PluginPath / (Properties.PluginName + TEXT(".uplugin"));

	// if one was already created here, delete it before making a new one
	if (FPaths::DirectoryExists(PluginPath))
	{
		IFileManager::Get().DeleteDirectory(*PluginPath, /* bEnsureExists */ false, /* bDeleteEntireTree */ true);
	}

	IFileManager::Get().MakeDirectory(*PluginPath, /* bCreateTree */ true);

	FString PluginDependsString = GeneratePluginString(Properties.Depends);
	FString PluginDetials = FString::Printf(TEXT(R"(
	{
		"FileVersion": 3,
		"Version": 1,
		"VersionName": "1.0",
		"FriendlyName": "%s",
		"Description": "Generated GFP for Testing",
		"Category": "GFPTesting",
		"CreatedBy": "Automated",
		"CreatedByURL": "",
		"DocsURL": "",
		"MarketplaceURL": "",
		"SupportURL": "",
		"EnabledByDefault": false,
		"CanContainContent": false,
		"IsBetaVersion": false,
		"IsExperimentalVersion": false,
		"Installed": false,
		"ExplicitlyLoaded": true,
		"BuiltInInitialFeatureState": "%s",
		"Plugins": [%s]
	})"), *Properties.PluginName, *LexToString(Properties.BuiltinAutoState), *PluginDependsString);

	bool bSavedFile = FFileHelper::SaveStringToFile(PluginDetials, *UPluginPath);

	OutPluginURL = UGameFeaturesSubsystem::GetPluginURL_FileProtocol(IFileManager::Get().ConvertToRelativePath(*UPluginPath));
	return bSavedFile;
}

#endif
