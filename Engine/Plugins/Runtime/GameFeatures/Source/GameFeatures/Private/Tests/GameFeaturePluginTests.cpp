// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "Misc/AutomationTest.h"
#include "Misc/ScopeExit.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

#include "GameFeaturePluginOperationResult.h"
#include "GameFeaturePluginStateMachine.h"
#include "GameFeaturePluginTestsHelper.h"
#include "GameFeaturesSubsystem.h"
#include "GameFeatureTypes.h"

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FWaitForTrue, bool*, bVariableToWaitFor);
bool FWaitForTrue::Update()
{
    return *bVariableToWaitFor;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FExecuteFunction, TFunction<bool()>, Function);
bool FExecuteFunction::Update()
{
	return Function();
}

EGameFeaturePluginState ConvertTargetStateToPluginState(const EGameFeatureTargetState TargetState)
{
	switch (TargetState)
	{
		case EGameFeatureTargetState::Installed:
			return EGameFeaturePluginState::Installed;
		case EGameFeatureTargetState::Registered:
			return EGameFeaturePluginState::Registered;
		case EGameFeatureTargetState::Loaded:
			return EGameFeaturePluginState::Loaded;
		case EGameFeatureTargetState::Active:
			return EGameFeaturePluginState::Active;
		default:
			break;
	}

	return EGameFeaturePluginState::MAX;
}

class FTestGameFeaturePluginBase : public FAutomationTestBase
{
public:
	FTestGameFeaturePluginBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	~FTestGameFeaturePluginBase()
	{
	}

	bool IsPluginInPluginStateRange(const FGameFeaturePluginStateRange PluginStateRange, const FString& PluginURL)
	{
		EGameFeaturePluginState CurrentPluginState = UGameFeaturesSubsystem::Get().GetPluginState(PluginURL);

		return PluginStateRange.Contains(CurrentPluginState);
	}

	void LatentTestPluginState(const EGameFeatureTargetState PluginTargetState, const FString& PluginURL)
	{
		LatentTestPluginState(FGameFeaturePluginStateRange(ConvertTargetStateToPluginState(PluginTargetState)), PluginURL);
	}

	void LatentTestPluginState(const FGameFeaturePluginStateRange PluginStateRange, const FString& PluginURL)
	{
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this, PluginStateRange, PluginURL]
		{
			TestTrue(FString::Printf(TEXT("Plugin %s in %s state, expected plugin state in range (%s, %s)"),
				*PluginURL, *UE::GameFeatures::ToString(UGameFeaturesSubsystem::Get().GetPluginState(PluginURL)), *UE::GameFeatures::ToString(PluginStateRange.MinState), *UE::GameFeatures::ToString(PluginStateRange.MaxState)),
				IsPluginInPluginStateRange(PluginStateRange, PluginURL));

			return true;
		}));
	}

	void LatentTestTransitionGFP(const EGameFeatureTargetState TargetState, const FString& PluginURL)
	{
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this, TargetState, PluginURL]
		{
			*bAsyncCommandComplete = false;

			UGameFeaturesSubsystem::Get().ChangeGameFeatureTargetState(PluginURL, TargetState,
				FGameFeaturePluginChangeStateComplete::CreateLambda([this, TargetState, PluginURL](const UE::GameFeatures::FResult& Result)
				{
					*bAsyncCommandComplete = true;
					TestFalse(FString::Printf(TEXT("Failed to transition to %s: error: %s"), *LexToString(TargetState), *UE::GameFeatures::ToString(Result)),
						Result.HasError());
				}
			));

			return true;
		}));

		ADD_LATENT_AUTOMATION_COMMAND(FWaitForTrue(&*bAsyncCommandComplete));

		LatentTestPluginState(TargetState, PluginURL);
	}

	void LatentCheckInitialPluginState()
	{
		// Check we are somewhere between uninited, and uninstalled for the first time we check this and after we restore the plugin state
		// depending on the initial state as well as deactivating/terminating the plugin we should be in the Terminal or UnknownStatus node
		LatentTestPluginState(FGameFeaturePluginStateRange(EGameFeaturePluginState::Uninitialized, EGameFeaturePluginState::Uninstalled), GFPFileURL);
	}

	void LatentRestorePluginState()
	{
		ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
		{
			*bAsyncCommandComplete = false;

			// We are in an uninstalled/terminal/not setup state. Dont try to Deactivate/Terminate when we are not Activated/Installed
			if (IsPluginInPluginStateRange(FGameFeaturePluginStateRange(EGameFeaturePluginState::Uninitialized, EGameFeaturePluginState::Uninstalled), GFPFileURL))
			{
				*bAsyncCommandComplete = true;
				return true;
			}

			UGameFeaturesSubsystem::Get().DeactivateGameFeaturePlugin(GFPFileURL,
				FGameFeaturePluginReleaseComplete::CreateLambda([this](const UE::GameFeatures::FResult& Result)
				{
					*bAsyncCommandComplete = true;
					TestFalse(FString::Printf(TEXT("Failed to deactivate plugin, error: %s"), *UE::GameFeatures::ToString(Result)),
						Result.HasError());
				}
			));

			return true;
		}));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForTrue(&*bAsyncCommandComplete));

		ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
		{
			*bAsyncCommandComplete = false;

			// We are in an uninstalled/terminal/not setup state. Dont try to Deactivate/Terminate when we are not Activated/Installed
			if (IsPluginInPluginStateRange(FGameFeaturePluginStateRange(EGameFeaturePluginState::Uninitialized, EGameFeaturePluginState::Uninstalled), GFPFileURL))
			{
				*bAsyncCommandComplete = true;
				return true;
			}

			UGameFeaturesSubsystem::Get().TerminateGameFeaturePlugin(GFPFileURL,
				FGameFeaturePluginReleaseComplete::CreateLambda([this](const UE::GameFeatures::FResult& Result)
				{
					*bAsyncCommandComplete = true;
					TestFalse(FString::Printf(TEXT("Failed to terminate plugin, error: %s"), *UE::GameFeatures::ToString(Result)),
						Result.HasError());
				}
			));

			return true;
		}));
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForTrue(&*bAsyncCommandComplete));
	}

	// For now hard-coded into EngineTest area but can always be adjusted later
	const FString GFPPluginPath = TEXT("../../../EngineTest/Plugins/GameFeatures/GameFeatureEngineTestC/GameFeatureEngineTestC.uplugin");
	const FString GFPFileURL = FString(TEXT("file:")) + GFPPluginPath;
	TSharedRef<bool> bAsyncCommandComplete = MakeShared<bool>(false);
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FGameFeatureSubsystemTestChangeState, FTestGameFeaturePluginBase, "GameFeaturePlugin.Subsystem.ChangeTargetState", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
bool FGameFeatureSubsystemTestChangeState::RunTest(const FString& Parameters)
{
	// Ensure if the test was canceled we restore the plugin back to an deactivated/terminated state
	LatentRestorePluginState();
	LatentCheckInitialPluginState();
	ON_SCOPE_EXIT
	{
		LatentRestorePluginState();
	};

	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, GFPFileURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Registered, GFPFileURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Loaded, GFPFileURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Active, GFPFileURL);

    return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FGameFeatureSubsystemTestUninstall, FTestGameFeaturePluginBase, "GameFeaturePlugin.Subsystem.FilePluginProtocol", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
bool FGameFeatureSubsystemTestUninstall::RunTest(const FString& Parameters)
{
	// Ensure if the test was canceled we restore the plugin back to an deactivated/terminated state
	LatentRestorePluginState();
	LatentCheckInitialPluginState();
	ON_SCOPE_EXIT
	{
		LatentRestorePluginState();
	};

	// Get us into an installed state so we can query info about the GFP
	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, GFPFileURL);

	ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
	{
		EGameFeaturePluginProtocol FilePluginProtocol = UGameFeaturesSubsystem::Get().GetPluginURLProtocol(GFPFileURL);
		if (!TestEqual(FString::Printf(TEXT("Expected PluginProtocol to be File but was %i"), (int)FilePluginProtocol),
				FilePluginProtocol, EGameFeaturePluginProtocol::File))
		{
			return true;
		}

		if (!TestTrue(TEXT("Expected PluginProtocol to be File but was not"),
				UGameFeaturesSubsystem::Get().IsPluginURLProtocol(GFPFileURL, EGameFeaturePluginProtocol::File)))
		{
			return true;
		}

		EGameFeaturePluginProtocol PluginProtocol;
		FStringView PluginPath;
		if (!TestTrue(TEXT("Failed to parse plugin URL"),
				UGameFeaturesSubsystem::Get().ParsePluginURL(GFPFileURL, &PluginProtocol, &PluginPath)))
		{
			return true;
		}

		if (!TestEqual(FString::Printf(TEXT("Expected PluginProtocol to be File but was %i"), (int)PluginProtocol),
				PluginProtocol, EGameFeaturePluginProtocol::File))
		{
			return true;
		}

		if (!TestEqual(FString::Printf(TEXT("Expected parsed PluginPath %s to equal %s"), PluginPath.GetData(), *GFPPluginPath),
			PluginPath, GFPPluginPath))
		{
			return true;
		}

		return true;
	}));

	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FGameFeatureSubsystemTestGetGameFeatureData, FTestGameFeaturePluginBase, "GameFeaturePlugin.Subsystem.GetGameFeatureData", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
bool FGameFeatureSubsystemTestGetGameFeatureData::RunTest(const FString& Parameters)
{
	// Ensure if the test was canceled we restore the plugin back to an deactivated/terminated state
	LatentRestorePluginState();
	LatentCheckInitialPluginState();
	ON_SCOPE_EXIT
	{
		LatentRestorePluginState();
	};

	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, GFPFileURL);

	ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
	{
		const UGameFeatureData* GameFeatureData = UGameFeaturesSubsystem::Get().GetGameFeatureDataForRegisteredPluginByURL(GFPFileURL);
		TestNull(TEXT("GameFeatureData is not NULL, GFP is only in the Installed state and should not have any GameFeatureData"), GameFeatureData);

		return true;
	}));

	LatentTestTransitionGFP(EGameFeatureTargetState::Registered, GFPFileURL);

	ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
	{
		const UGameFeatureData* GameFeatureData = UGameFeaturesSubsystem::Get().GetGameFeatureDataForRegisteredPluginByURL(GFPFileURL);
		TestNotNull(TEXT("GameFeatureData is NULL, but the GFP should have a valid GameFeatureData"), GameFeatureData);

		return true;
	}));

	LatentTestTransitionGFP(EGameFeatureTargetState::Active, GFPFileURL);

	ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
	{
		const UGameFeatureData* GameFeatureData = UGameFeaturesSubsystem::Get().GetGameFeatureDataForActivePluginByURL(GFPFileURL);
		TestNotNull(TEXT("GameFeatureData is NULL, but the GFP should have a valid GameFeatureData"), GameFeatureData);

		return true;
	}));

	return true;
}

// This test is testing that non-compiled in plugins do not get marked as built in once they are loaded through external APIs
// To see the test fail set GameFeaturePlugin.TrimNonStartupEnabledPlugins=false, which will go back to the old way the plugin system would handle new plugins not set as built in
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FGameFeatureSubsystemTestNonBuiltinPluginDoesntConvertToBuiltinPlugin, FTestGameFeaturePluginBase, "GameFeaturePlugin.Subsystem.NonBuiltinPluginDoesntConvertToBuiltinPlugin", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
bool FGameFeatureSubsystemTestNonBuiltinPluginDoesntConvertToBuiltinPlugin::RunTest(const FString& Parameters)
{
	// Ensure if the test was canceled we restore the plugin back to an deactivated/terminated state
	LatentRestorePluginState();
	LatentCheckInitialPluginState();
	ON_SCOPE_EXIT
	{
		LatentRestorePluginState();
	};

	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, GFPFileURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Registered, GFPFileURL);

	// Test we get to registered, installed -> mounted which will get our plugin in the enabled/mounted state
	ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
	{
		TestFalse(FString::Printf(TEXT("WasGameFeaturePluginLoadedAsBuiltIn on GFP %s to be false but was true"), *GFPFileURL),
			UGameFeaturesSubsystem::Get().WasGameFeaturePluginLoadedAsBuiltIn(GFPFileURL));
		return true;
	}));

	ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
	{
		*bAsyncCommandComplete = false;

		auto AdditionalFilter = [&](const FString& PluginFilename, const FGameFeaturePluginDetails& PluginDetails, FBuiltInGameFeaturePluginBehaviorOptions& OutOptions) -> bool
		{
			return true;
		};

		UGameFeaturesSubsystem::Get().LoadBuiltInGameFeaturePlugins(AdditionalFilter,
			FBuiltInGameFeaturePluginsLoaded::CreateLambda([this](const TMap<FString, UE::GameFeatures::FResult>& Results)
			{
				*bAsyncCommandComplete = true;
				for (const TPair<FString, UE::GameFeatures::FResult>& Result : Results)
				{
					TestFalse(FString::Printf(TEXT("Failed to LoadBuiltInGameFeaturePlugins on %s error: %s"), *Result.Get<0>(), *UE::GameFeatures::ToString(Result.Get<1>())),
						Result.Get<1>().HasError());
				}
			}
		));

		return true;
	}));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitForTrue(&*bAsyncCommandComplete));

	ADD_LATENT_AUTOMATION_COMMAND(FExecuteFunction([this]
	{
		TestFalse(FString::Printf(TEXT("WasGameFeaturePluginLoadedAsBuiltIn on GFP %s to be false but was true"), *GFPFileURL),
			UGameFeaturesSubsystem::Get().WasGameFeaturePluginLoadedAsBuiltIn(GFPFileURL));
		return true;
	}));

	return true;
}


/**
 * This is a test that will fail currently:
 *   Create GFPs A -> C, D and B -> C, D then activate A and B
 *   move B to registered causing C, D to transition to Loaded (since they are ShouldActive)
 *     expect:
 *       C, D to stay active since A is still active and depends on that
 *     result:
 *       C, D become deactiving/loaded since B we downgraded to registered, which deactivates its depends
 *
 *   TODO this is simply testing our CreatePlugin logic for now, Leaving on even though it doesnt test the failure at the bottom!
 */
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FGameFeatureSubsystemTestCreatePlugin, FTestGameFeaturePluginBase, "GameFeaturePlugin.Subsystem.DeactivatePreventsDependsDowngradeIfRefCount", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
bool FGameFeatureSubsystemTestCreatePlugin::RunTest(const FString& Parameters)
{
	FGameFeatureProperties PropertiesA;
	PropertiesA.PluginName = TEXT("TestingA");
	PropertiesA.BuiltinAutoState = EGameFeatureTargetState::Installed;
	PropertiesA.Depends = { FGameFeatureDependsProperties{ TEXT("TestingD"), EShouldActivate::Yes }, FGameFeatureDependsProperties{ TEXT("TestingC"), EShouldActivate::Yes } };

	FString PluginAURL;
	CreateGameFeaturePlugin(PropertiesA, PluginAURL);

	FGameFeatureProperties PropertiesB;
	PropertiesB.PluginName = TEXT("TestingB");
	PropertiesB.BuiltinAutoState = EGameFeatureTargetState::Installed;
	PropertiesB.Depends = { FGameFeatureDependsProperties{ TEXT("TestingD"), EShouldActivate::Yes }, FGameFeatureDependsProperties{ TEXT("TestingC"), EShouldActivate::Yes } };

	FString PluginBURL;
	CreateGameFeaturePlugin(PropertiesB, PluginBURL);

	FGameFeatureProperties PropertiesC;
	PropertiesC.PluginName = TEXT("TestingC");
	PropertiesC.BuiltinAutoState = EGameFeatureTargetState::Installed;

	FString PluginCURL;
	CreateGameFeaturePlugin(PropertiesC, PluginCURL);

	FGameFeatureProperties PropertiesD;
	PropertiesD.PluginName = TEXT("TestingD");
	PropertiesD.BuiltinAutoState = EGameFeatureTargetState::Installed;

	FString PluginDURL;
	CreateGameFeaturePlugin(PropertiesD, PluginDURL);

	// make sure they are all installed
	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, PluginCURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, PluginDURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, PluginAURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Installed, PluginBURL);

	// register A, and B which will pull in C, D to registered
	LatentTestTransitionGFP(EGameFeatureTargetState::Registered, PluginAURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Registered, PluginBURL);
	LatentTestPluginState(EGameFeatureTargetState::Registered, PluginCURL);
	LatentTestPluginState(EGameFeatureTargetState::Registered, PluginDURL);

	// activate A, and B which will pull in C, D to active
	LatentTestTransitionGFP(EGameFeatureTargetState::Active, PluginAURL);
	LatentTestTransitionGFP(EGameFeatureTargetState::Active, PluginBURL);
	LatentTestPluginState(EGameFeatureTargetState::Active, PluginCURL);
	LatentTestPluginState(EGameFeatureTargetState::Active, PluginDURL);

	// drop A to registered, which currently drops C, D to Loaded which is a bug
	LatentTestTransitionGFP(EGameFeatureTargetState::Registered, PluginAURL);

	/* TODO remove this once we have ref counting working, and stop deactiving C and D here
	LatentTestPluginState(EGameFeaturePluginState::Active, PluginCURL);
	LatentTestPluginState(EGameFeaturePluginState::Active, PluginDURL);
	*/

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
