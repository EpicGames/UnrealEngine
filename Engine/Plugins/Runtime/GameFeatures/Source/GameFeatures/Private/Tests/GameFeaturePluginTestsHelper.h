// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeaturesSubsystem.h"

enum class EShouldActivate : uint8
{
	No,
	Yes
};

struct FGameFeatureDependsProperties
{
	FString PluginName;
	EShouldActivate ShouldActivate = EShouldActivate::No;
};

struct FGameFeatureProperties
{
	FString PluginName;
	EGameFeatureTargetState	BuiltinAutoState = EGameFeatureTargetState::Installed;
	TArray<FGameFeatureDependsProperties> Depends;
};

bool CreateGameFeaturePlugin(FGameFeatureProperties Properties, FString& OutPluginURL);
