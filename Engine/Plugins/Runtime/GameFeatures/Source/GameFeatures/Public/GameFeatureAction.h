// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.generated.h"

#define UE_API GAMEFEATURES_API

class UGameFeatureData;
struct FGameFeatureActivatingContext;
struct FGameFeatureDeactivatingContext;
struct FAssetBundleData;

/** Represents an action to be taken when a game feature is activated */
UCLASS(MinimalAPI, DefaultToInstanced, EditInlineNew, Abstract)
class UGameFeatureAction : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual UGameFeatureData* GetGameFeatureData() const;

	/** Called when the object owning the action is registered for possible activation, this is called even if a feature never activates */
	virtual void OnGameFeatureRegistering() {}

	/** Called to unregister an action, it will not be activated again without being registered again */
	virtual void OnGameFeatureUnregistering() {}
	
	/** Called to indicate that a feature is being loaded for activation in the near future */
	virtual void OnGameFeatureLoading() {}

	/** Called to indicate that a feature is being unloaded */
	virtual void OnGameFeatureUnloading() {}

	/** Called when the feature is actually applied */
	UE_API virtual void OnGameFeatureActivating(FGameFeatureActivatingContext& Context);

	/** Older-style activation function with no context, called by base class if context version is not overridden */
	virtual void OnGameFeatureActivating() {}

	/** Called when the feature is fully active */
	virtual void OnGameFeatureActivated() {}

	/** Called when game feature is deactivated, it may be activated again in the near future */
	virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) {}

	/** Returns whether the action game feature plugin is registered or not. */
	UE_API bool IsGameFeaturePluginRegistered(bool bCheckForRegistering = false) const;

	/** Returns whether the action game feature plugin is active or not. */
	UE_API bool IsGameFeaturePluginActive(bool bCheckForActivating = false) const;

#if WITH_EDITORONLY_DATA
	virtual void AddAdditionalAssetBundleData(FAssetBundleData& AssetBundleData) {}
#endif
};

#undef UE_API
