// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"
#include "GameFeatureAction_AddActorFactory.generated.h"

class UActorComponent;
class UActorFactory;

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddActorFactory

/**
 * GameFeatureAction to add an actor factory when this plugin registers.
 * Useful for factories that might load BP classes within a plugin which might not have been discovered yet.
 */
UCLASS(MinimalAPI, meta=(DisplayName="Add Actor Factory"), Config=Engine)
class UGameFeatureAction_AddActorFactory final : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	//~Start of UGameFeatureAction interface
	virtual void OnGameFeatureRegistering() override;
	virtual void OnGameFeatureUnregistering() override;
	//~End of UGameFeatureAction interface

#if WITH_EDITOR

	/** UObject overrides */
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	/** ~UObject overrides */
#endif // WITH_EDITOR
	
protected:
#if WITH_EDITORONLY_DATA
	/**
	 * The actor factory class to add once this plugin registers
	 * Actor factories should be setup with bShouldAutoRegister so that they do not register during engine boot.
	 */
	UPROPERTY(EditAnywhere, Category = "Actor Factory")
	TSoftClassPtr<UObject> ActorFactory;

	TWeakObjectPtr<UObject> AddedFactory;
#endif

	void AddActorFactory();
	void RemoveActorFactory();
};
