// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"

#include "GameFeatureAction_AddCheats.generated.h"

class UCheatManager;
class UCheatManagerExtension;
template <typename T> class TSubclassOf;

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddCheats

/**
 * Adds cheat manager extensions to the cheat manager for each player
 */
UCLASS(MinimalAPI, meta=(DisplayName="Add Cheats"))
class UGameFeatureAction_AddCheats final : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	//~UGameFeatureAction interface
	virtual void OnGameFeatureActivating() override;
	virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;
	//~End of UGameFeatureAction interface

	//~UObject interface
#if WITH_EDITOR
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	//~End of UObject interface

private:
	void OnCheatManagerCreated(UCheatManager* CheatManager);
	void SpawnCheatManagerExtension(UCheatManager* CheatManager, const TSubclassOf<UCheatManagerExtension>& CheatManagerClass);

public:
	/** Cheat managers to setup for the game feature plugin */
	UPROPERTY(EditAnywhere, Category="Cheats")
	TArray<TSoftClassPtr<UCheatManagerExtension>> CheatManagers;

	UPROPERTY(EditAnywhere, Category="Cheats")
	bool bLoadCheatManagersAsync;

private:
	FDelegateHandle CheatManagerRegistrationHandle;

	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<UCheatManagerExtension>> SpawnedCheatManagers;

	bool bIsActive = false;
};
