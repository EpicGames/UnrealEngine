// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"
#include "GameFeatureAction_AddChunkOverride.generated.h"

class UActorComponent;

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddChunkOverride

/**
 * Used to cook assets from a GFP into a specified chunkId.
 * This can be useful when individually cooking GFPs for iteration or splitting up a packaged
 * game into smaller downloadable chunks.
 */
UCLASS(MinimalAPI, meta=(DisplayName="Add Chunk Override"), Config=Engine)
class UGameFeatureAction_AddChunkOverride final : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	//~Start of UGameFeatureAction interface
	virtual void OnGameFeatureRegistering() override;
	virtual void OnGameFeatureUnregistering() override;
	//~End of UGameFeatureAction interface

	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldAddChunkOverride, const UGameFeatureData*);
	/**
	 * Optionally bound delegate to determine when to add the chunk override.
	 * When bound this will be checked before attempting to add the chunk override.
	 * Bound delegates should return true if the GameFeatureData should have a chunk id overriden; otherwise, false.
	 */
	GAMEFEATURES_API static FShouldAddChunkOverride ShouldAddChunkOverride;

#if WITH_EDITOR
	/**
	 * Given the package name will check if this is a package from a GFP that we want to assign a specific chunk to.
	 * returns the override chunk for this package if one is set.
	 * 
	 * Should be used in combination with overriding UAssetManager::GetPackageChunkIds so that you are able to reassign a startup package.
	 * This can be necessary to reassign startup packages such as the GameFeatureData asset.
	 */
	GAMEFEATURES_API static TOptional<int32> GetChunkForPackage(const FString& PackageName);

	GAMEFEATURES_API static TArray<FString> GetPluginNameFromChunkID(int32 ChunkID);

	/** UObject overrides */
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	/** ~UObject overrides */

	bool ShouldOverrideChunk() const { return bShouldOverrideChunk; }

	int32 GetChunkID() const { return ChunkId; };

	GAMEFEATURES_API static int32 GetLowestAllowedChunkId();
#endif // WITH_EDITOR
	
private:
#if WITH_EDITORONLY_DATA
	/**
	 * Should this GFP have their packages cooked into the specified ChunkId
	 */
	UPROPERTY(EditAnywhere, Category = "Asset Management")
	bool bShouldOverrideChunk = false;

	/**
	 * What ChunkId to place the packages inside for this particular GFP.
	 */
	UPROPERTY(EditAnywhere, Category = "Asset Management", meta=(EditCondition="bShouldOverrideChunk"))
	int32 ChunkId = -1;

	/**
	 * What Chunk we are parented to.
	 * This is used by the ChunkDependencyInfo for when mutiple chunk overrides might conflict requiring assets to be pulled into a lower chunk
	 */
	UPROPERTY(EditAnywhere, Category = "Asset Management", meta=(EditCondition="bShouldOverrideChunk"))
	int32 ParentChunk = 10;

	/**
	 * Config defined value for what is the smallest chunk index the autogeneration code can generate.
	 * If autogeneration produces a chunk index lower than this value users will need to manually define the chunk index this GFP will cook into.
	 */
	UPROPERTY(config)
	int32 LowestAllowedChunkIndexForAutoGeneration = INDEX_NONE;
#endif

	void AddChunkIdOverride();
	void RemoveChunkIdOverride();

#if WITH_EDITOR
	/**
	 * Attempts to generate a unique int32 id for the given plugin based on the name of the plugin.
	 * returns -1 if a unique name couldn't be generated with consideration to other plugins that have an override id.
	 */
	int32 GenerateUniqueChunkId() const;
#endif // WITH_EDITOR
};
