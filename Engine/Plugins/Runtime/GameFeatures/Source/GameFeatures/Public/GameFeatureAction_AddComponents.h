// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "GameFeatureAction.h"
#include "GameFeaturesSubsystem.h"
#include "UObject/ObjectKey.h"

#include "GameFeatureAction_AddComponents.generated.h"

#define UE_API GAMEFEATURES_API

class AActor;
class UActorComponent;
class UGameFrameworkComponentManager;
class UGameInstance;
struct FComponentRequestHandle;
struct FWorldContext;

enum class EGameFrameworkAddComponentFlags : uint8;

// Description of a component to add to a type of actor when this game feature is enabled
// (the actor class must be game feature aware, it does not happen magically)
//@TODO: Write more documentation here about how to make an actor game feature / modular gameplay aware
USTRUCT()
struct FGameFeatureComponentEntry
{
	GENERATED_BODY()

	UE_API FGameFeatureComponentEntry();

	// The base actor class to add a component to
	UPROPERTY(EditAnywhere, Category="Components", meta=(AllowAbstract="True"))
	TSoftClassPtr<AActor> ActorClass;

	// The component class to add to the specified type of actor
	UPROPERTY(EditAnywhere, Category="Components")
	TSoftClassPtr<UActorComponent> ComponentClass;
	
	// Should this component be added for clients
	UPROPERTY(EditAnywhere, Category="Components")
	uint8 bClientComponent:1;

	// Should this component be added on servers
	UPROPERTY(EditAnywhere, Category="Components")
	uint8 bServerComponent:1;

	// Observe these rules when adding the component, if any
	UPROPERTY(EditAnywhere, Category = "Components", meta = (Bitmask, BitmaskEnum = "/Script/ModularGameplay.EGameFrameworkAddComponentFlags"))
	uint8 AdditionFlags;
};	

//////////////////////////////////////////////////////////////////////
// UGameFeatureAction_AddComponents

/**
 * Adds actor<->component spawn requests to the component manager
 *
 * @see UGameFrameworkComponentManager
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Add Components"))
class UGameFeatureAction_AddComponents final : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	//~UGameFeatureAction interface
	virtual void OnGameFeatureActivating(FGameFeatureActivatingContext& Context) override;
	virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;
#if WITH_EDITORONLY_DATA
	virtual void AddAdditionalAssetBundleData(FAssetBundleData& AssetBundleData) override;
#endif
	//~End of UGameFeatureAction interface

	//~UObject interface
#if WITH_EDITOR
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	//~End of UObject interface

	/** List of components to add to gameplay actors when this game feature is enabled */
	UPROPERTY(EditAnywhere, Category="Components", meta=(TitleProperty="{ActorClass} -> {ComponentClass}"))
	TArray<FGameFeatureComponentEntry> ComponentList;

private:
	struct FContextHandles
	{
		FDelegateHandle GameInstanceStartHandle;
		TArray<TSharedPtr<FComponentRequestHandle>> ComponentRequestHandles;
	};

	struct FGameInstanceData
	{
		// NetMode of the current world, used to determine if client/server components should be requested
		ENetMode WorldNetMode = ENetMode::NM_MAX;

		// May contain null entries, this array is parallel to ComponentList so that client/server components
		// can be added or removed in-place when the world changes
		TArray<TSharedPtr<FComponentRequestHandle>> ComponentRequestHandles;
	};

	struct FActivationContextData
	{
		// Map of GameInstance names to component data for it's world
		TMap<FObjectKey, FGameInstanceData> GameInstanceDataMap;
	};

	void AddToWorld(const FWorldContext& WorldContext, FContextHandles& Handles);

	void HandleGameInstanceStart(UGameInstance* GameInstance, FGameFeatureStateChangeContext ChangeContext);

	void HandleGameInstanceStart_NewWorldTracking(UGameInstance* GameInstance);
	void HandleGameInstanceWorldChanged(UGameInstance* GameInstance, UWorld* OldWorld, UWorld* NewWorld);
	void AddGameInstanceForActivation(TNotNull<UGameInstance*> GameInstance, FActivationContextData& ActivationContextData);
	void UpdateComponentsOnManager(TNotNull<UWorld*> World, TNotNull<UGameFrameworkComponentManager*> Manager, FGameInstanceData& ComponentData);
	TSharedPtr<FComponentRequestHandle> AddComponentRequest(TNotNull<UWorld*> World, TNotNull<UGameFrameworkComponentManager*> Manager, const FGameFeatureComponentEntry& Entry);

	TMap<FGameFeatureStateChangeContext, FContextHandles> ContextHandles;

	FDelegateHandle GameInstanceStartHandle;
	FDelegateHandle GameInstanceWorldChangedHandle;
	TMap<FGameFeatureStateChangeContext, FActivationContextData> ActivationContextDataMap;
	
};

#undef UE_API
