// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"

#include "GameFeatureAction_AddWPContent.generated.h"

#define UE_API GAMEFEATURES_API

class UGameFeatureData;
class IPlugin;
class FContentBundleClient;
class UContentBundleDescriptor;

/**
 *
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Add World Partition Content (Content Bundle)"))
class UGameFeatureAction_AddWPContent : public UGameFeatureAction
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UGameFeatureAction interface
	UE_API virtual void OnGameFeatureRegistering() override;
	UE_API virtual void OnGameFeatureUnregistering() override;
	UE_API virtual void OnGameFeatureActivating() override;
	UE_API virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;
	//~ End UGameFeatureAction interface

	const UContentBundleDescriptor* GetContentBundleDescriptor() const { return ContentBundleDescriptor; }

private:
	UPROPERTY(VisibleAnywhere, Category = ContentBundle)
	TObjectPtr<UContentBundleDescriptor> ContentBundleDescriptor;

	TSharedPtr<FContentBundleClient> ContentBundleClient;
};

#undef UE_API
