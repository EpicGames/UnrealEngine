// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"
#include "GameFeatureAction_AddWorldPartitionContent.generated.h"

#define UE_API GAMEFEATURES_API

class UExternalDataLayerAsset;

/**
 *
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Add World Partition Content"))
class UGameFeatureAction_AddWorldPartitionContent : public UGameFeatureAction
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UGameFeatureAction interface
	UE_API virtual void OnGameFeatureRegistering() override;
	UE_API virtual void OnGameFeatureUnregistering() override;
	UE_API virtual void OnGameFeatureActivating(FGameFeatureActivatingContext& Context) override;
	UE_API virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;
	//~ End UGameFeatureAction interface

#if WITH_EDITOR
	//~ Begin UObject interface
	UE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	//~ End UObject interface

	const TObjectPtr<const UExternalDataLayerAsset>& GetExternalDataLayerAsset() const { return ExternalDataLayerAsset; }
#endif

protected:
	/** External Data Layer used by this action. */
	UPROPERTY(EditAnywhere, Category = DataLayer)
	TObjectPtr<const UExternalDataLayerAsset> ExternalDataLayerAsset;

	/** Optional console variable name which (when specified) must return true for the external data layer to be added. If not specified, this condition is skipped. */
	UPROPERTY(EditAnywhere, Category = DataLayer)
	FName RequiredCVar;

private:
	/** Returns whether the external data layer should be activated based on RequiredCVarName. Returns true when no CVar is configured. */
	UE_API bool ShouldActivateExternalDataLayer() const;

#if WITH_EDITOR
	UE_API void OnExternalDataLayerAssetChanged(const UExternalDataLayerAsset* OldAsset, const UExternalDataLayerAsset* NewAsset);
#endif

	/** Used to detect changes on the Data Layer Asset in the action. */
	TWeakObjectPtr<const UExternalDataLayerAsset> PreEditChangeExternalDataLayerAsset;
	TWeakObjectPtr<const UExternalDataLayerAsset> PreEditUndoExternalDataLayerAsset;

#if WITH_EDITORONLY_DATA
	/** Only used when converting from UGameFeatureAction_AddWPContent */
	UPROPERTY()
	FGuid ConvertedContentBundleGuid;
#endif

	friend class UGameFeatureActionConvertContentBundleWorldPartitionBuilder;
};

#undef UE_API
