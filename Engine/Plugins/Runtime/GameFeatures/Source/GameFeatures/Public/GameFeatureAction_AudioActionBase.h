// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceHandle.h"
#include "GameFeatureAction.h"
#include "GameFeaturesSubsystem.h"

#include "GameFeatureAction_AudioActionBase.generated.h"

#define UE_API GAMEFEATURES_API

/**
 * Base class for GameFeatureActions that affect the audio engine
 */
UCLASS(MinimalAPI, Abstract)
class UGameFeatureAction_AudioActionBase : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	//~ Begin of UGameFeatureAction interface
	UE_API virtual void OnGameFeatureActivating(FGameFeatureActivatingContext& Context) override;
	UE_API virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;
	//~ End of UGameFeatureAction interface

protected:
	/** Handle to delegate callbacks */
	FDelegateHandle DeviceCreatedHandle;
	FDelegateHandle DeviceDestroyedHandle;

	UE_API void OnDeviceCreated(Audio::FDeviceId InDeviceId);
	UE_API void OnDeviceDestroyed(Audio::FDeviceId InDeviceId);

	virtual void AddToDevice(const FAudioDeviceHandle& AudioDeviceHandle) {}
	virtual void RemoveFromDevice(const FAudioDeviceHandle& AudioDeviceHandle) {}
};

#undef UE_API
