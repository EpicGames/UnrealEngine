// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"
#include "GameFeatureAction_DataRegistry.generated.h"

class UDataRegistry;

/** Specifies a list of Data Registries to load and initialize with this feature */
UCLASS(MinimalAPI, meta = (DisplayName = "Add Data Registry"))
class UGameFeatureAction_DataRegistry : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	virtual void OnGameFeatureRegistering() override;
	virtual void OnGameFeatureUnregistering() override;
	virtual void OnGameFeatureActivating() override;
	virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;

	/** If true, we should load the registry at registration time instead of activation time */
	virtual bool ShouldPreloadAtRegistration();

#if WITH_EDITORONLY_DATA
	virtual void AddAdditionalAssetBundleData(FAssetBundleData& AssetBundleData) override;
#endif

	//~UObject interface
#if WITH_EDITOR
	virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	//~End of UObject interface

private:
	/** List of registry assets to load and initialize */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	TArray<TSoftObjectPtr<UDataRegistry> > RegistriesToAdd;

	/** If true, this will preload the registries when the feature is registered in the editor to support the editor pickers */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	bool bPreloadInEditor;

	/** If true, this will preload the registries when the feature is registered whilst a commandlet is running */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	bool bPreloadInCommandlets;
};
