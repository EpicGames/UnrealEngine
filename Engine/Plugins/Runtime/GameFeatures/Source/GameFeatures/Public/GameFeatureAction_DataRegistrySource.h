// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"
#include "GameFeatureAction_DataRegistrySource.generated.h"

class UCurveTable;
class UDataTable;

/** Defines which source assets to add and conditions for adding */
USTRUCT()
struct FDataRegistrySourceToAdd
{
	GENERATED_BODY()

	FDataRegistrySourceToAdd()
		: AssetPriority(0)
		, bClientSource(false)
		, bServerSource(false)
	{}

	/** Name of the registry to add to */
	UPROPERTY(EditAnywhere, Category="Registry Data")
	FName RegistryToAddTo;

	/** Priority to use when adding to the registry.  Higher priorities are searched first */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	int32 AssetPriority;

	/** Should this component be added for clients */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	uint8 bClientSource : 1;

	/** Should this component be added on servers */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	uint8 bServerSource : 1;

	/** Link to the data table to add to the registry */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	TSoftObjectPtr<UDataTable> DataTableToAdd;

	/** Link to the curve table to add to the registry */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	TSoftObjectPtr<UCurveTable> CurveTableToAdd;
};

/** Specifies a list of source assets to add to Data Registries when this feature is activated */
UCLASS(MinimalAPI, meta = (DisplayName = "Add Data Registry Source"))
class UGameFeatureAction_DataRegistrySource : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	GAMEFEATURES_API virtual void OnGameFeatureRegistering() override;
	GAMEFEATURES_API virtual void OnGameFeatureUnregistering() override;
	GAMEFEATURES_API virtual void OnGameFeatureActivating(FGameFeatureActivatingContext& Context) override;
	GAMEFEATURES_API virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;

	/** If true, we should load the sources at registration time instead of activation time */
	GAMEFEATURES_API virtual bool ShouldPreloadAtRegistration();

#if WITH_EDITORONLY_DATA
	GAMEFEATURES_API virtual void AddAdditionalAssetBundleData(FAssetBundleData& AssetBundleData) override;
#endif

	//~UObject interface
#if WITH_EDITOR
	GAMEFEATURES_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	//~End of UObject interface

#if WITH_EDITOR
	/** Used by an editor tool to programmatically register a new source */
	GAMEFEATURES_API void AddSource(const FDataRegistrySourceToAdd& NewSource);
#endif

protected:
	GAMEFEATURES_API void RegisterRuntimeSources();
	
	GAMEFEATURES_API void UnregisterSources();
	
private:
	
	/** List of sources to add when this feature is activated */
	UPROPERTY(EditAnywhere, Category = "Registry Data", meta=(TitleProperty="RegistryToAddTo"))
	TArray<FDataRegistrySourceToAdd> SourcesToAdd;

	/** If true, this will preload the sources when the feature is registered in the editor to support the editor pickers */
	UPROPERTY(EditAnywhere, Category = "Registry Data")
	bool bPreloadInEditor;
};
