// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "GameFeatureAction.h"
#include "Engine/ExternalObjectAndActorDependencyGatherer.h" // For IExternalAssetPathsProvider

#include "GameFeatureData.generated.h"

#define UE_API GAMEFEATURES_API

class FConfigFile;
struct FPrimaryAssetTypeInfo;
struct FExternalDataLayerUID;

struct FAssetData;

#if WITH_EDITOR

class FPathTree;
class FGameFeatureDataExternalAssetsPathCache : public IExternalAssetPathsProvider
{
	TMultiMap<FName, FName> PerLevelAssetDirectories;
	TMap<FName, FString> GameFeatureDataAssetsToMountPoint;
	TSet<FName> AllLevels;
	bool bCacheIsUpToDate = false;
	FDelegateHandle OnPathAddedDelegateHandle;

public:
		
	FGameFeatureDataExternalAssetsPathCache();
	virtual ~FGameFeatureDataExternalAssetsPathCache();

	void OnPathsAdded(TConstArrayView<FStringView>);
	void UpdateCache(const FUpdateCacheContext& Context) override;
	TArray<FName> GetPathsForPackage(FName LevelPath) override;
};
#endif

/** Data related to a game feature, a collection of code and content that adds a separable discrete feature to the game */
UCLASS(MinimalAPI)
class UGameFeatureData : public UPrimaryDataAsset
{
	GENERATED_BODY()

public:
	/** Method to get where the primary assets should scanned from in the plugin hierarchy */
	virtual const TArray<FPrimaryAssetTypeInfo>& GetPrimaryAssetTypesToScan() const { return PrimaryAssetTypesToScan; }

#if WITH_EDITOR
	virtual TArray<FPrimaryAssetTypeInfo>& GetPrimaryAssetTypesToScan() { return PrimaryAssetTypesToScan; }
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	static UE_API void GetDependencyDirectoriesFromAssetData(const FAssetData& AssetData, TArray<FString>& OutDependencyDirectories);
#endif //if WITH_EDITOR

	/** Method to process the base ini file for the plugin during loading */
	static UE_API void InitializeBasePluginIniFile(const FString& PluginInstalledFilename);

	/** Method to process ini files for the plugin during activation */
	static UE_API void InitializeHierarchicalPluginIniFiles(const FString& PluginInstalledFilename);
	static UE_API void InitializeHierarchicalPluginIniFiles(const TArrayView<FString>& PluginInstalledFilenames);

	UFUNCTION(BlueprintCallable, Category = "GameFeature")
	static UE_API void GetPluginName(const UGameFeatureData* GFD, FString& PluginName);

	UE_API void GetPluginName(FString& PluginName) const;

	/** Returns whether the game feature plugin is registered or not. */
	UE_API bool IsGameFeaturePluginRegistered(bool bCheckForRegistering = false) const;

	/** Returns whether the game feature plugin is active or not. */
	UE_API bool IsGameFeaturePluginActive(bool bCheckForActivating = false) const;

	/**
	 * Returns the install bundle name if one exists for this plugin.
	 * @param - PluginName - the name of the GameFeaturePlugin we want to get a bundle for. Should be the same name as the .uplugin file
	 * @param - bEvenIfDoesntExist - when true will return the name of bundle we are looking for without checking if it exists or not.
	 */
	static UE_API FString GetInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist = false);

	/**
	 * Returns the optional install bundle name if one exists for this plugin.
	 * @param - PluginName - the name of the GameFeaturePlugin we want to get a bundle for. Should be the same name as the .uplugin file
	 * @param - bEvenIfDoesntExist - when true will return the name of bundle we are looking for without checking if it exists or not.
	 */
	static UE_API FString GetOptionalInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist = false);

public:
	//~UPrimaryDataAsset interface
#if WITH_EDITORONLY_DATA
	UE_API virtual void UpdateAssetBundleData() override;
#endif
	//~End of UPrimaryDataAsset interface

	//~UObject interface
#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;
#endif
	//~End of UObject interface

	const TArray<UGameFeatureAction*>& GetActions() const { return Actions; }

#if WITH_EDITOR
	TArray<TObjectPtr<UGameFeatureAction>>& GetMutableActionsInEditor() { return Actions; }
#endif

private:
	/** Internal helper function to reload config data on objects as a result of a plugin INI being loaded */
	static UE_API void ReloadConfigs(FConfigFile& PluginConfig);

protected:
#if WITH_EDITOR
	UFUNCTION()
	UE_API virtual TArray<UClass*> GetDisallowedActions() const;
#endif

	/** List of actions to perform as this game feature is loaded/activated/deactivated/unloaded */
	UPROPERTY(EditDefaultsOnly, Instanced, Category="Game Feature | Actions", meta = (GetDisallowedClasses = "GetDisallowedActions"))
	TArray<TObjectPtr<UGameFeatureAction>> Actions;

	/** List of asset types to scan at startup */
	UPROPERTY(EditAnywhere, Category="Game Feature | Asset Manager", meta=(TitleProperty="PrimaryAssetType"))
	TArray<FPrimaryAssetTypeInfo> PrimaryAssetTypesToScan;
};

#undef UE_API
