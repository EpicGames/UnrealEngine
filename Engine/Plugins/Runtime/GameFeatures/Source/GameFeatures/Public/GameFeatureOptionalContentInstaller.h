// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BundlePrereqCombinedStatusHelper.h"
#include "GameFeatureStateChangeObserver.h"
#include "HAL/IConsoleManager.h"
#include "InstallBundleManagerInterface.h"
#include "GameFeatureOptionalContentInstaller.generated.h"

namespace UE::GameFeatures
{
	struct FResult;
}

enum class EInstallBundleReleaseRequestFlags : uint32;

/** 
 * Utilty class to install GFP optional paks (usually containing optional mips) in sync with GFP content installs.
 * NOTE: This only currently supports LRU cached install bundles. It would need UI callbacks and additional support 
 * for free space checks and progress tracking to fully support non-LRU GFPs.
 */
UCLASS(MinimalAPI)
class UGameFeatureOptionalContentInstaller : public UObject, public IGameFeatureStateChangeObserver
{
	GENERATED_BODY()

public:
	static GAMEFEATURES_API TMulticastDelegate<void(const FString& PluginName, const UE::GameFeatures::FResult&)> OnOptionalContentInstalled;
	static GAMEFEATURES_API TMulticastDelegate<void()> OnOptionalContentInstallStarted;
	static GAMEFEATURES_API TMulticastDelegate<void(const bool bInstallSuccessful)> OnOptionalContentInstallFinished;

public:
	virtual void BeginDestroy() override;

	void GAMEFEATURES_API Init(
		TUniqueFunction<TArray<FName>(FString)> GetOptionalBundlePredicate,
		TUniqueFunction<TArray<FName>(FString)> GetOptionalKeepBundlePredicate = [](auto) { return TArray<FName>{}; });

	void GAMEFEATURES_API Enable(bool bEnable);

	void GAMEFEATURES_API UninstallContent();

	void GAMEFEATURES_API EnableCellularDownloading(bool bEnable);
	
	bool GAMEFEATURES_API HasOngoingInstalls() const;

	/**
	 * Returns the total progress (between 0 and 1) of all the optional bundles currently being installed.
	 * Returns 0 while the progress tracker is starting which happens the first time it is called while bundles are being installed.
	 */
	float GAMEFEATURES_API GetAllInstallsProgress();

private:
	bool UpdateContent(const FString& PluginName, bool bIsPredownload);

	void OnContentInstalled(FInstallBundleRequestResultInfo InResult, FString PluginName);

	void ReleaseContent(const FString& PluginName, EInstallBundleReleaseRequestFlags Flags = EInstallBundleReleaseRequestFlags::None);

	void OnEnabled();
	void OnDisabled();

	bool IsEnabled() const;

	void OnCVarsChanged();

	void StartTotalProgressTracker();

	// IGameFeatureStateChangeObserver Interface
	virtual void OnGameFeaturePredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier) override;
	virtual void OnGameFeatureDownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier) override;
    virtual void OnGameFeatureRegistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FString& PluginURL) override;
	virtual void OnGameFeatureReleasing(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier) override;

private:
	struct FGFPInstall
	{
		FDelegateHandle CallbackHandle;
		TArray<FName> BundlesEnqueued;
		bool bIsPredownload = false;
	};

private:
	TUniqueFunction<TArray<FName>(FString)> GetOptionalBundlePredicate;
	TUniqueFunction<TArray<FName>(FString)> GetOptionalKeepBundlePredicate;
	TSharedPtr<IInstallBundleManager> BundleManager;
	TSet<FString> RelevantGFPs;
	TMap<FString, FGFPInstall> ActiveGFPInstalls;
	/** Map of bundle to the plugins that requested them. Used for knowing when we can release an optional bundle. */
	TMap<FName, TSet<FName>> BundleToPlugins;
	TOptional<FInstallBundleCombinedProgressTracker> TotalProgressTracker;
	static const FName GFOContentRequestName;

	/** Delegate handle for a console variable sink */
	FConsoleVariableSinkHandle CVarSinkHandle;

	bool bEnabled = false;
	bool bEnabledCVar = false;
	bool bAllowCellDownload = false;
};
