// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/ValueOrError.h"

#define UE_API GAMEFEATURES_API

enum class EInstallBundleResult : uint32;
enum class EInstallBundleReleaseResult : uint32;

namespace UE::GameFeatures
{
	//Type used to determine if our FResult is actually an error or success (and hold the error code)
	using FErrorCodeType = TValueOrError<void, FString>;

	/** Struct wrapper on a combined TValueOrError ErrorCode and FText OptionalErrorText that can provide an FString error code and
		if set more detailed FText OptionalErrorText information for the results of a GameFeaturesPlugin operation. */
	struct FResult
	{
		/** Error Code representing the error that occurred */
		FErrorCodeType ErrorCode;

		/** Optional Localized error description to bubble up to the user if one was generated */
		FText OptionalErrorText;

		/** Quick functions that just pass through to the ErrorCode */
		bool HasValue() const { return ErrorCode.HasValue(); }
		bool HasError() const { return ErrorCode.HasError(); }
		FString GetError() const { return ErrorCode.GetError(); }
		FString StealError() { return ErrorCode.StealError(); }

		UE_API FResult(FErrorCodeType ErrorCodeIn, FText ErrorTextIn);

		/** Explicit constructor for times we want to create an FResult directly from an FErrorCodeType and not through MakeValue or MakeError.
			Explicit to avoid any confusion with the following templated constructors. */
		UE_API explicit FResult(FErrorCodeType ErrorCodeIn);

		/** Template Conversion Constructor to allow us to initialize from TValueOrError MakeValue
			This is needed because of how TValueOrError implements MakeValue through the same templated approach with TValueOrError_ValueProxy. */
		template <typename... ArgTypes>
		FResult(TValueOrError_ValueProxy<ArgTypes...>&& ValueProxyIn)
			: ErrorCode(MoveTemp(ValueProxyIn))
			, OptionalErrorText()
		{
		}

		/** Template Conversion Constructor to allow us to initialize from TValueOrError MakeError 
			This is needed because of how TValueOrError implements MakeError through the same templated approach with TValueOrError_ErrorProxy. */
		template <typename... ArgTypes>
		FResult(TValueOrError_ErrorProxy<ArgTypes...>&& ErrorProxyIn)
			: ErrorCode(MoveTemp(ErrorProxyIn))
			, OptionalErrorText()
		{
		}

	private:
		//No default constructor as we want to force you to always specify at the minimum 
		//if the FResult is an error or not through a supplied FErrorCodeType
		FResult() = delete;
	};

	GAMEFEATURES_API FString ToString(const FResult& Result);

	namespace CommonErrorCodes
	{
		const FText& GetErrorTextForBundleResult(EInstallBundleResult ErrorResult);
		const FText& GetErrorTextForReleaseResult(EInstallBundleReleaseResult ErrorResult);

		const FText& GetGenericFatalError();
		const FText& GetGenericConnectionError();
		const FText& GetGenericMountError();
		const FText& GetGenericReleaseResult();
	}
}

#undef UE_API
