// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "GameFeatureStateChangeObserver.generated.h"

class UGameFeatureData;
struct FGameFeaturePluginIdentifier;
struct FGameFeaturePreMountingContext;
struct FGameFeaturePostMountingContext;
struct FGameFeatureDeactivatingContext;
struct FGameFeaturePauseStateChangeContext;

UINTERFACE(MinimalAPI)
class UGameFeatureStateChangeObserver : public UInterface
{
	GENERATED_BODY()
};

/**
 * This class is meant to be overridden in your game to handle game-specific reactions to game feature plugins
 * being mounted or unmounted
 *
 * Generally you should prefer to use UGameFeatureAction instances on your game feature data asset instead of
 * this, especially if any data is involved
 *
 * If you do use these, create them in your UGameFeaturesProjectPolicies subclass and register them via
 * AddObserver / RemoveObserver on UGameFeaturesSubsystem
 */
class IGameFeatureStateChangeObserver
{
	GENERATED_BODY()

public:

	// Invoked when going from the UnknownStatus state to the CheckingStatus state
	virtual void OnGameFeatureCheckingStatus(const FString& PluginURL) {}

	// Invoked prior to terminating a game feature plugin
	virtual void OnGameFeatureTerminating(const FString& PluginURL) {}

	// Invoked when content begins installing via predownload
	virtual void OnGameFeaturePredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier) {}

	// Invoked when pre-downloading content has finished.
	virtual void OnGameFeaturePostPredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier) {}

	// Invoked when content begins installing
	virtual void OnGameFeatureDownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier) {}

	// Invoked when content is released (the point it at which it is safe to remove it)
	virtual void OnGameFeatureReleasing(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier) {}

	// Invoked prior to mounting a plugin (but after its install bundles become available, if any)
	virtual void OnGameFeaturePreMounting(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier, FGameFeaturePreMountingContext& Context) {}

	// Invoked at the end of the plugin mounting phase (whether it was successfully mounted or not)
	virtual void OnGameFeaturePostMounting(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier, FGameFeaturePostMountingContext& Context) {}

	// Invoked after a game feature plugin has been registered
	virtual void OnGameFeatureRegistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FString& PluginURL) {}

	// Invoked prior to unregistering a game feature plugin
	virtual void OnGameFeatureUnregistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FString& PluginURL) {}

	// Invoked in the early stages of the game feature plugin loading phase
	virtual void OnGameFeatureLoading(const UGameFeatureData* GameFeatureData, const FString& PluginURL) {}

	// Invoked after a game feature plugin is unloaded
	virtual void OnGameFeatureUnloading(const UGameFeatureData* GameFeatureData, const FString& PluginURL) {}

	// Invoked prior to activating a game feature plugin
	virtual void OnGameFeatureActivating(const UGameFeatureData* GameFeatureData, const FString& PluginURL) {}

	// Invoked after a game feature plugin is activated
	virtual void OnGameFeatureActivated(const UGameFeatureData* GameFeatureData, const FString& PluginURL) {}

	// Invoked prior to deactivating a game feature plugin
	virtual void OnGameFeatureDeactivating(const UGameFeatureData* GameFeatureData, FGameFeatureDeactivatingContext& Context, const FString& PluginURL) {}

	/** Called whenever a GameFeature State either pauses or resumes work without transitioning out of that state.
		EX: Downloading paused due to a users cellular data settings or the user taking a pause action. We
		may not yet transition to a download error, but want a way to observe this behavior. */
	virtual void OnGameFeaturePauseChange(const FString& PluginURL, const FString& PluginName, FGameFeaturePauseStateChangeContext& Context) {}

	virtual void OnGameFeatureBatchOperationStarted() {}

	virtual void OnGameFeatureBatchOperationComplete() {}
};
