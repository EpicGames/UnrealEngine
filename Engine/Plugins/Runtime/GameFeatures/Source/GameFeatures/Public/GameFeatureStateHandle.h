// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "Misc/Guid.h"

struct FScriptContainerElement;

#define UE_API GAMEFEATURES_API

enum class EGameFeatureStateHandleOptions : uint8
{
	None = 0,
	TrackDependencies	=  1 << 0,
	KeepRegistered		=  1 << 1,
};
ENUM_CLASS_FLAGS(EGameFeatureStateHandleOptions)

/**
 * Public class keep track of ownership and reference counts on which plugins + depends through a public GUID
 *   Used internally to reference a structure containing this data
 *   You must call the InitAndRegister function this to Properly create a version of this so it is registered internally, otherwise it is invalid to use this
 */
class FGameFeatureStateHandle
{
public:
	FGameFeatureStateHandle() = default;
	explicit UE_API FGameFeatureStateHandle(const FString& Owner, EGameFeatureStateHandleOptions Options = EGameFeatureStateHandleOptions::TrackDependencies);

	/** Releases all the ref counts, and if any GFPs need to be downgraded moves them. When done we call the Callback if setup. The Dtor call this */
	UE_API void ReleaseAndUnregister(TFunction<void(bool)> ResetCompleteCallback = [](bool){});
	UE_API ~FGameFeatureStateHandle();

	/** Move only, so delete copy ctor/assign */
	FGameFeatureStateHandle(const FGameFeatureStateHandle&) = delete;
	FGameFeatureStateHandle& operator=(const FGameFeatureStateHandle&) = delete;

	UE_API FGameFeatureStateHandle(FGameFeatureStateHandle&&);
	UE_API FGameFeatureStateHandle& operator=(FGameFeatureStateHandle&&);

	/** Init the StateHandle with a unique id, owner, options and registers it with the StateHandleRefController, if this StateHandle already has a valid GUID, we return false */
	UE_API bool InitAndRegister(const FString& Owner, EGameFeatureStateHandleOptions Options = EGameFeatureStateHandleOptions::TrackDependencies);

	UE_API FString ToString() const;
	UE_API FGuid GetUniqueId() const;

	/** Takes over the ref counts of the StateHandle and its dtor is called */
	UE_API void TakeOwnership(FGameFeatureStateHandle&& StateHandle);

	UE_API bool IsValid() const;
	UE_API bool operator==(const FGameFeatureStateHandle& Other) const;

#if !UE_BUILD_SHIPPING
	/** Compare and log differences between two state handle owned GFPs. Note: only compares plugin sets across handles (not target states) */ 
	UE_API int32 CompareAndLogGameFeatureDifferences(const FGameFeatureStateHandle& FinalizedStateHandle);
#endif // !UE_BUILD_SHIPPING

private:
	FGuid UniqueId;
};

#undef UE_API
