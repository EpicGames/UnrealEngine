// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureStateHandle.h"
#include "GameFeatureTypes.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "GameFeaturesSubsystem.h"

namespace UE::GameFeatures { struct FResult; }
struct FGameFeatureProtocolOptions;
class IPlugin;

#define UE_API GAMEFEATURES_API

// TODO these currently are no longer needed but keeping for now incase we find the return handle + refresh idea doesnt work out
using UMultipleGameFeatureStateHandleLoadComplete = TDelegate<void(const TMap<FString, UE::GameFeatures::FResult>& /*Results*/)>;
using UGameFeatureStateHandleLoadComplete = TDelegate<void(const UE::GameFeatures::FResult& /*Result*/)>;

struct FGameFeatureStateHandleReferenceControllerInternal;
class FGameFeatureStateHandleReferenceController
{
public:
	static UE_API FGameFeatureStateHandleReferenceController& Get();
	static UE_API bool IsLifetimeControlEnabled();

	UE_API void RegisterNewStateHandle(const FGameFeatureStateHandle& StateHandle, const FString& Owner, EGameFeatureStateHandleOptions Options);
	UE_API void UnregisterStateHandle(const FGameFeatureStateHandle& StateHandle);

	UE_API void LoadAndActivateGameFeaturePlugin(const FGameFeatureStateHandle& StateHandle, const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const UGameFeatureStateHandleLoadComplete& CompleteDelegate);
	UE_API void LoadAndActivateGameFeaturePlugin(const FGameFeatureStateHandle& StateHandle, const TArray<FString>& PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const UMultipleGameFeatureStateHandleLoadComplete& CompleteDelegate);

	UE_API void LoadGameFeaturePlugin(const FGameFeatureStateHandle& StateHandle, const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const UGameFeatureStateHandleLoadComplete& CompleteDelegate);
	UE_API void LoadGameFeaturePlugin(const FGameFeatureStateHandle& StateHandle, const TArray<FString>& PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const UMultipleGameFeatureStateHandleLoadComplete& CompleteDelegate);

	UE_API void RegisterGameFeaturePlugin(const FGameFeatureStateHandle& StateHandle, const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const UGameFeatureStateHandleLoadComplete& CompleteDelegate);
	UE_API void RegisterGameFeaturePlugin(const FGameFeatureStateHandle& StateHandle, const TArray<FString>& PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const UMultipleGameFeatureStateHandleLoadComplete& CompleteDelegate);

	UE_API void LoadBuiltInGameFeaturePlugins_Amortized(const FGameFeatureStateHandle& StateHandle, const TArray<TSharedRef<IPlugin>>& Plugins, const UGameFeaturesSubsystem::FBuiltInPluginAdditionalFilters_Copyable& AdditionalFilter, int32 AmortizeRateMS, const UMultipleGameFeatureStateHandleLoadComplete& CompleteDelegate);

	/** Removes all the GFPs references held by this list of handles, and will empty out the StateHandle list */
	UE_API void ResetGameFeatureStateHandle(const FGameFeatureStateHandle& StateHandle, TFunction<void(bool)> OnComplete = [](bool) {});

	/** Adds or Updates the GFP URL to the PluginState */
	UE_API void AddOrUpdateReference(const FGameFeatureStateHandle& StateHandle, const FString& GameFeaturePluginURL, EGameFeaturePluginState PluginState, bool bCollectDepends = true);

	/** From the RefCounts gets the lowest GFP state required by a GFP we are tracking into OutPluginState */
	UE_API bool GetMinimumRequiredStateForGameFeaturePlugin(const FString& GameFeaturePluginURL, EGameFeaturePluginState& OutPluginState);

	/** The From StateHandles internals are appended to the To StateHandle, and the list of GFPs in From are Emptied */
	UE_API void MergeGameFeatureStateHandle(const FGameFeatureStateHandle& From, const FGameFeatureStateHandle& To);

#if !UE_BUILD_SHIPPING
	/** Compare and log differences between two state handle owned GFPs. Note: only compares plugin sets across handles (not target states) */
	UE_API int32 CompareAndLogGameFeatureDifferences(const FGameFeatureStateHandle& A, const FGameFeatureStateHandle& B);
#endif // !UE_BUILD_SHIPPING

private:
	FGameFeatureStateHandleReferenceController();
	FGameFeatureStateHandleReferenceController(FGameFeatureStateHandleReferenceController const&) = delete;
	FGameFeatureStateHandleReferenceController& operator=(FGameFeatureStateHandleReferenceController const&) = delete;

	/** Internal function so we can handle callbacks safely if the StateHandle passed in goes out of scope, we can detect missing guids over corrupted memory */
	void AddOrUpdateReference(const FGuid& StateHandleId, const FString& GameFeaturePluginURL, EGameFeaturePluginState PluginState, bool bCollectDepends = true);

	void RemoveReferences(const FGameFeatureStateHandle& StateHandle, const TArray<FString>& PluginsToRemoveRefences, TFunction<void(bool)> OnComplete);

	bool GetDowngradedTargetState(const FString& PluginURL, EGameFeaturePluginState& OutTransitionState) const;

	TUniquePtr<FGameFeatureStateHandleReferenceControllerInternal> Internal;
};

#undef UE_API
