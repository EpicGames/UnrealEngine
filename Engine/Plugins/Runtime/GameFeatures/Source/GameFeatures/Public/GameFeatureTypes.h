// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/EnumRange.h"

class FCbWriter;
class FString;

/** 
  GFP States 
  Desitination states must be fully ordered, Transistion and Error states should be in between the Destination states they transition to/from.
  See the state chart in GameFeaturePluginStateMachine.h for reference.
*/
#define GAME_FEATURE_PLUGIN_STATE_LIST(XSTATE)	\
	XSTATE(Uninitialized,					NSLOCTEXT("GameFeatures", "UninitializedStateDisplayName", "Uninitialized"))									/* Unset. Not yet been set up. */ \
	XSTATE(Terminal,						NSLOCTEXT("GameFeatures", "TerminalStateDisplayName", "Terminal"))												/* Final State before removal of the state machine. */ \
	XSTATE(UnknownStatus,					NSLOCTEXT("GameFeatures", "UnknownStatusStateDisplayName", "UnknownStatus"))									/* Initialized, but the only thing known is the URL to query status. */ \
	XSTATE(Uninstalled,						NSLOCTEXT("GameFeatures", "UninstalledStateDisplayName", "Uninstalled"))										/* All installed data for this plugin has now been uninstalled from local storage (i.e the hard drive) */ \
	XSTATE(Uninstalling,					NSLOCTEXT("GameFeatures", "UninstallingStateDisplayName", "Uninstalling"))										/* Transition state between StatusKnown -> Terminal for any plugin that can have data that needs to have local data uninstalled. */ \
	XSTATE(ErrorUninstalling,				NSLOCTEXT("GameFeatures", "ErrorUninstallingStateDisplayName", "ErrorUninstalling"))							/* Error state for Uninstalling -> Terminal transition. */  \
	XSTATE(CheckingStatus,					NSLOCTEXT("GameFeatures", "CheckingStatusStateDisplayName", "CheckingStatus"))									/* Transition state UnknownStatus -> StatusKnown. The status is in the process of being queried. */ \
	XSTATE(ErrorCheckingStatus,				NSLOCTEXT("GameFeatures", "ErrorCheckingStatusStateDisplayName", "ErrorCheckingStatus"))						/* Error state for UnknownStatus -> StatusKnown transition. */ \
	XSTATE(ErrorUnavailable,				NSLOCTEXT("GameFeatures", "ErrorUnavailableStateDisplayName", "ErrorUnavailable"))								/* Error state for UnknownStatus -> StatusKnown transition. */ \
	XSTATE(StatusKnown,						NSLOCTEXT("GameFeatures", "StatusKnownStateDisplayName", "StatusKnown"))										/* The plugin's information is known, but no action has taken place yet. */ \
	XSTATE(Releasing,						NSLOCTEXT("GameFeatures", "ReleasingStateDisplayName", "Releasing"))											/* Transition State for Installed -> StatusKnown. Releases local data from any relevant caches. */ \
	XSTATE(ErrorManagingData,				NSLOCTEXT("GameFeatures", "ErrorManagingDataStateDisplayName", "ErrorManagingData"))							/* Error state for Installed -> StatusKnown and StatusKnown -> Installed transitions. */ \
	XSTATE(Downloading,						NSLOCTEXT("GameFeatures", "DownloadingStateDisplayName", "Downloading"))										/* Transition state StatusKnown -> Installed. In the process of adding to local storage. */ \
	XSTATE(Installed,						NSLOCTEXT("GameFeatures", "InstalledStateDisplayName", "Installed"))											/* The plugin is in local storage (i.e. it is on the hard drive) */ \
	XSTATE(ErrorMounting,					NSLOCTEXT("GameFeatures", "ErrorMountingStateDisplayName", "ErrorMounting"))									/* Error state for Installed -> Registered and Registered -> Installed transitions. */ \
	XSTATE(ErrorWaitingForDependencies,		NSLOCTEXT("GameFeatures", "ErrorWaitingForDependenciesStateDisplayName", "ErrorWaitingForDependencies"))		/* Error state for Installed -> Registered and Registered -> Installed transitions. */ \
	XSTATE(ErrorRegistering,				NSLOCTEXT("GameFeatures", "ErrorRegisteringDisplayName", "ErrorRegistering"))									/* Error state for Installed -> Registered and Registered -> Installed transitions. */ \
	XSTATE(WaitingForDependencies,			NSLOCTEXT("GameFeatures", "WaitingForDependenciesStateDisplayName", "WaitingForDependencies"))					/* Transition state Installed -> Registered. In the process of loading code/content for all dependencies into memory. */ \
	XSTATE(AssetDependencyStreamOut,		NSLOCTEXT("GameFeatures", "AssetDependencyStreamOutDisplayName", "AssetDependencyStreamOut"))					/* Transition state Registered -> Installed. In the process of streaming out individual assets from dependencies. */ \
	XSTATE(ErrorAssetDependencyStreaming,	NSLOCTEXT("GameFeatures", "ErrorAssetDependencyStreamingStateDisplayName", "ErrorAssetDependencyStreaming"))	/* Error state for Installed -> Registered and Registered -> Installed transitions. */ \
	XSTATE(AssetDependencyStreaming,		NSLOCTEXT("GameFeatures", "AssetDependencyStreamingDisplayName", "AssetDependencyStreaming"))					/* Transition state Installed -> Registered. In the process of streaming individual assets from dependencies. */ \
	XSTATE(Unmounting,						NSLOCTEXT("GameFeatures", "UnmountingStateDisplayName", "Unmounting"))											/* Transition state Registered -> Installed. The content file(s) (i.e. pak file) for the plugin is unmounting. */ \
	XSTATE(Mounting,						NSLOCTEXT("GameFeatures", "MountingStateDisplayName", "Mounting"))												/* Transition state Installed -> Registered. The content files(s) (i.e. pak file) for the plugin is getting mounted. */ \
	XSTATE(Unregistering,					NSLOCTEXT("GameFeatures", "UnregisteringStateDisplayName", "Unregistering"))									/* Transition state Registered -> Installed. Cleaning up data gathered in Registering. */ \
	XSTATE(Registering,						NSLOCTEXT("GameFeatures", "RegisteringStateDisplayName", "Registering"))										/* Transition state Installed -> Registered. Discovering assets in the plugin, but not loading them, except a few for discovery reasons. */ \
	XSTATE(Registered,						NSLOCTEXT("GameFeatures", "RegisteredStateDisplayName", "Registered"))											/* The assets in the plugin are known, but have not yet been loaded, except a few for discovery reasons. */ \
	XSTATE(ErrorLoading,					NSLOCTEXT("GameFeatures", "ErrorLoadingDisplayName", "ErrorLoading"))											/* Error state for Loading -> Loaded transition */ \
	XSTATE(Unloading,						NSLOCTEXT("GameFeatures", "UnloadingStateDisplayName", "Unloading"))											/* Transition state Loaded -> Registered. In the process of removing code/content from memory. */ \
	XSTATE(Loading,							NSLOCTEXT("GameFeatures", "LoadingStateDisplayName", "Loading"))												/* Transition state Registered -> Loaded. In the process of loading code/content into memory. */ \
	XSTATE(Loaded,							NSLOCTEXT("GameFeatures", "LoadedStateDisplayName", "Loaded"))													/* The plugin is loaded into memory, but not registered with game systems and active. */ \
	XSTATE(ErrorActivatingDependencies,		NSLOCTEXT("GameFeatures", "ErrorActivatingDependenciesStateDisplayName", "ErrorActivatingDependencies"))		/* Error state for Registered -> Active transition. */ \
	XSTATE(ActivatingDependencies,			NSLOCTEXT("GameFeatures", "ActivatingDependenciesStateDisplayName", "ActivatingDependencies"))					/* Transition state Registered -> Active. In the process of selectively activating dependencies.*/ \
	XSTATE(Deactivating,					NSLOCTEXT("GameFeatures", "DeactivatingStateDisplayName", "Deactivating"))										/* Transition state Active -> Loaded. Currently unregistering with game systems. */ \
	XSTATE(ErrorDeactivatingDependencies,	NSLOCTEXT("GameFeatures", "ErrorDeactivatingDependenciesStateDisplayName", "DeactivatingDependencies"))			/* Error state for Active -> Loaded transition. */ \
	XSTATE(DeactivatingDependencies,		NSLOCTEXT("GameFeatures", "DeactivatingDependenciesStateDisplayName", "DeactivatingDependencies"))				/* Transition state Active -> Loaded. In the process of selectively deactivating dependencies.*/ \
	XSTATE(Activating,						NSLOCTEXT("GameFeatures", "ActivatingStateDisplayName", "Activating"))											/* Transition state Loaded -> Active. Currently registering plugin code/content with game systems. */ \
	XSTATE(Active,							NSLOCTEXT("GameFeatures", "ActiveStateDisplayName", "Active"))													/* Plugin is fully loaded and active. It is affecting the game.  */ 

#define GAME_FEATURE_PLUGIN_STATE_ENUM(inEnum, inText) inEnum,
// EGameFeaturePluginState defines the list of public states for game feature plugins.
// Internally, GFP state machines may use additional states to manage transitions between public states
namespace GameFeaturePluginStatePrivate
{
	enum EGameFeaturePluginState : uint8
	{
		GAME_FEATURE_PLUGIN_STATE_LIST(GAME_FEATURE_PLUGIN_STATE_ENUM)
		MAX
	};
}
using EGameFeaturePluginState = GameFeaturePluginStatePrivate::EGameFeaturePluginState;
#undef GAME_FEATURE_PLUGIN_STATE_ENUM
ENUM_RANGE_BY_COUNT(EGameFeaturePluginState, EGameFeaturePluginState::MAX);

GAMEFEATURES_API void SerializeForLog(FCbWriter& Writer, EGameFeaturePluginState State);

namespace UE::GameFeatures
{
	GAMEFEATURES_API FString ToString(EGameFeaturePluginState InType);
}

/** GFP Protocols */
#define GAME_FEATURE_PLUGIN_PROTOCOL_LIST(XPROTO)	\
	XPROTO(File,			TEXT("file:"))			\
	XPROTO(InstallBundle,	TEXT("installbundle:"))	\
	XPROTO(Unknown,			TEXT(""))

#define GAME_FEATURE_PLUGIN_PROTOCOL_ENUM(inEnum, inString) inEnum,
enum class EGameFeaturePluginProtocol : uint8
{
	GAME_FEATURE_PLUGIN_PROTOCOL_LIST(GAME_FEATURE_PLUGIN_PROTOCOL_ENUM)
	Count
};
#undef GAME_FEATURE_PLUGIN_PROTOCOL_ENUM

ENUM_RANGE_BY_COUNT(EGameFeaturePluginProtocol, EGameFeaturePluginProtocol::Count);

/** GFP URL Options */
#define GAME_FEATURE_PLUGIN_URL_OPTIONS_LIST(XOPTION)	\
	XOPTION(Bundles,	(1 << 0))

#define GAME_FEATURE_PLUGIN_URL_OPTIONS_ENUM(inEnum, inVal) inEnum = inVal,
#define GAME_FEATURE_PLUGIN_URL_OPTIONS_ALL(inEnum, inVal) | inEnum
enum class EGameFeatureURLOptions : uint8
{
	None,
	GAME_FEATURE_PLUGIN_URL_OPTIONS_LIST(GAME_FEATURE_PLUGIN_URL_OPTIONS_ENUM)
	All = None GAME_FEATURE_PLUGIN_URL_OPTIONS_LIST(GAME_FEATURE_PLUGIN_URL_OPTIONS_ALL)
};
#undef GAME_FEATURE_PLUGIN_URL_OPTIONS_ALL
#undef GAME_FEATURE_PLUGIN_URL_OPTIONS_ENUM

ENUM_CLASS_FLAGS(EGameFeatureURLOptions);

const TCHAR* LexToString(EGameFeatureURLOptions);
void LexFromString(EGameFeatureURLOptions& ValueOut, const FStringView& StringIn);
