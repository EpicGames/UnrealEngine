// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FString;
enum class EGameFeaturePluginProtocol : uint8;

namespace GameFeaturePluginStatePrivate
{
	enum EGameFeaturePluginState : uint8;
}
using EGameFeaturePluginState = GameFeaturePluginStatePrivate::EGameFeaturePluginState;

namespace UE::GameFeatures
{
	GAMEFEATURES_API FString ToString(EGameFeaturePluginState InType);
}

enum class EGameFeatureURLOptions : uint8;
