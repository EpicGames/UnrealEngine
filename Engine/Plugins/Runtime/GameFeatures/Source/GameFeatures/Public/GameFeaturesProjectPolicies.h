// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TransactionallySafeRecursiveMutex.h"
#include "UObject/PrimaryAssetId.h"
#include "GameFeaturesSubsystem.h"

#include "GameFeaturesProjectPolicies.generated.h"

#define UE_API GAMEFEATURES_API

class UGameFeatureData;

namespace UE::GameFeatures
{
	UE_API extern const TAutoConsoleVariable<bool>& GetCVarForceAsyncLoad();
}

struct FPluginDependencyDetails
{
	bool bFailIfNotFound = false;
};

enum class EStreamingAssetInstallMode : uint8
{
	GfpRequiredOnly, // only stream in data required for the GFP to load
	Full, // stream in all data
};

// Specification of thread safety contract for a class derived from UGameFeaturesProjectPolicies.
// @see UGameFeaturesProjectPolicies::GetThreadSafetyVersion
enum class EGameFeaturesProjectPoliciesThreadSafetyVersion
{
	// No methods are threadsafe, default for derived classes
	None = 0, 
	// Initial version when this enum was added. The follow methods must be callable from any thread:
	//	ShouldReadPluginDetails
	// 	GetGameFeaturePluginURL
	//  IsPluginAllowed
	Initial,
};

// This class allows project-specific rules to be implemented for game feature plugins.
// Create a subclass and choose it in Project Settings .. Game Features
UCLASS(MinimalAPI)
class UGameFeaturesProjectPolicies : public UObject
{
	GENERATED_BODY()

public:
	// Called when the game feature manager is initialized
	virtual void InitGameFeatureManager() { }

	// Called when the game feature manager is shut down
	virtual void ShutdownGameFeatureManager() { }

	/**
	 * Returns which methods of this class may be called from any thread.
	 * The methods which must be callable from any thread for each version are marked as such in this file.
	 * Derived classes should return the maximum value of the enum when they were written.
	 * The base implementation returns the maximum current version if this is not a derived class.
	 *
	 * Specifically, implementations should expect that these methods may be called concurrently with calls 
	 * the game makes into UGameFeaturesSubsystem, and during the subsystem's own ticks.
	 * The subsystem makes no particular guarantees about avoid reentrancy if these methods call into UGameFeaturesSubsystem.
	 * e.g. if the implementation of IsPluginAllowed calls into the subsystem, the subsystem may call back into another method such as
	 * GetGameFeaturePluginURL. The possibility of priority inversion / deadlocks should be considered.
	 *
	 * Note that no implementation needs to support methods being called concurrently with InitGameFeatureManager
	 * or ShutdownGameFeatureManager unless otherwise specified.
	 */
	UE_API virtual EGameFeaturesProjectPoliciesThreadSafetyVersion GetThreadSafetyVersion() const;

	// Called to determined the expected state of a plugin under the WhenLoading conditions.
	UE_API virtual bool WillPluginBeCooked(const FString& PluginFilename, const FGameFeaturePluginDetails& PluginDetails) const;

	// Called when a game feature plugin enters the Loading state to determine additional assets to load
	virtual TArray<FPrimaryAssetId> GetPreloadAssetListForGameFeature(const UGameFeatureData* GameFeatureToLoad, bool bIncludeLoadedAssets = false) const { return TArray<FPrimaryAssetId>(); }

	// Returns the bundle state to use for assets returned by GetPreloadAssetListForGameFeature()
	// See the Asset Manager documentation for more information about asset bundles
	// Eventually, we should deprecate GetPreloadBundleStateForGameFeature in favor of GetPreloadBundleStateForPlugin since having the plugin name is useful
	virtual const TArray<FName> GetPreloadBundleStateForGameFeature() const { return TArray<FName>(); }
	virtual const TArray<FName> GetPreloadBundleStateForPlugin(const FString& PluginName) const { return GetPreloadBundleStateForGameFeature(); }

	// Called to determine if this should be treated as a client, server, or both for data preloading
	// Actions can use this to decide what to load at runtime
	virtual void GetGameFeatureLoadingMode(bool& bLoadClientData, bool& bLoadServerData) const { bLoadClientData = true; bLoadServerData = true; }

	// Called to determine if we are still during engine startup, which can modify loading behavior.
	// This defaults to true for the first few frames of a normal game or editor, but can be overridden.
	UE_API virtual bool IsLoadingStartupPlugins() const;

	/**
	  * Called to determine the plugin URL for a given known Plugin. Can be used if the policy wants to deliver non file based URLs.
	  *
	  * THREAD SAFETY: Should be threadsafe for EGameFeaturesProjectPoliciesThreadSafetyVersion::Initial
	  */
	UE_API virtual bool GetGameFeaturePluginURL(const TSharedRef<IPlugin>& Plugin, FString& OutPluginURL) const;

	UE_DEPRECATED(5.6, "Replaced with IsPluginAllowed(PluginURL, OutReason).")
	virtual bool IsPluginAllowed(const FString& PluginURL) const { return true; }

	/**
	  * Called to determine if a plugin is allowed to be loaded or not
	  * (e.g., when doing a fast cook a game might want to disable some or all game feature plugins)
	  *
	  * THREAD SAFETY: Should be threadsafe for EGameFeaturesProjectPoliciesThreadSafetyVersion::Initial
	  */
	virtual bool IsPluginAllowed(const FString& PluginURL, FString* OutReason) const 
	{ 
		if (OutReason)
		{
			*OutReason = {};
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return IsPluginAllowed(PluginURL); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	  * Called to determine if a plugin is allowed to be loaded or not.
	  * (e.g., when doing a fast cook a game might want to disable some or all game feature plugins)
	  * New version that provides plugin details when they are already available to avoid re-entrant calls.
	  * Prior version is not yet deprecated.
	  *
	  * THREAD SAFETY: Should be threadsafe for EGameFeaturesProjectPoliciesThreadSafetyVersion::Initial
	  */
	virtual bool IsPluginAllowed(const FString& PluginURL, const FGameFeaturePluginDetails* InDetails, FString* OutReason) const 
	{ 
		return IsPluginAllowed(PluginURL, OutReason); 
	}

	/**
	  * Return true if a uplugin's details should be read and false if it should be skipped.
	  * Skipped plugins will not be processed as GFPs and skipped as though they didn't exist.
	  * Useful to limit the number of uplugin files opened for perf reasons.
	  *
	  * THREAD SAFETY: Should be threadsafe for EGameFeaturesProjectPoliciesThreadSafetyVersion::Initial
	  */
	virtual bool ShouldReadPluginDetails(const FString& PluginDescriptorFilename) const { return true; }

	// Return true if a plugin should be allowed to register if it doesn't have a GameFeatureData. Useful in cases where you want to allow the content of certain plugins to not exist.
	virtual bool ShouldAllowMissingGameFeatureData(const FString& PluginName) const { return false; }

	// Called to resolve plugin dependencies, will successfully return an empty string if a dependency is not a GFP.
	// This may be called with file protocol for built-in plugins in some cases, even if a different protocol is used at runtime.
	// returns The dependency URL or an error if the dependency could not be resolved
	UE_API virtual TValueOrError<FString, FString> ResolvePluginDependency(const FString& PluginURL, const FString& DependencyName, FPluginDependencyDetails& OutDetails) const;
	UE_API virtual TValueOrError<FString, FString> ResolvePluginDependency(const FString& PluginURL, const FString& DependencyName) const;

	// Called to resolve install bundles for streaming asset dependencies
	UE_DEPRECATED(5.6, "Use GetStreamingAssetInstallModes instead of creating new bundles for streaming assets.")
	virtual TValueOrError<TArray<FName>, FString> GetStreamingAssetInstallBundles(FStringView PluginURL) const { return MakeValue(); }

	// Called to resolve install modes for streaming asset dependencies
	// Return a streaming asset install mode for each install bundle
	UE_API virtual TValueOrError<TArray<EStreamingAssetInstallMode>, FString> GetStreamingAssetInstallModes(FStringView PluginURL, TConstArrayView<FName> InstallBundleNames) const;

	// Called by code that explicitly wants to load a specific plugin
	// (e.g., when using a fast cook a game might want to allow explicitly loaded game feature plugins)
	UE_API virtual void ExplicitLoadGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate, const bool bActivateGameFeatures);

	// Called to determine if async loading is allowed
	UE_API virtual bool AllowAsyncLoad(FStringView PluginURL) const;

	/**
	 * Returns the install bundle name if one exists for this plugin.
	 * @param - PluginName - the name of the GameFeaturePlugin we want to get a bundle for. Should be the same name as the .uplugin file
	 * @param - bEvenIfDoesntExist - when true will return the name of bundle we are looking for without checking if it exists or not.
	 */
	UE_API virtual FString GetInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist = false);

	/**
	 * Returns the optional install bundle name if one exists for this plugin.
	 * @param - PluginName - the name of the GameFeaturePlugin we want to get a bundle for. Should be the same name as the .uplugin file
	 * @param - bEvenIfDoesntExist - when true will return the name of bundle we are looking for without checking if it exists or not.
	 */
	UE_API virtual FString GetOptionalInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist = false);

protected:
	/** This mutex can be used by derived classes for broad mutual exclusion */
	UE::FTransactionallySafeRecursiveMutex Mutex;
};

// This is a default implementation that immediately processes all game feature plugins the based on
// their BuiltInAutoRegister, BuiltInAutoLoad, and BuiltInAutoActivate settings.
//
// It will be used if no project-specific policy is set in Project Settings .. Game Features
UCLASS(MinimalAPI)
class UDefaultGameFeaturesProjectPolicies : public UGameFeaturesProjectPolicies
{
	GENERATED_BODY()

public:
	//~UGameFeaturesProjectPolicies interface
	UE_API virtual void InitGameFeatureManager() override;
	UE_API virtual void GetGameFeatureLoadingMode(bool& bLoadClientData, bool& bLoadServerData) const override;
	UE_API virtual const TArray<FName> GetPreloadBundleStateForPlugin(const FString& PluginName) const override;
	//~End of UGameFeaturesProjectPolicies interface
};

#undef UE_API
