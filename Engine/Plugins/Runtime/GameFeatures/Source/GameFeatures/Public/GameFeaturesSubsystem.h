// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Union.h"
#include "Delegates/Delegate.h"
#include "Engine/Engine.h"
#include "GameFeatureTypesFwd.h"
#include "GameFeaturePluginOperationResult.h"
#include "JsonUtils/RapidJsonUtils.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Stats/StatsMisc.h"
#include "Templates/PimplPtr.h"

#include "GameFeaturesSubsystem.generated.h"

#define UE_API GAMEFEATURES_API

namespace UE::GameFeatures
{
	class FPackageLoadTracker;
	class FRealtimeMode;
	class FStateMachineExecutor;
	struct FResult;

	bool TickUnloadPluginAssets(float DeltaTime);
}

class UGameFeaturePluginStateMachine;
class IGameFeatureStateChangeObserver;
struct FStreamableHandle;
struct FAssetIdentifier;
class UGameFeatureData;
class UGameFeaturesProjectPolicies;
class IPlugin;
class FJsonObject;
struct FWorldContext;
struct FGameFeaturePluginStateInfo;
struct FGameFeaturePluginStateRange;
struct FGameFeaturePluginStateMachineProperties;
struct FGameFeatureProtocolMetadata;
struct FInstallBundleReleaseRequestInfo;
enum class EInstallBundleResult : uint32;
enum class EInstallBundleRequestFlags : uint32;
enum class EInstallBundleReleaseRequestFlags : uint32;

namespace GameFeaturesSubsystem
{
	struct FNotifyObservers;
}

/** Holds static global information about how our PluginURLs are structured */
namespace UE::GameFeatures
{
	namespace PluginURLStructureInfo
	{
		/** Character used to denote what value is being assigned to the option before it */
		extern GAMEFEATURES_API const TCHAR* OptionAssignOperator;

		/** Character used to separate options on the URL. Used between each assigned value and the next Option name. */
		extern GAMEFEATURES_API const TCHAR* OptionSeperator;

		/** Character used to separate lists of values for a single option. Used between each entry in the list. */
		extern GAMEFEATURES_API const TCHAR* OptionListSeperator;
	};

	namespace CommonErrorCodes
	{
		extern const TCHAR* DependencyFailedRegister;
	};
};

/** 
 * Struct that determines if game feature action state changes should be applied for cases where there are multiple worlds or contexts.
 * The default value means to apply to all possible objects. This can be safely copied and used for later querying.
 */
struct FGameFeatureStateChangeContext
{
public:

	/** Sets a specific world context handle to limit changes to */
	UE_API void SetRequiredWorldContextHandle(FName Handle);

	/** Sees if the specific world context matches the application rules */
	UE_API bool ShouldApplyToWorldContext(const FWorldContext& WorldContext) const;

	/** True if events bound using this context should apply when using other context */
	UE_API bool ShouldApplyUsingOtherContext(const FGameFeatureStateChangeContext& OtherContext) const;

	/** Check if this has the exact same state change application rules */
	inline bool operator==(const FGameFeatureStateChangeContext& OtherContext) const
	{
		if (OtherContext.WorldContextHandle == WorldContextHandle)
		{
			return true;
		}

		return false;
	}

	/** Allow this to be used as a map key */
	inline friend uint32 GetTypeHash(const FGameFeatureStateChangeContext& OtherContext)
	{
		return GetTypeHash(OtherContext.WorldContextHandle);
	}

private:
	/** Specific world context to limit changes to, if none then it will apply to all */
	FName WorldContextHandle;
};

/** Context that provides extra information for activating a game feature */
struct FGameFeatureActivatingContext : public FGameFeatureStateChangeContext
{
public:
	//@TODO: Add rules specific to activation when required

private:

	friend struct FGameFeaturePluginState_Activating;
};

/** Context that provides extra information for deactivating a game feature, will use the same change context rules as the activating context */
struct FGameFeatureDeactivatingContext : public FGameFeatureStateChangeContext
{
public:
	// Call this if your observer has an asynchronous action to complete as part of shutdown, and invoke the returned delegate when you are done (on the game thread!)
	GAMEFEATURES_API FSimpleDelegate PauseDeactivationUntilComplete(FString InPauserTag);

	FGameFeatureDeactivatingContext(FStringView InPluginName, TFunction<void(FStringView InPauserTag)>&& InCompletionCallback)
		: PluginName(InPluginName)
		, CompletionCallback(MoveTemp(InCompletionCallback))
	{
	}

	int32 GetNumPausers() const { return NumPausers; }
private:
	FStringView PluginName;
	TFunction<void(FStringView InPauserTag)> CompletionCallback;
	int32 NumPausers = 0;

	friend struct FGameFeaturePluginState_Deactivating;
};

/** Context that provides extra information for a game feature changing its pause state */
struct FGameFeaturePauseStateChangeContext : public FGameFeatureStateChangeContext
{
public:
	FGameFeaturePauseStateChangeContext(FString PauseStateNameIn, FString PauseReasonIn, bool bIsPausedIn)
		: PauseStateName(MoveTemp(PauseStateNameIn))
		, PauseReason(MoveTemp(PauseReasonIn))
		, bIsPaused(bIsPausedIn)
	{
	}

	/** Returns true if the State has paused or false if it is resuming */
	bool IsPaused() const { return bIsPaused; }

	/** Returns an FString description of why the state has paused work. */
	const FString& GetPauseReason() const { return PauseReason; }

	/** Returns an FString description of what state has issued the pause change */
	const FString& GetPausingStateName() const { return PauseStateName; }

private:
	FString PauseStateName;
	FString PauseReason;
	bool bIsPaused = false;
};


/** Context that provides extra information prior to mounting a plugin */
struct FGameFeaturePreMountingContext : public FGameFeatureStateChangeContext
{
public:
	bool bOpenPluginShaderLibrary = true;

private:

	friend struct FGameFeaturePluginState_Mounting;
};

/** Context that allows pausing prior to transitioning out of the mounting state */
struct FGameFeaturePostMountingContext : public FGameFeatureStateChangeContext
{
public:
	// Call this if your observer has an asynchronous action to complete prior to transitioning out of the mounting state
	// and invoke the returned delegate when you are done (on the game thread!)
	GAMEFEATURES_API FSimpleDelegate PauseUntilComplete(FString InPauserTag);

	FGameFeaturePostMountingContext(FStringView InPluginName, TFunction<void(FStringView InPauserTag)>&& InCompletionCallback)
		: PluginName(InPluginName)
		, CompletionCallback(MoveTemp(InCompletionCallback))
	{}

	int32 GetNumPausers() const { return NumPausers; }

private:
	FStringView PluginName;
	TFunction<void(FStringView InPauserTag)> CompletionCallback;
	int32 NumPausers = 0;

	friend struct FGameFeaturePluginState_Mounting;
};

GAMEFEATURES_API DECLARE_LOG_CATEGORY_EXTERN(LogGameFeatures, Log, All);
/** Notification that a game feature plugin install/register/load/unload has finished */
DECLARE_DELEGATE_OneParam(FGameFeaturePluginChangeStateComplete, const UE::GameFeatures::FResult& /*Result*/);

/** A request to update the state machine and process states */
DECLARE_DELEGATE(FGameFeaturePluginRequestUpdateStateMachine);
DECLARE_MULTICAST_DELEGATE(FNotifyGameFeaturePluginRequestUpdateStateMachine)

using FGameFeaturePluginLoadComplete = FGameFeaturePluginChangeStateComplete;
using FGameFeaturePluginDeactivateComplete = FGameFeaturePluginChangeStateComplete;
using FGameFeaturePluginUnloadComplete = FGameFeaturePluginChangeStateComplete;
using FGameFeaturePluginReleaseComplete = FGameFeaturePluginChangeStateComplete;
using FGameFeaturePluginUninstallComplete = FGameFeaturePluginChangeStateComplete;
using FGameFeaturePluginTerminateComplete = FGameFeaturePluginChangeStateComplete;
using FGameFeaturePluginUpdateProtocolComplete = FGameFeaturePluginChangeStateComplete;

using FMultipleGameFeaturePluginChangeStateComplete = TDelegate<void(const TMap<FString, UE::GameFeatures::FResult>& Results)>;

using FBuiltInGameFeaturePluginsLoaded = FMultipleGameFeaturePluginChangeStateComplete;
using FMultipleGameFeaturePluginsLoaded = FMultipleGameFeaturePluginChangeStateComplete;
using FMultipleGameFeaturePluginsDeactivate = FMultipleGameFeaturePluginChangeStateComplete;
using FMultipleGameFeaturePluginsUnload = FMultipleGameFeaturePluginChangeStateComplete;
using FMultipleGameFeaturePluginsReleased = FMultipleGameFeaturePluginChangeStateComplete;
using FMultipleGameFeaturePluginsTerminated = FMultipleGameFeaturePluginChangeStateComplete;

enum class EBuiltInAutoState : uint8
{
	Invalid,
	Installed,
	Registered,
	Loaded,
	Active
};
const FString GAMEFEATURES_API LexToString(const EBuiltInAutoState BuiltInAutoState);

UENUM(BlueprintType)
enum class EGameFeatureTargetState : uint8
{
	Installed,
	Registered,
	Loaded,
	Active,
	Count	UMETA(Hidden)
};
const FString GAMEFEATURES_API LexToString(const EGameFeatureTargetState GameFeatureTargetState);
void GAMEFEATURES_API LexFromString(EGameFeatureTargetState& Value, const TCHAR* StringIn);

// TODO: clean this up in 5.10 because we can't neatly deprecate it without breaking external projects
// So the Editor has to have this optimization disabled until 5.10 can enable it by default
#ifndef UE_GAME_FEATURE_PLUGIN_DETAILS_USE_UTF8_STRING
//	Do not trigger deprecated warning for Editor builds, only for clients
#	if !WITH_EDITOR
	UE_DEPRECATED(5.8, "Please set UE_GAME_FEATURE_PLUGIN_DETAILS_USE_UTF8_STRING to 1 in your project settings and migrate to UTF8 implementation")
#	endif
	typedef FString FPluginString;
#	define UE_GAME_FEATURE_PLUGIN_DETAILS_USE_UTF8_STRING 0
#endif

#if UE_GAME_FEATURE_PLUGIN_DETAILS_USE_UTF8_STRING
	typedef FUtf8String FPluginString;
#	define PLUGIN_TEXT(x) UTF8TEXT(x)
#	define TO_PLUGIN_STRING(x) FUtf8String(x)
#else
#	define PLUGIN_TEXT(x) TEXT(x)
#	define TO_PLUGIN_STRING(x) x
#endif

struct FGameFeaturePluginReferenceDetails
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPluginString PluginName;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	bool bShouldActivate = false;
};

struct FGameFeaturePluginDetails
{
	TArray<FGameFeaturePluginReferenceDetails> PluginDependencies;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<FPluginString, TSharedPtr<class FJsonValue>> AdditionalMetadata;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	bool bHotfixable = false;
	EBuiltInAutoState BuiltInAutoState = EBuiltInAutoState::Invalid;
};

struct FBuiltInGameFeaturePluginBehaviorOptions
{
	EBuiltInAutoState AutoStateOverride = EBuiltInAutoState::Invalid;

	/** Maximum state this plugin is allowed to reach */
	EGameFeatureTargetState MaximumState = EGameFeatureTargetState::Active;

	/** Force this GFP to load synchronously even if async loading is allowed */
	bool bForceSyncLoading = false;

	/** Batch process GFPs if/when possible (could be used when processing multiple plugins)*/
	bool bBatchProcess = false;

	/** Disallows downloading, useful for conditionally loading content only if it's already been installed */
	bool bDoNotDownload = false;

	/** Log Warning if loading this GFP forces dependencies to be created, useful for catching GFP load filtering bugs */
	bool bLogWarningOnForcedDependencyCreation = false;

	/** Log Error if loading this GFP forces dependencies to be created, useful for catching GFP load filtering bugs */
	bool bLogErrorOnForcedDependencyCreation = false;
};

struct FGameFeaturePluginAsyncHandle : public TSharedFromThis<FGameFeaturePluginAsyncHandle>
{
	virtual ~FGameFeaturePluginAsyncHandle() = default;
	virtual bool IsComplete() const = 0;
	virtual const UE::GameFeatures::FResult& GetResult() const = 0;
	virtual float GetProgress() const = 0;
	virtual void Cancel() = 0;
};

/** Handle to track a GFP predownload */
struct FGameFeaturePluginPredownloadHandle : public FGameFeaturePluginAsyncHandle
{
};

/** Struct used to transform a GameFeaturePlugin URL into something that can uniquely identify the GameFeaturePlugin
    without including any transient data being passed in through the URL */
USTRUCT()
struct FGameFeaturePluginIdentifier
{
	GENERATED_BODY()

	FGameFeaturePluginIdentifier() = default;
	UE_API explicit FGameFeaturePluginIdentifier(FString PluginURL);

	FGameFeaturePluginIdentifier(const FGameFeaturePluginIdentifier& Other)
		: FGameFeaturePluginIdentifier(Other.PluginURL)
	{}

	UE_API FGameFeaturePluginIdentifier(FGameFeaturePluginIdentifier&& Other);

	FGameFeaturePluginIdentifier& operator=(const FGameFeaturePluginIdentifier& Other)
	{
		FromPluginURL(Other.PluginURL);
		return *this;
	}

	UE_API FGameFeaturePluginIdentifier& operator=(FGameFeaturePluginIdentifier&& Other);

	/** Used to determine if 2 FGameFeaturePluginIdentifiers are referencing the same GameFeaturePlugin.
		Only matching on Identifying information instead of all the optional bundle information */
	UE_API bool operator==(const FGameFeaturePluginIdentifier& Other) const;

	/** Function that fills out IdentifyingURLSubset from the given PluginURL */
	UE_API void FromPluginURL(FString PluginURL);

	/** Returns true if this FGameFeaturePluginIdentifier exactly matches the given PluginURL.
		To match exactly all information in the PluginURL has to match and not just the IdentifyingURLSubset */
	UE_API bool ExactMatchesURL(const FString& PluginURL) const;

	EGameFeaturePluginProtocol GetPluginProtocol() const { return PluginProtocol; }

	/** Returns the Identifying information used for this Plugin. It is a subset of the URL used to create it.*/
	FStringView GetIdentifyingString() const { return IdentifyingURLSubset; }

	/** Returns the name of the plugin */
	UE_API FStringView GetPluginName() const;

	/** Get the Full PluginURL used to originally construct this identifier */
	const FString& GetFullPluginURL() const { return PluginURL; }

	friend inline uint32 GetTypeHash(const FGameFeaturePluginIdentifier& PluginIdentifier)
	{
		return GetTypeHash(PluginIdentifier.IdentifyingURLSubset);
	}

private:
	/** Full PluginURL used to originally construct this identifier */
	FString PluginURL;

	/** The part of the URL that can be used to uniquely identify this plugin without any transient data */
	FStringView IdentifyingURLSubset;

	/** The protocol used in the URL for this GameFeaturePlugin URL */
	EGameFeaturePluginProtocol PluginProtocol;

	//Friend class so that it can access parsed URL data from under the hood
	friend struct FGameFeaturePluginStateMachineProperties;
};

USTRUCT()
struct FInstallBundlePluginProtocolOptions
{
	GENERATED_BODY()

	UE_API FInstallBundlePluginProtocolOptions();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FInstallBundlePluginProtocolOptions(const FInstallBundlePluginProtocolOptions&) = default;
	FInstallBundlePluginProtocolOptions& operator=(const FInstallBundlePluginProtocolOptions&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** EInstallBundleRequestFlags utilized during the download/install by InstallBundleManager */
	EInstallBundleRequestFlags InstallBundleFlags;

	/** EInstallBundleReleaseRequestFlags utilized during our release and uninstall states */
	UE_DEPRECATED(5.6, "Release flags are now applied internally and no longer need to be explicitly set.")
	EInstallBundleReleaseRequestFlags ReleaseInstallBundleFlags;

	/** If we want to attempt to uninstall InstallBundle data installed by this plugin before terminating */
	bool bUninstallBeforeTerminate : 1;

	/** If we want to set the Downloading state to pause because of user interaction */
	bool bUserPauseDownload : 1;

	/** Allow the GFP to load INI files, should only be allowed for trusted content */
	bool bAllowIniLoading : 1;

	/** Disallows downloading, useful for conditionally loading content only if it's already been installed */
	bool bDoNotDownload : 1;

	UE_API bool operator==(const FInstallBundlePluginProtocolOptions& Other) const;
};

struct FGameFeatureProtocolOptions : public TUnion<FInstallBundlePluginProtocolOptions, FNull>
{
	GAMEFEATURES_API FGameFeatureProtocolOptions();
	GAMEFEATURES_API explicit FGameFeatureProtocolOptions(const FInstallBundlePluginProtocolOptions& InOptions);
	GAMEFEATURES_API explicit FGameFeatureProtocolOptions(FNull InOptions);

	bool operator==(const FGameFeatureProtocolOptions& Other) const
	{
		return TUnion<FInstallBundlePluginProtocolOptions, FNull>::operator==(Other) &&
			bForceSyncLoading == Other.bForceSyncLoading &&
			bBatchProcess == Other.bBatchProcess &&
			bLogWarningOnForcedDependencyCreation == Other.bLogWarningOnForcedDependencyCreation &&
			bLogErrorOnForcedDependencyCreation == Other.bLogErrorOnForcedDependencyCreation;
	}

	/** Force this GFP to load synchronously even if async loading is allowed */
	bool bForceSyncLoading : 1;

	/** Batch process GFPs if/when possible (could be used when processing multiple plugins)*/
	bool bBatchProcess : 1;

	/** Log Warning if loading this GFP forces dependencies to be created, useful for catching GFP load filtering bugs */
	bool bLogWarningOnForcedDependencyCreation : 1;

	/** Log Error if loading this GFP forces dependencies to be created, useful for catching GFP load filtering bugs */
	bool bLogErrorOnForcedDependencyCreation : 1;
};

// some important information about a gamefeature
struct FGameFeatureInfo
{
	FString Name;
	FString URL;
	bool bLoadedAsBuiltIn;
	EGameFeaturePluginState CurrentState;
};

/** The manager subsystem for game features */
UCLASS(MinimalAPI)
class UGameFeaturesSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	UGameFeaturesSubsystem(const FObjectInitializer& ObjectInitializer);

	//~UEngineSubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~End of UEngineSubsystem interface

	static UGameFeaturesSubsystem& Get() { return *GEngine->GetEngineSubsystem<UGameFeaturesSubsystem>(); }

	UE_API static FSimpleMulticastDelegate OnGameFeaturePolicyPreInit;
	UE_API static FSimpleMulticastDelegate OnGameFeaturePolicyPostInit;

public:
	/** Loads the specified game feature data and its bundles */
	static UE_API TSharedPtr<FStreamableHandle> LoadGameFeatureData(const FString& GameFeatureToLoad, bool bStartStalled = false);
	static UE_API void UnloadGameFeatureData(const UGameFeatureData* GameFeatureToUnload);

	UE_DEPRECATED(5.7, "Use AddObserver with EObserverPluginStateUpdateMode parameter")
	UE_API void AddObserver(UObject* Observer);

	enum class EObserverPluginStateUpdateMode
	{
		FutureOnly, //Only call the observer with future plugin state changes
		CurrentAndFuture, //Also update the observer with the current plugin state at add time - note, if project has a lot of plugins this can be expensive
	};

	//Add Observer to be notified when plugin states change. Observer must implement IGameFeatureStateChangeObserver and be non-null
	UE_API void AddObserver(UObject* Observer, const EObserverPluginStateUpdateMode UpdateMode);
	UE_API void RemoveObserver(UObject* Observer);

	UE_API void ForEachGameFeature(TFunctionRef<void(FGameFeatureInfo&&)> Visitor) const;

	/**
	 * Calls the compile-time lambda on each active game feature data of the specified type
	 * @param GameFeatureDataType       The kind of data required
	 */
	template<class GameFeatureDataType, typename Func>
	void ForEachActiveGameFeature(Func InFunc) const
	{
		for (auto StateMachineIt = GameFeaturePluginStateMachines.CreateConstIterator(); StateMachineIt; ++StateMachineIt)
		{
			if (UGameFeaturePluginStateMachine* GFSM = StateMachineIt.Value())
			{
				if (const GameFeatureDataType* GameFeatureData = Cast<const GameFeatureDataType>(GetDataForStateMachine(GFSM)))
				{
					InFunc(GameFeatureData);
				}
			}
		}
	}

	/**
	 * Calls the compile-time lambda on each registered game feature data of the specified type
	 * @param GameFeatureDataType       The kind of data required
	 */
	template<class GameFeatureDataType, typename Func>
	void ForEachRegisteredGameFeature(Func InFunc) const
	{
		for (auto StateMachineIt = GameFeaturePluginStateMachines.CreateConstIterator(); StateMachineIt; ++StateMachineIt)
		{
			if (UGameFeaturePluginStateMachine* GFSM = StateMachineIt.Value())
			{
				if (const GameFeatureDataType* GameFeatureData = Cast<const GameFeatureDataType>(GetRegisteredDataForStateMachine(GFSM)))
				{
					InFunc(GameFeatureData);
				}
			}
		}
	}

public:
	/**
	  * Construct a 'file:' Plugin URL using from the PluginDescriptorPath 
	  * THREAD SAFETY: Any thread
	  */
	static UE_API FString GetPluginURL_FileProtocol(const FString& PluginDescriptorPath);
	static UE_API FString GetPluginURL_FileProtocol(const FString& PluginDescriptorPath, TArrayView<const TPair<FString, FString>> AdditionalOptions);
	
	/** Get the file path portion of a file protocol URL*/
	static UE_API FString GetPluginFilename_FileProtocol(const FString& PluginUrlFileProtocol);

	/** Construct a 'installbundle:' Plugin URL using from the PluginName and required install bundles */
	static UE_API FString GetPluginURL_InstallBundleProtocol(const FString& PluginName, TArrayView<const FString> BundleNames);
	static UE_API FString GetPluginURL_InstallBundleProtocol(const FString& PluginName, const FString& BundleName);
	static UE_API FString GetPluginURL_InstallBundleProtocol(const FString& PluginName, TArrayView<const FName> BundleNames);
	static UE_API FString GetPluginURL_InstallBundleProtocol(const FString& PluginName, FName BundleName);
	static UE_API FString GetPluginURL_InstallBundleProtocol(const FString& PluginName, TArrayView<const FName> BundleNames, TArrayView<const TPair<FString, FString>> AdditionalOptions);

	/** Returns the plugin protocol for the specified URL */
	static UE_API EGameFeaturePluginProtocol GetPluginURLProtocol(FStringView PluginURL);

	/** Tests whether the plugin URL is the specified protocol */
	static UE_API bool IsPluginURLProtocol(FStringView PluginURL, EGameFeaturePluginProtocol PluginProtocol);

	/** 
	  * Parse the plugin URL into subparts 
	  * THREAD SAFETY: Any thread
	  */
	static UE_API bool ParsePluginURL(FStringView PluginURL, EGameFeaturePluginProtocol* OutProtocol = nullptr, FStringView* OutPath = nullptr, FStringView* OutOptions = nullptr);

	/** Parse options from a plugin URL or the options subpart of the plugin URL */
	static UE_API bool ParsePluginURLOptions(FStringView URLOptionsString,
		TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output);
	static UE_API bool ParsePluginURLOptions(FStringView URLOptionsString, EGameFeatureURLOptions OptionsFlags,
		TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output);
	static UE_API bool ParsePluginURLOptions(FStringView URLOptionsString, TConstArrayView<FStringView> AdditionalOptions,
		TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output);
	static UE_API bool ParsePluginURLOptions(FStringView URLOptionsString, EGameFeatureURLOptions OptionsFlags, TConstArrayView<FStringView> AdditionalOptions,
		TFunctionRef<void(EGameFeatureURLOptions Option, FStringView OptionString, FStringView OptionValue)> Output);


public:
	/** Returns all the active plugins GameFeatureDatas */
	UE_API void GetGameFeatureDataForActivePlugins(TArray<const UGameFeatureData*>& OutActivePluginFeatureDatas);

	/** Returns the game feature data for an active plugin specified by PluginURL */
	UE_API const UGameFeatureData* GetGameFeatureDataForActivePluginByURL(const FString& PluginURL);

	/** Returns the game feature data for a registered plugin specified by PluginURL */
	UE_API const UGameFeatureData* GetGameFeatureDataForRegisteredPluginByURL(const FString& PluginURL, bool bCheckForRegistering = false);

	/** Determines if a plugin is in an error state */
	UE_API bool IsGameFeaturePluginInErrorState(const FString& PluginURL) const;

	/** Determines if a plugin is in the Installed state (or beyond) */
	UE_API bool IsGameFeaturePluginInstalled(const FString& PluginURL) const;

	/** Determines if a plugin is beyond the Mounting state */
	UE_API bool IsGameFeaturePluginMounted(const FString& PluginURL) const;

	/** Determines if a plugin is in the Registered state (or beyond) */
	UE_API bool IsGameFeaturePluginRegistered(const FString& PluginURL, bool bCheckForRegistering = false) const;

	/** Determines if a plugin is in the Loaded state (or beyond) */
	UE_API bool IsGameFeaturePluginLoaded(const FString& PluginURL) const;

	/** Was this game feature plugin loaded using the LoadBuiltInGameFeaturePlugin path */
	UE_API bool WasGameFeaturePluginLoadedAsBuiltIn(const FString& PluginURL) const;

	/** Loads a single game feature plugin. */
	UE_API void LoadGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate);
	UE_API void LoadGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginLoadComplete& CompleteDelegate);
	UE_API void LoadGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate);

	/** Registers a single game feature plugin. */
	UE_API void RegisterGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate);
	UE_API void RegisterGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginLoadComplete& CompleteDelegate);
	UE_API void RegisterGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate);

	/** Loads a single game feature plugin and activates it. */
	UE_API void LoadAndActivateGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginLoadComplete& CompleteDelegate);
	UE_API void LoadAndActivateGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginLoadComplete& CompleteDelegate);
	UE_API void LoadAndActivateGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate);

	/** Changes the target state of a game feature plugin */
	UE_API void ChangeGameFeatureTargetState(const FString& PluginURL, EGameFeatureTargetState TargetState, const FGameFeaturePluginChangeStateComplete& CompleteDelegate);
	UE_API void ChangeGameFeatureTargetState(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, EGameFeatureTargetState TargetState, const FGameFeaturePluginChangeStateComplete& CompleteDelegate);
	UE_API void ChangeGameFeatureTargetState(TConstArrayView<FString> PluginURLs, const FGameFeatureProtocolOptions& ProtocolOptions, EGameFeatureTargetState TargetState, const FMultipleGameFeaturePluginsLoaded& CompleteDelegate);

	/** Changes the protocol options of a game feature plugin. Useful to change any options data such as settings flags */
	UE_API UE::GameFeatures::FResult UpdateGameFeatureProtocolOptions(const FString& PluginURL, const FGameFeatureProtocolOptions& NewOptions, bool* bOutDidUpdate = nullptr);

	/** Gets the Install_Percent for single game feature plugin if it is active. */
	UE_API bool GetGameFeaturePluginInstallPercent(const FString& PluginURL, float& Install_Percent) const;
	UE_API bool GetGameFeaturePluginInstallPercent(TConstArrayView<FString> PluginURLs, float& Install_Percent) const;

	/** Determines if a plugin is in the Active state.*/
	UE_API bool IsGameFeaturePluginActive(const FString& PluginURL, bool bCheckForActivating = false) const;

	/** Determines if a plugin is in the Active state.*/
	UE_API bool IsGameFeaturePluginActiveByName(FStringView PluginName, bool bCheckForActivating = false) const;

	/** Determines if a plugin is up to date or needs an update. Returns true if an update is available.*/
	UE_API bool DoesGameFeaturePluginNeedUpdate(const FString& PluginURL) const;

	/** Deactivates the specified plugin. Transition to [Terminal, Loaded] */
	UE_API void DeactivateGameFeaturePlugin(const FString& PluginURL);
	UE_API void DeactivateGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginDeactivateComplete& CompleteDelegate);
	UE_API void DeactivateGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsDeactivate& CompleteDelegate);

	/** Unloads the specified game feature plugin. Transition to bKeepRegistered = false -> [Terminal, Installed], bKeepRegistered = true -> [Terminal, Registered]*/
	UE_API void UnloadGameFeaturePlugin(const FString& PluginURL, bool bKeepRegistered = false);
	UE_API void UnloadGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginUnloadComplete& CompleteDelegate, bool bKeepRegistered = false);
	UE_API void UnloadGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsUnload& CompleteDelegate, bool bKeepRegistered = false);

	/** Releases any game data stored for this GameFeaturePlugin. Does not uninstall data and it will remain on disk. [Terminal, StatusKnown] */
	UE_API void ReleaseGameFeaturePlugin(const FString& PluginURL);
	UE_API void ReleaseGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginReleaseComplete& CompleteDelegate);
	UE_API void ReleaseGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsReleased& CompleteDelegate);

	/** Uninstalls any game data stored for this GameFeaturePlugin and terminates the GameFeaturePlugin.
		If the given PluginURL is not found this will create a GameFeaturePlugin first and attempt to run it through the uninstall flow.
		This allows for the uninstalling of data that was installed on previous runs of the application where we haven't yet requested the
		GameFeaturePlugin that we would like to uninstall data for on this run. */
	UE_API void UninstallGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginUninstallComplete& CompleteDelegate = FGameFeaturePluginUninstallComplete());
	UE_API void UninstallGameFeaturePlugin(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginUninstallComplete& CompleteDelegate = FGameFeaturePluginUninstallComplete());

	/** Terminate the GameFeaturePlugin and remove all associated plugin tracking data. Transition to [Terminal, Terminal] */
	UE_API void TerminateGameFeaturePlugin(const FString& PluginURL);
	UE_API void TerminateGameFeaturePlugin(const FString& PluginURL, const FGameFeaturePluginTerminateComplete& CompleteDelegate);
	UE_API void TerminateGameFeaturePlugin(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginsTerminated& CompleteDelegate);

	/** Attempt to cancel any state change. Calls back when cancelation is complete. Any other pending callbacks will be called with a canceled error. */
	UE_API void CancelGameFeatureStateChange(const FString& PluginURL);
	UE_API void CancelGameFeatureStateChange(const FString& PluginURL, const FGameFeaturePluginChangeStateComplete& CompleteDelegate);
	UE_API void CancelGameFeatureStateChange(TConstArrayView<FString> PluginURLs, const FMultipleGameFeaturePluginChangeStateComplete& CompleteDelegate);

	/**
	 * If the specified plugin is known by the game feature system, returns the URL used to identify it
	 * @return true if the plugin exists, false if it was not found
	 * THREAD SAFETY: Any thread as long as GameSpecificPolicies meets version EGameFeaturesProjectPoliciesThreadSafetyVersion::Initial
	 */
	UE_API bool GetPluginURLByName(FStringView PluginName, FString& OutPluginURL) const;
	UE_API bool GetPluginURLByName(FUtf8StringView PluginName, FString& OutPluginURL) const;

	/** Get the plugin path from the plugin URL */
	UE_API FString GetPluginFilenameFromPluginURL(const FString& PluginURL) const;

	/** Get the plugin name from the plugin URL */
	UE_API FString GetPluginNameFromPluginURL(const FString& PluginURL) const;

	/** Fixes a package path/directory to either be relative to plugin root or not. Paths relative to different roots will not be modified */
	static UE_API void FixPluginPackagePath(FString& PathToFix, const FString& PluginRootPath, bool bMakeRelativeToPluginRoot);

	/** Returns the game-specific policy for managing game feature plugins */
	template <typename T = UGameFeaturesProjectPolicies>
	T& GetPolicy() const
	{
		ensureMsgf(bInitializedPolicyManager, TEXT("Attemting to get policy before GameFeaturesSubsystem is ready!"));
		return *CastChecked<T>(GameSpecificPolicies, ECastCheckedType::NullChecked);
	}

	typedef TFunctionRef<bool(const FString& PluginFilename, const FGameFeaturePluginDetails& Details, FBuiltInGameFeaturePluginBehaviorOptions& OutOptions)> FBuiltInPluginAdditionalFilters;
	typedef TFunction<bool(const FString& PluginFilename, const FGameFeaturePluginDetails& Details, FBuiltInGameFeaturePluginBehaviorOptions& OutOptions)> FBuiltInPluginAdditionalFilters_Copyable;

	/** Loads a built-in game feature plugin if it passes the specified filter */
	UE_API void LoadBuiltInGameFeaturePlugin(const TSharedRef<IPlugin>& Plugin, FBuiltInPluginAdditionalFilters AdditionalFilter, const FGameFeaturePluginLoadComplete& CompleteDelegate = FGameFeaturePluginLoadComplete());

	/** Loads all built-in game feature plugins that pass the specified filters */
	UE_API void LoadBuiltInGameFeaturePlugins(FBuiltInPluginAdditionalFilters AdditionalFilter, const FBuiltInGameFeaturePluginsLoaded& CompleteDelegate = FBuiltInGameFeaturePluginsLoaded());

	/** Loads all built-in game feature plugins that pass the specified filters, split over multiple frames processing only AmortizeRate plugins per frame if greater than 0. Note that AdditionalFilter must be a TFunction and not a TFunctionRef since it will be used in future ticks */
	UE_API void LoadBuiltInGameFeaturePlugins_Amortized(const FBuiltInPluginAdditionalFilters_Copyable& AdditionalFilter_Copyable, int32 AmortizeRate, const FBuiltInGameFeaturePluginsLoaded& CompleteDelegate = FBuiltInGameFeaturePluginsLoaded());
	UE_API void LoadBuiltInGameFeaturePlugins_Amortized(const TArray<TSharedRef<IPlugin>>& Plugins, const FBuiltInPluginAdditionalFilters_Copyable& AdditionalFilter, int32 AmortizeRate, const FBuiltInGameFeaturePluginsLoaded& CompleteDelegate = FBuiltInGameFeaturePluginsLoaded());
private:
	TArray<TSharedRef<IPlugin>> GetInternalEnabledPlugins() const;
	void LoadBuiltInGameFeaturePluginsInternal(TArray<TSharedRef<IPlugin>> Plugins, FBuiltInPluginAdditionalFilters AdditionalFilter, const FBuiltInPluginAdditionalFilters_Copyable& AdditionalFilter_Copyable, int32 AmortizeRate, const FBuiltInGameFeaturePluginsLoaded& CompleteDelegate = FBuiltInGameFeaturePluginsLoaded());
public:

	/** Returns the list of plugin filenames that have progressed beyond installed. Used in cooking to determine which will be cooked. */
	//@TODO: GameFeaturePluginEnginePush: Might not be general enough for engine level, TBD
	UE_API void GetLoadedGameFeaturePluginFilenamesForCooking(TArray<FString>& OutLoadedPluginFilenames) const;

	/** Removes assets that are in plugins we know to be inactive.  Order is not maintained. */
	UE_API void FilterInactivePluginAssets(TArray<FAssetIdentifier>& AssetsToFilter) const;

	/** Removes assets that are in plugins we know to be inactive.  Order is not maintained. */
	UE_API void FilterInactivePluginAssets(TArray<FAssetData>& AssetsToFilter) const;

	/** Returns the current state of the state machine for the specified plugin URL */
	UE_API EGameFeaturePluginState GetPluginState(const FString& PluginURL) const;

	/** Returns the current state of the state machine for the specified plugin PluginIdentifier */
	UE_API EGameFeaturePluginState GetPluginState(FGameFeaturePluginIdentifier PluginIdentifier) const;

	/** Gets relevant properties out of a uplugin file. Should only be used for built-in GFPs */
	UE_API bool GetBuiltInGameFeaturePluginDetails(const TSharedRef<IPlugin>& Plugin, struct FGameFeaturePluginDetails& OutPluginDetails) const;

	/** Returns a shared pointer to immutable cached data about the given plugin. Should only be used for built-in GFPs */
	UE_API TSharedPtr<const FGameFeaturePluginDetails> GetBuiltInGameFeaturePluginDetails(const TSharedRef<IPlugin>& Plugin) const;

	/** 
	  * Load and cache all details for all builtin game feature plugins 
	  * @param bForce If true, will try to load all details even if it has been done already. 
	  * 	Use if the GameSpecificPolicies has changed which plugins are allowed.
	  */
	UE_API void LoadAllBuiltInGameFeaturePluginDetails(bool bForce = false);

	/** Gets the URL for the given plugin, applying game-specific policies where appropriate. Should only be used for built-in GFPs */
	/** 
	  * Gets the URL for the given plugin, applying game-specific policies where appropriate. Should only be used for built-in GFPs 
	  * THREAD SAFETY: Any thread as long as GameSpecificPolicies meets version EGameFeaturesProjectPoliciesThreadSafetyVersion::Initial
	  */
	UE_API bool GetBuiltInGameFeaturePluginURL(const TSharedRef<IPlugin>& Plugin, FString& OutPluginURL) const;

	/** Gets relevant properties out of a uplugin file if it's installed */
	UE_API bool GetGameFeaturePluginDetails(const FString& PluginURL, struct FGameFeaturePluginDetails& OutPluginDetails) const;
	/**  Returns a shared pointer to immutable cached data about the given plugin if it's installed. */
	UE_API TSharedPtr<const FGameFeaturePluginDetails> GetGameFeaturePluginDetails(const FString& PluginURL) const;

	/** 
	 * Sets OutGameFeatureControlsUPlugin to true if the uplugin was added by a GFP as opposed to existing independent of the GFP subsystem.
	 */
	UE_API bool GetGameFeatureControlsUPlugin(const FString& PluginURL, bool& OutGameFeatureControlsUPlugin) const;

	/**
	 * THREAD SAFETY: Game thread only
	 */
	UE_API bool IsPluginAllowed(const FString& PluginURL, FString* OutReason = nullptr) const;

	/**
	 * THREAD SAFETY: Any thread
	 */
	UE_API bool IsPluginAllowed(const FString& PluginURL, const FGameFeaturePluginDetails* InDetails, FString* OutReason) const;

	/** 
	 * Pre-install any required game feature data, which can be useful for larger payloads. 
	 * This does not instantiate any GFP although it is safe to do so before this finishes. 
	 * This doesn't not resolve any dependencies, PluginURLs should contain all dependencies.
	 * If bWaitForBundlesToRelease is true, OnComplete will not be called until the bundles have been released
	 */
	UE_API TSharedRef<FGameFeaturePluginPredownloadHandle> PredownloadGameFeaturePlugins(TConstArrayView<FString> PluginURLs, 
		TUniqueFunction<void(const UE::GameFeatures::FResult&)> OnComplete = nullptr, TUniqueFunction<void(float)> OnProgress = nullptr, bool bWaitForBundlesToRelease = false);
	friend struct FGameFeaturePluginPredownloadContext;

	/** Determine the initial feature state for a built-in plugin */
	static UE_API EBuiltInAutoState DetermineBuiltInInitialFeatureState(TSharedPtr<FJsonObject> Descriptor, const FString& ErrorContext);
	static UE_API EBuiltInAutoState DetermineBuiltInInitialFeatureState(UE::Json::FConstObject Descriptor, const FString& ErrorContext);

	/**
	 * THREAD SAFETY: Any thread.
	 */
	static UE_API EGameFeaturePluginState ConvertInitialFeatureStateToTargetState(EBuiltInAutoState InitialState);

	/** Used during a DLC cook to determine which plugins should be cooked */
	static UE_API void GetPluginsToCook(TSet<FString>& OutPlugins);

	static UE_API const TSet<FString>& GetDlcPlugins();

	/**
	 * Returns true if plugin is a GameFeaturePlugin
	 * @param InPlugin The plugin to check
	 * @return True if plugin considered a GameFeaturePlugin
	 */
	static UE_API bool IsGameFeaturePlugin(const TSharedPtr<IPlugin>& InPlugin);

	UE_API bool GetPluginDebugStateEnabled(const FString& PluginUrl) const;
	UE_API void SetPluginDebugStateEnabled(const FString& PluginUrl, bool bEnabled);

	/**
	 * Returns the install bundle name if one exists for this plugin. Can be overridden by the policy provider.
	 * @param - PluginName - the name of the GameFeaturePlugin we want to get a bundle for. Should be the same name as the .uplugin file
	 * @param - bEvenIfDoesntExist - when true will return the name of bundle we are looking for without checking if it exists or not.
	 */
	UE_API FString GetInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist = false);

	/**
	 * Returns the optional install bundle name if one exists for this plugin. Can be overridden by the policy provider
	 * @param - PluginName - the name of the GameFeaturePlugin we want to get a bundle for. Should be the same name as the .uplugin file
	 * @param - bEvenIfDoesntExist - when true will return the name of bundle we are looking for without checking if it exists or not.
	 */
	UE_API FString GetOptionalInstallBundleName(FStringView PluginName, bool bEvenIfDoesntExist = false);

	/**
	 * Returns the required bundles for this plugin based upon the current state machine.
	 * Unlike GetInstallBundleName this will return all the bundles and dependencies for the given plugin.
	 * @param - PluginName - the name of the GameFeaturePlugin we want to get the bundles for.
	 * @param - OutBundles - Output reference array for the found bundles for the plugin.
	 * @return true if an existing state machine with install bundle protocol is found.
	 */
	UE_API bool GetBundlesForPlugin(FStringView PluginName, TArray<FName>& OutBundles);
	UE_API bool GetBundlesForPluginURL(FStringView PluginURL, TArray<FName>& OutBundles);

private:
	UE_API TSet<FString> GetActivePluginNames() const;

	UE_API void OnGameFeatureTerminating(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Terminal;

	UE_API void OnGameFeatureCheckingStatus(const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_UnknownStatus;

	UE_API void OnGameFeatureStatusKnown(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifierL);
	friend struct FGameFeaturePluginState_CheckingStatus;

	UE_API void OnGameFeaturePredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);
	UE_API void OnGameFeaturePostPredownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);

	UE_API void OnGameFeatureDownloading(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Downloading;

	UE_API void OnGameFeatureReleasing(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Releasing;

	UE_API void OnGameFeaturePreMounting(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier, FGameFeaturePreMountingContext& Context);
	UE_API void OnGameFeaturePostMounting(const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier, FGameFeaturePostMountingContext& Context);
	friend struct FGameFeaturePluginState_Mounting;

	UE_API void OnGameFeatureRegistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Registering;

	UE_API void OnGameFeatureUnregistering(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Unregistering;

	UE_API void OnGameFeatureActivating(const UGameFeatureData* GameFeatureData, const FString& PluginName, FGameFeatureActivatingContext& Context, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Activating;

	UE_API void OnGameFeatureActivated(const UGameFeatureData* GameFeatureData, const FString& PluginName, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Active;

	UE_API void OnGameFeatureDeactivating(const UGameFeatureData* GameFeatureData, const FString& PluginName, FGameFeatureDeactivatingContext& Context, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Deactivating;

	UE_API void OnGameFeatureLoading(const UGameFeatureData* GameFeatureData, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Loading;

	UE_API void OnGameFeatureUnloading(const UGameFeatureData* GameFeatureData, const FGameFeaturePluginIdentifier& PluginIdentifier);
	friend struct FGameFeaturePluginState_Unloading;

	UE_API void OnGameFeaturePauseChange(const FGameFeaturePluginIdentifier& PluginIdentifier, const FString& PluginName, FGameFeaturePauseStateChangeContext& Context);
	friend struct FGameFeaturePluginState_Deactivating;

	UE_API void OnAssetManagerCreated();

	/** Scans for assets specified in the game feature data */
	static UE_API void AddGameFeatureToAssetManager(const UGameFeatureData* GameFeatureToAdd, const FString& PluginName, TArray<FName>& OutNewPrimaryAssetTypes);

	static UE_API void RemoveGameFeatureFromAssetManager(const UGameFeatureData* GameFeatureToRemove, const FString& PluginName, const TArray<FName>& AddedPrimaryAssetTypes);

	// Provide additional causal information when a package is unavailable for load
	UE_API void GetExplanationForUnavailablePackage(const FString& SkippedPackage, IPlugin* PluginIfFound, FStringBuilderBase& InOutExplanation);

private:

	/** 
	  * THREA SAFETY: Any thread
	  */
	UE_API bool ShouldUpdatePluginProtocolOptions(const UGameFeaturePluginStateMachine* StateMachine, const FGameFeatureProtocolOptions& NewOptions);
	/** 
	  * THREAD SAFETY: Any thread as long as no other thread is operating on that state machine.
	  * e.g. from a parallel for on the game thread, as state machine updates only happen on the game thread.
	  */
	UE_API UE::GameFeatures::FResult UpdateGameFeatureProtocolOptions(UGameFeaturePluginStateMachine* StateMachine, const FGameFeatureProtocolOptions& NewOptions, bool* bOutDidUpdate = nullptr);

	UE_API const UGameFeatureData* GetDataForStateMachine(UGameFeaturePluginStateMachine* GFSM) const;
	UE_API const UGameFeatureData* GetRegisteredDataForStateMachine(UGameFeaturePluginStateMachine* GFSM) const;

	/** Gets relevant properties out of a uplugin file */
	UE_API TSharedPtr<const struct FGameFeaturePluginDetails> GetGameFeaturePluginDetailsInternal(const FString& PluginDescriptorFilename) const;

	/** Load details for the given plugin if they are not cached and return them asynchronously. */
	UE_API UE::Tasks::TTask<TSharedPtr<const struct FGameFeaturePluginDetails>> LoadGameFeaturePluginDetailsAsync(const FString& PluginDescriptorFilename) const;

	/** Parse FGameFeaturePluginDetails from a string. Note that parsing may modify the string in-place */
	TSharedPtr<const FGameFeaturePluginDetails> ParseGameFeaturePluginDetails(const FString& ErrorContext, TArrayView<TCHAR> JsonTxt) const;
	/** Parse FGameFeaturePluginDetails from a JSON object */
	TSharedPtr<const FGameFeaturePluginDetails> ParseGameFeaturePluginDetails(const FString& ErrorContext, TSharedRef<FJsonObject> JsonObject) const;
	/** Parse FGameFeaturePluginDetails from a RapidJSON object */
	TSharedPtr<const FGameFeaturePluginDetails> ParseGameFeaturePluginDetails(const FString& ErrorContext, UE::Json::FConstObject JsonObject) const;

	/** Prunes any cached GFP details */
	UE_API void PruneCachedGameFeaturePluginDetails(const FString& PluginURL, const FString& PluginDescriptorFilename) const;
	friend struct FGameFeaturePluginState_Unmounting;
	friend struct FBaseDataReleaseGameFeaturePluginState;

	/**
	  * Gets the state machine associated with the specified URL 
	  * THREAD SAFETY: Any thread.
	  */
	UE_API UGameFeaturePluginStateMachine* FindGameFeaturePluginStateMachine(const FString& PluginURL) const;

	/**
	  * Gets the state machine associated with the specified PluginIdentifier 
	  * THREAD SAFETY: Any thread.
	  */
	UE_API UGameFeaturePluginStateMachine* FindGameFeaturePluginStateMachine(const FGameFeaturePluginIdentifier& PluginIdentifier) const;

	/**
	  * Gets the state machine associated with the specified URL, creates it if it doesnt exist 
	  * THREAD SAFETY: Any thread.
	  */
	UE_API UGameFeaturePluginStateMachine* FindOrCreateGameFeaturePluginStateMachine(const FString& PluginURL, const FGameFeatureProtocolOptions& ProtocolOptions, bool* bOutFoundExisting = nullptr);

	/** Notification that a game feature has finished loading, and whether it was successful */
	UE_API void LoadBuiltInGameFeaturePluginComplete(const UE::GameFeatures::FResult& Result, UGameFeaturePluginStateMachine* Machine, FGameFeaturePluginStateRange RequestedDestination);

	/** 
	 * Sets a new destination state. Will attempt to cancel the current transition if the new destination is incompatible with the current destination 
	 * Note: In the case that the existing machine is terminal, a new one will need to be created. In that case ProtocolOptions will be used for the new machine.
	 * THREAD SAFETY: Game thread.
	 */
	UE_API void ChangeGameFeatureDestination(UGameFeaturePluginStateMachine* Machine, const FGameFeaturePluginStateRange& StateRange, FGameFeaturePluginChangeStateComplete CompleteDelegate);
	UE_API void ChangeGameFeatureDestination(UGameFeaturePluginStateMachine* Machine, const FGameFeatureProtocolOptions& ProtocolOptions, const FGameFeaturePluginStateRange& StateRange, FGameFeaturePluginChangeStateComplete CompleteDelegate);

	/** Generic notification that calls the Complete delegate without broadcasting anything else.*/
	UE_API void ChangeGameFeatureTargetStateComplete(UGameFeaturePluginStateMachine* Machine, const UE::GameFeatures::FResult& Result, FGameFeaturePluginChangeStateComplete CompleteDelegate);

	UE_API void BeginTermination(UGameFeaturePluginStateMachine* Machine);
	UE_API void FinishTermination(UGameFeaturePluginStateMachine* Machine);
	friend class UGameFeaturePluginStateMachine;

	/** Handler for when a state machine requests its dependencies. Returns false if the dependencies could not be read */
	UE_API bool FindOrCreatePluginDependencyStateMachines(const FString& PluginURL, const FGameFeaturePluginStateMachineProperties& InStateProperties, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines);
	template <typename> friend struct FTransitionDependenciesGameFeaturePluginState;
	friend struct FWaitingForDependenciesTransitionPolicy;

	UE_API bool FindPluginDependencyStateMachinesToActivate(const FString& PluginURL, const FString& PluginFilename, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines) const;
	friend struct FActivatingDependenciesTransitionPolicy;

	UE_API bool FindPluginDependencyStateMachinesToDeactivate(const FString& PluginURL, const FString& PluginFilename, TArray<UGameFeaturePluginStateMachine*>& OutDependencyMachines) const;
	friend struct FDeactivatingDependenciesTransitionPolicy;

	template <typename CallableT>
	bool EnumeratePluginDependenciesWithShouldActivate(const FString& PluginURL, const FString& PluginFilename, CallableT Callable) const;

	/** Handle 'ListGameFeaturePlugins' console command */
	UE_API void ListGameFeaturePlugins(const TArray<FString>& Args, UWorld* InWorld, FOutputDevice& Ar);

	UE_API void SetExplanationForNotMountingPlugin(const FString& PluginURL, const FString& Explanation);

	enum class EObserverCallback
	{
		CheckingStatus,
		Terminating,
		Predownloading,
		PostPredownloading,
		Downloading,
		Releasing,
		PreMounting,
		PostMounting,
		Registering,
		Unregistering,
		Loading,
		Unloading,
		Activating,
		Activated,
		Deactivating,
		PauseChanged,
		Count
	};

	UE_API void CallbackObservers(EObserverCallback CallbackType, const FGameFeaturePluginIdentifier& PluginIdentifier,
		const FString* PluginName = nullptr, 
		const UGameFeatureData* GameFeatureData = nullptr, 
		FGameFeatureStateChangeContext* StateChangeContext = nullptr);

	enum class EBatchObserverCallback : uint8
	{
		BatchOperationStarted,
		BatchOperationComplete
	};

	UE_API void BatchCallbackObservers(EBatchObserverCallback BatchCallbackType);
	friend struct GameFeaturesSubsystem::FNotifyObservers;

	/** Registers a state machine that is in transition and running */
	UE_API void RegisterRunningStateMachine(UGameFeaturePluginStateMachine* GFPSM);

	/** Unregisters a state machine that was in transition is no longer running */
	UE_API void UnregisterRunningStateMachine(UGameFeaturePluginStateMachine* GFPSM);

	/** Adds a batching request for a given state so we can start listening for when state machines have arrived at a fence */
	UE_API FDelegateHandle AddBatchingRequest(EGameFeaturePluginState State, FGameFeaturePluginRequestUpdateStateMachine UpdateDelegate);

	/** Cancels an existing batching request */
	UE_API void CancelBatchingRequest(EGameFeaturePluginState State, FDelegateHandle DelegateHandle);

	UE_API void EnableTick();
	UE_API void DisableTick();
	UE_API bool Tick(float DeltaTime);

	UE_API bool TickBatchProcessing();

	friend class UE::GameFeatures::FRealtimeMode;
	friend bool UE::GameFeatures::TickUnloadPluginAssets(float DeltaTime);
	void TryUpdateStateMachineImmediate(int64 Id);

private:
	/** The list of all game feature plugin state machine objects addressed by FGameFeaturePluginStateMachineProperties::Id */
	UPROPERTY(Transient)
	TMap<int64, TObjectPtr<UGameFeaturePluginStateMachine>> GameFeaturePluginStateMachines;

	/** Lookup of game feature plugins by FGameFeaturePluginIdentifier::GetIdentifyingString */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UGameFeaturePluginStateMachine>> GameFeaturesByIdentifier;

	/** Private type controlling execution of state machine transitions when UE::GameFeatures::bUseNewExecutor */
	TPimplPtr<UE::GameFeatures::FStateMachineExecutor> StateMachineExecutor;
	friend class UE::GameFeatures::FStateMachineExecutor;

	/** The tick handle if currently registered for a tick */
	FTSTicker::FDelegateHandle TickHandle;

	/** State machine currently in transition, used to limit search space when checking a batch processing fence or similar */
	TArray<UGameFeaturePluginStateMachine*> RunningStateMachines;

	/** Active fences */
	struct FGameFeatureBatchProcessingFence
	{
		FNotifyGameFeaturePluginRequestUpdateStateMachine NotifyUpdateStateMachines;
	};
	TMap<EGameFeaturePluginState, FGameFeatureBatchProcessingFence> BatchProcessingFences;

	/** Game feature plugin state machine objects that are being terminated. Used to prevent GC until termination is complete. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UGameFeaturePluginStateMachine>> TerminalGameFeaturePluginStateMachines;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<FPluginString, FPluginString> GameFeaturePluginNameToPathMap;
	mutable FTransactionallySafeRWLock GameFeaturePluginNameToPathMapLock;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	mutable TMap<FString, TSharedRef<const FGameFeaturePluginDetails>> CachedPluginDetailsByFilename;
	mutable FTransactionallySafeRWLock CachedGameFeaturePluginDetailsLock;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> Observers;

	UPROPERTY(Transient)
	TObjectPtr<UGameFeaturesProjectPolicies> GameSpecificPolicies;

	TUniquePtr<class UE::GameFeatures::FPackageLoadTracker> PackageLoadTracker;

#if WITH_EDITOR
	// When we decide not to mount a plugin, we can store an explanation here so that if we later attempt to load an asset from it we can tell the user why it's not available
	TMap<FString, FString> UnmountedPluginNameToExplanation;

	TUniquePtr<class FGameFeatureDataExternalAssetsPathCache> GameFeatureDataExternalAssetsPathCache;
#endif

#if !UE_BUILD_SHIPPING
	TSet<FString> DebugStateChangedForPlugins;
#endif
	FDelegateHandle GetExplanationForUnavailablePackageDelegateHandle;

	bool bInitializedPolicyManager = false;
	bool bLoadedBuiltInDetails = false;

	static UE_API TSet<FString> DlcPlugins;

	/**
	 * Next identifier to be used for a GFP state machine.
	 * Identifiers are not reused, and when a GFP is terminated and later recreated, a new identifier is assigned.
	 */
	int64 NextInstanceId = 0;

private:
	/**
	 * State machine execution functions hoisted from UGameFeaturePluginStateMachine etc as a refactor
	 * to enable more batch & parallel processing.
	 * These fields could be further factored out into another class.
	 * These methods are marked const until we make changes that hoist state out into the subsystem.
	 */

	/** Initializes the state machine and assigns the URL for the plugin it manages. This sets the machine to the 'UnknownStatus' state. */
	void InitStateMachine(UGameFeaturePluginStateMachine* This, int64 Id, FGameFeaturePluginIdentifier InPluginIdentifier, const FGameFeatureProtocolOptions& InProtocolOptions);

	/** Asynchronously transitions the state machine to the destination state range and reports when it is done. 
	  * DestinationState must be of type EGameFeaturePluginStateType::Destination.
	  * If returns true and OnFeatureStateTransitionComplete is not called immediately, OutCallbackHandle will be set
	  * Returns false and does not callback if a transition is already in progress and the destination range is not compatible with the current range. */
	bool SetDestination(
		UGameFeaturePluginStateMachine* This,
		FGameFeaturePluginStateRange InDestination,
		TDelegate<void(UGameFeaturePluginStateMachine*, const UE::GameFeatures::FResult&)> OnFeatureStateTransitionComplete,
		FDelegateHandle* OutCallbackHandle = nullptr);

	/** Cancel the current transition if possible */
	bool TryCancel(UGameFeaturePluginStateMachine* This, TDelegate<void(UGameFeaturePluginStateMachine*)> OnFeatureStateTransitionCanceled, FDelegateHandle* OutCallbackHandle = nullptr) const;

	/** Update the current PluginURL data for this plugin if possible. Returns false if this update fails or
		if the supplied InPluginURL matches the existing URL data for this plugin.**/
	UE::GameFeatures::FResult TryUpdatePluginProtocolOptions(UGameFeaturePluginStateMachine* This, const FGameFeatureProtocolOptions& InOptions, bool& bOutDidUpdate) const;

	/** Remove any pending callback from SetDestination */
	void RemovePendingTransitionCallback(UGameFeaturePluginStateMachine* This, FDelegateHandle InHandle) const;

	/** Remove any pending callback from SetDestination */
	void RemovePendingTransitionCallback(UGameFeaturePluginStateMachine* This, FDelegateUserObject DelegateObject) const;

	/** Remove any pending callback from TryCancel */
	void RemovePendingCancelCallback(UGameFeaturePluginStateMachine* This, FDelegateHandle InHandle) const;

	/** Remove any pending callback from TryCancel */
	void RemovePendingCancelCallback(UGameFeaturePluginStateMachine* This, FDelegateUserObject DelegateObject) const;

	/** Returns the PluginIdentifier used to identify this plugin uniquely to the GameFeaturePlugin Subsystem */
	const FGameFeaturePluginIdentifier& GetPluginIdentifier(const UGameFeaturePluginStateMachine* This) const;

	/** Returns the name of the game feature. Before StatusKnown, this returns the URL. */
	const FString& GetGameFeatureName(const UGameFeaturePluginStateMachine* This) const;

	/** Returns the URL */
	const FString& GetPluginURL(const UGameFeaturePluginStateMachine* This) const;

	/** Returns protocol medata */
	const FGameFeatureProtocolMetadata& GetProtocolMetadata(const UGameFeaturePluginStateMachine* This) const;

	/** Returns any protocol options */
	const FGameFeatureProtocolOptions& GetProtocolOptions(const UGameFeaturePluginStateMachine* This) const;

	/** Returns protocol options suitable for reuse by another state machine */
	FGameFeatureProtocolOptions RecycleProtocolOptions(const UGameFeaturePluginStateMachine* This) const;

	/** Returns the plugin name if known (plugin must have been registered to know the name). */
	const FString& GetPluginName(const UGameFeaturePluginStateMachine* This) const;

	/** Returns the uplugin filename of the game feature. Before StatusKnown, this returns false. */
	bool GetPluginFilename(const UGameFeaturePluginStateMachine* This, FString& OutPluginFilename) const;

	/** Returns the enum state for this machine */
	EGameFeaturePluginState GetCurrentState(const UGameFeaturePluginStateMachine* This) const;

	/** Returns the state range this machine is trying to move to */
	FGameFeaturePluginStateRange GetDestination(const UGameFeaturePluginStateMachine* This) const;

	/** Returns information about the current state */
	FGameFeaturePluginStateInfo GetCurrentStateInfo(const UGameFeaturePluginStateMachine* This) const;

	/** Returns true if attempting to reach a new destination state */
	bool IsRunning(const UGameFeaturePluginStateMachine* This) const;

	/** Returns true if the state is at least StatusKnown so we can query info about the game feature plugin */
	bool IsStatusKnown(const UGameFeaturePluginStateMachine* This) const;

	/** Returns true if the plugin is available to download/load. Only call if IsStatusKnown is true */
	bool IsAvailable(const UGameFeaturePluginStateMachine* This) const;

	/** Returns true if the plugin is in an error state */
	bool IsInErrorState(const UGameFeaturePluginStateMachine* This) const;

	/** Whether this machine is allowed to be asynchronous */
	bool AllowAsyncLoading(const UGameFeaturePluginStateMachine* This) const;

	/** Returns true if the plugin will stream dependencies after installing. Only call if IsStatusKnown is true */
	bool HasAssetStreamingDependencies(const UGameFeaturePluginStateMachine* This) const;

	void SetWasLoadedAsBuiltIn(UGameFeaturePluginStateMachine* This);

	bool WasLoadedAsBuiltIn(const UGameFeaturePluginStateMachine* This) const;

	/** If the plugin is activated already, we will retrieve its game feature data */
	UGameFeatureData* GetGameFeatureDataForActivePlugin(UGameFeaturePluginStateMachine* This) const;
	const UGameFeatureData* GetGameFeatureDataForActivePlugin(const UGameFeaturePluginStateMachine* This) const;

	/** If the plugin is registered already, we will retrieve its game feature data */
	UGameFeatureData* GetGameFeatureDataForRegisteredPlugin(UGameFeaturePluginStateMachine* This, bool bCheckForRegistering = false) const;
	const UGameFeatureData* GetGameFeatureDataForRegisteredPlugin(const UGameFeaturePluginStateMachine* This, bool bCheckForRegistering = false) const;

	const FGameFeaturePluginStateMachineProperties& GetProperties(const UGameFeaturePluginStateMachine* This) const;

	/** Returns true if the state machine is in an unrecoverable error state. */
	bool IsErrorStateUnrecoverable(const UGameFeaturePluginStateMachine* This) const;

	/** Sets that the state machine is in an unrecoverable error. Used when the GFD is missing as that can't be recovered from. */
	void SetUnrecoverableError(UGameFeaturePluginStateMachine* This) const;

	/** Request to exclude this plugin from batch processing, for e.g. if the plugin is a dependency and has batch processing enabled */
	void ExcludeFromBatchProcessing(UGameFeaturePluginStateMachine* This);

	/** Helper static function to trigger state specific batch processing functions */
	static void BatchProcess(EGameFeaturePluginState State, TConstArrayView<UGameFeaturePluginStateMachine*> GFPSMs);	

private:
	/** Returns true if the specified state is not a transition state */
	bool IsValidTransitionState(const UGameFeaturePluginStateMachine* This, EGameFeaturePluginState InState) const;

	/** Returns true if the specified state is a destination state */
	bool IsValidDestinationState(const UGameFeaturePluginStateMachine* This, EGameFeaturePluginState InDestinationState) const;

	/** Returns true if the specified state is a error state */
	bool IsValidErrorState(const UGameFeaturePluginStateMachine* This, EGameFeaturePluginState InDestinationState) const;

	/** Processes the current state and looks for state transitions */
	void UpdateStateMachine(UGameFeaturePluginStateMachine* This, TOptional<int32> MaxTransitions = {});

	/** Helper to register the state machine with the manager as a state machine in transiton */
	void RegisterAsTransitioningStateMachine(UGameFeaturePluginStateMachine* This);

	/** Helper to unregister the state machine with the manager as a state machine in transiton */
	void UnregisterAsTransitioningStateMachine(UGameFeaturePluginStateMachine* This);
	
	/** Adds a batching request for current state if state can be batched */
	void CheckAddBatchingRequestForCurrentState(UGameFeaturePluginStateMachine* This);

	/** Cancels batching re1quest for current state if one exists */
	void CheckAndCancelBatchingRequestForCurrentState(UGameFeaturePluginStateMachine* This);

	struct FAmmortizedLoadContext
	{
		FScopeLogTime ScopeLogTime{ TEXT("BuiltInGameFeaturePlugins loaded."), nullptr, FConditionalScopeLogTime::ScopeLog_Seconds };

		TArray<TSharedRef<IPlugin>> AmortizedPluginsRemaining;
		TMap<FString, UE::GameFeatures::FResult> Results;
		FBuiltInGameFeaturePluginsLoaded CompleteDelegate;

		int32 NumPluginsLoaded = 0;
		int32 PluginAmortizeRate = 0;
		bool bPushedTagsBroadcast = false;
		bool bPushedAssetBulkScanning = false;
		bool bCancelled = false;
		FGuid GUID;

		FAmmortizedLoadContext();

		~FAmmortizedLoadContext();
	};

	bool CancellAmortizedLoads(const FString& PluginURL);
	TMap<FGuid, TWeakPtr<FAmmortizedLoadContext>> ActiveAmmoritzedLoads;
};

#undef UE_API
