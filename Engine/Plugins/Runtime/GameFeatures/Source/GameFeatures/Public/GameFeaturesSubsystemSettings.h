// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "GameFeaturesSubsystemSettings.generated.h"

#define UE_API GAMEFEATURES_API

/** Settings for the Game Features framework */
UCLASS(MinimalAPI, config=Game, defaultconfig, meta = (DisplayName = "Game Features"))
class UGameFeaturesSubsystemSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UGameFeaturesSubsystemSettings();

	/** State/Bundle to always load on clients */
	static UE_API const FName LoadStateClient;

	/** State/Bundle to always load on dedicated server */
	static UE_API const FName LoadStateServer;

	/** Name of a singleton class to spawn as the game feature project policy. If empty, it will spawn the default one (UDefaultGameFeaturesProjectPolicies) */
	UPROPERTY(config, EditAnywhere, Category=DefaultClasses, meta=(MetaClass="/Script/GameFeatures.GameFeaturesProjectPolicies", DisplayName="Game Feature Project Policy Class", ConfigRestartRequired=true))
	FSoftClassPath GameFeaturesManagerClassName;

	/** List of plugins that are forcibly enabled (e.g., via a hotfix) */
	UPROPERTY(config, EditAnywhere, Category = GameFeatures)
	TArray<FString> EnabledPlugins;

	/** List of plugins that are forcibly disabled (e.g., via a hotfix) */
	UPROPERTY(config, EditAnywhere, Category=GameFeatures)
	TArray<FString> DisabledPlugins;

	/** List of metadata (additional keys) to try parsing from the .uplugin to provide to FGameFeaturePluginDetails */
	UPROPERTY(config, EditAnywhere, Category=GameFeatures)
	TArray<FString> AdditionalPluginMetadataKeys;

	/** 
	  * Returns true if the specified (normalized or full) path is a game feature plugin
	  * Safe to call from any thread.
	  */ 
	UE_API bool IsValidGameFeaturePlugin(const FString& PluginDescriptorFilename) const;
	UE_API bool IsValidGameFeaturePlugin(FStringView PluginDescriptorFilename) const;

};

#undef UE_API
