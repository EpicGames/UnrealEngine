// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "GameFeatureAction.h"
#include "Iris/ReplicationSystem/Filtering/NetObjectFilterDefinitions.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "IrisFilterGameFeatureAction.generated.h"

class UWorld;
struct FGameFeatureStateChangeContext;
struct FWorldContext;

UCLASS(MinimalAPI, CollapseCategories, meta = (DisplayName = "Create Iris Network Filter"))
class UIrisFilterGameFeatureAction : public UGameFeatureAction
{
	GENERATED_BODY()

public:
	virtual void OnGameFeatureActivating(FGameFeatureActivatingContext&) override;
	virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext&) override;

private:
	bool ShouldApplyToWorld(const FGameFeatureStateChangeContext& StateChangeContext, const FWorldContext& WorldContext) const;
	void AddFilterToWorld(UWorld* World);
	void RemoveFilterFromWorld(UWorld* World);

	// Filters to create on activation & destroy on deactivation.
	UPROPERTY(EditAnywhere, Category=Filter)
	FNetObjectFilterDefinition FilterDefinition;

	// Worlds the filter was added to, and should be removed from.
	TSet<TWeakObjectPtr<UWorld>> AddedToWorlds;
};
