// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameFeaturesEditor : ModuleRules
	{
        public GameFeaturesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
                {
                    "Core",
                    "CoreUObject",
					"GameFeatures",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"AssetRegistry",
					"DataLayerEditor",
					"DataValidation",
					"DeveloperSettings",
					"Engine",
					"ModularGameplay",
					"EditorSubsystem",
					"Projects",
					"EditorFramework",
					"Slate",
					"SlateCore",
					"PropertyEditor",
					"SharedSettingsWidgets",
					"Json"
				}
			);
		}
	}
}
