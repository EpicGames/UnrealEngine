// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureActionConvertContentBundleWorldPartitionBuilder.h"
#include "GameFeatureAction_AddWorldPartitionContent.h"
#include "GameFeatureAction_AddWPContent.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleClient.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "DataLayer/ExternalDataLayerFactory.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/MetaData.h"
#include "Commandlets/Commandlet.h"
#include "GameFeaturesSubsystem.h"
#include "GameFeatureData.h"
#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "Misc/PathViews.h"
#include "AssetSelection.h"
#include "PackageTools.h"
#include "IAssetTools.h"
#include "Algo/Find.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureActionConvertContentBundleWorldPartitionBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogConvertContentBundleBuilder, All, All);

UGameFeatureActionConvertContentBundleWorldPartitionBuilder::UGameFeatureActionConvertContentBundleWorldPartitionBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bReportOnly(false)
	, bRemoveContentBundleAction(false)
	, bDeleteHLODs(false)
{}

bool UGameFeatureActionConvertContentBundleWorldPartitionBuilder::PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	TArray<FString> Tokens, Switches;
	TMap<FString, FString> CommandLineParams;
	UCommandlet::ParseCommandLine(*GetBuilderArgs(), Tokens, Switches, CommandLineParams);

	FString const* DestFolderPtr = CommandLineParams.Find(TEXT("DestinationFolder"));
	DestinationFolder = DestFolderPtr ? *DestFolderPtr : FString();
	bReportOnly = Switches.Contains(TEXT("ReportOnly"));
	bRemoveContentBundleAction = Switches.Contains(TEXT("RemoveContentBundleAction"));
	bDeleteHLODs = Switches.Contains(TEXT("DeleteHLODs"));

	if (FString const* ContentBundlesToConvertString = CommandLineParams.Find(TEXT("ContentBundles")))
	{
		if ((*ContentBundlesToConvertString).ParseIntoArray(ContentBundlesToConvert, TEXT("+")) == 0)
		{
			UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to parse ContentBundles argument '%ls'.", **ContentBundlesToConvertString);
			return false;
		}
	}

	return true;
}

bool UGameFeatureActionConvertContentBundleWorldPartitionBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	TStrongObjectPtr<UExternalDataLayerFactory> ExternalDataLayerFactory(NewObject<UExternalDataLayerFactory>(GetTransientPackage()));

	UContentBundleManager* ContentBundleManager = World->ContentBundleManager;
	if (!ContentBundleManager)
	{
		UE_LOGF(LogConvertContentBundleBuilder, Error, "World %ls does not have content bundles.", *World->GetName());
		return false;
	}

	TArray<TSharedPtr<FContentBundleEditor>> ContentBundles;
	if (!ContentBundleManager->GetEditorContentBundle(ContentBundles))
	{
		UE_LOGF(LogConvertContentBundleBuilder, Error, "World %ls does not have content bundles.", *World->GetName());
		return false;
	}

	UExternalDataLayerManager* ExternalDataLayerManager = UExternalDataLayerManager::GetExternalDataLayerManager(World);
	if (!ExternalDataLayerManager)
	{
		UE_LOGF(LogConvertContentBundleBuilder, Error, "World %ls does not have an ExternalDataLayerManager", *World->GetName());
		return false;
	}

	UE_LOGF(LogConvertContentBundleBuilder, Log, "Found %d Content bundles in World %ls", ContentBundles.Num(), *World->GetName());

	TArray<TSharedPtr<FContentBundleEditor>> ContentBundlesToProcess;
	if (!ContentBundlesToConvert.IsEmpty())
	{
		for (TSharedPtr<FContentBundleEditor>& ContentBundle : ContentBundles)
		{
			if (!ContentBundlesToConvert.Contains(ContentBundle->GetDisplayName()))
			{
				UE_LOGF(LogConvertContentBundleBuilder, Log, "Skipping %ls: Not in the list of content bundles to convert.", *ContentBundle->GetDisplayName());
				continue;
			}
			ContentBundlesToProcess.Add(ContentBundle);
		}
	}
	else
	{
		ContentBundlesToProcess = ContentBundles;
	}

	bool bIsSuccess = true;
	for (TSharedPtr<FContentBundleEditor>& ContentBundle : ContentBundlesToProcess)
	{
		TSet<UPackage*> PackagesToSave;
		TSet<UPackage*> PackagesToDelete;

		TSharedPtr<FContentBundleClient> ContentBundleClient = ContentBundle->GetClient().Pin();
		if (!ContentBundleClient.IsValid())
		{
			UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to access Client of Content Bundle %ls", *ContentBundle->GetDisplayName());
			bIsSuccess = false;
			continue;
		}

		if (ContentBundleClient->GetState() == EContentBundleClientState::Registered)
		{
			if (const FContentBundleBase* CB = ContentBundleManager->GetContentBundle(World, ContentBundle->GetDescriptor()->GetGuid()))
			{
				if (CB->GetStatus() != EContentBundleStatus::ContentInjected)
				{
					UE_LOGF(LogConvertContentBundleBuilder, Log, "Requesting forced injection of Content Bundle %ls for conversion purposes.", *ContentBundle->GetDisplayName());
					ContentBundleClient->RequestContentInjection();
				}
			}
		}

		if (const FContentBundleBase* CB = ContentBundleManager->GetContentBundle(World, ContentBundle->GetDescriptor()->GetGuid()))
		{
			if (CB->GetStatus() != EContentBundleStatus::ContentInjected)
			{
				UE_LOGF(LogConvertContentBundleBuilder, Log, "Skipping %ls: Not injected in this world.", *ContentBundle->GetDisplayName());
				continue;
			}
		}

		UActorDescContainerInstance* ContentBundleContainerInstance = ContentBundle->GetActorDescContainerInstance().Get();
		const UContentBundleDescriptor* ContentBundleDescriptor = ContentBundle->GetDescriptor();
		UGameFeatureAction_AddWPContent* OldAddWPContentAction = ContentBundleDescriptor->GetTypedOuter<UGameFeatureAction_AddWPContent>();
		UGameFeatureData* GameFeatureData = ContentBundleDescriptor->GetTypedOuter<UGameFeatureData>();
		if (!OldAddWPContentAction || !GameFeatureData || !ContentBundleContainerInstance)
		{
			UE_LOGF(LogConvertContentBundleBuilder, Error, "Skipping invalid Content Bundle %ls", *ContentBundle->GetDisplayName());
			bIsSuccess = false;
			continue;
		}

		if (ContentBundleContainerInstance->IsEmpty())
		{
			UE_LOGF(LogConvertContentBundleBuilder, Log, "Skipping conversion : Empty Content Bundle %ls", *ContentBundle->GetDisplayName());
			SkippedEmptyContentBundles.Add(ContentBundle);
			continue;
		}

		// Find or create EDL Asset for this Content Bundle
		UExternalDataLayerAsset* ExternalDataLayerAsset = GetOrCreateExternalDataLayerAsset(ContentBundleDescriptor, ExternalDataLayerFactory.Get(), PackagesToSave);
		UE_CLOGF(ExternalDataLayerAsset, LogConvertContentBundleBuilder, Log, "Converting Content Bundle %ls using External Data Layer Asset %ls.", *ContentBundle->GetDisplayName(), *ExternalDataLayerAsset->GetPathName());
		if (!ExternalDataLayerAsset)
		{
			UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to retrieve or create External Data Layer Asset for Content Bundle %ls", *ContentBundle->GetDisplayName());
			bIsSuccess = false;
			continue;
		}

		// Allow injection of External Data Layer (only used by this builder)
		UExternalDataLayerEngineSubsystem::FForcedExternalDataLayerInjectionKey ForcedInjectionKey(World, ExternalDataLayerAsset);
		UExternalDataLayerEngineSubsystem::Get().ForcedAllowInjection.Add(ForcedInjectionKey);
		ON_SCOPE_EXIT { UExternalDataLayerEngineSubsystem::Get().ForcedAllowInjection.Remove(ForcedInjectionKey); };

		// Find existing GameFeatureAction_AddWorldPartitionContent for this EDL Asset
		UGameFeatureAction* const* ExistingAddWorldPartitionContent = Algo::FindByPredicate(GameFeatureData->GetActions(), [ExternalDataLayerAsset](UGameFeatureAction* Action) { return Action && Action->IsA<UGameFeatureAction_AddWorldPartitionContent>() && Cast<UGameFeatureAction_AddWorldPartitionContent>(Action)->GetExternalDataLayerAsset() == ExternalDataLayerAsset; });
		UGameFeatureAction_AddWorldPartitionContent* AddWorldPartitionContent = ExistingAddWorldPartitionContent ? Cast<UGameFeatureAction_AddWorldPartitionContent>(*ExistingAddWorldPartitionContent) : nullptr;
		if (!AddWorldPartitionContent)
		{
			// Find existing GameFeatureAction_AddWorldPartitionContent with no EDL asset
			UGameFeatureAction* const* ExistingAddWorldPartitionContentWithNoEDLAsset = Algo::FindByPredicate(GameFeatureData->GetActions(), [ExternalDataLayerAsset](UGameFeatureAction* Action) { return Action && Action->IsA<UGameFeatureAction_AddWorldPartitionContent>() && Cast<UGameFeatureAction_AddWorldPartitionContent>(Action)->GetExternalDataLayerAsset() == nullptr; });
			AddWorldPartitionContent = ExistingAddWorldPartitionContentWithNoEDLAsset ? Cast<UGameFeatureAction_AddWorldPartitionContent>(*ExistingAddWorldPartitionContentWithNoEDLAsset) : nullptr;
			if (!AddWorldPartitionContent)
			{
				// Create new GameFeatureAction_AddWorldPartitionContent for this EDL Asset
				AddWorldPartitionContent = NewObject<UGameFeatureAction_AddWorldPartitionContent>(GameFeatureData);
				GameFeatureData->GetMutableActionsInEditor().Add(AddWorldPartitionContent);
				PackagesToSave.Add(GameFeatureData->GetPackage());
				UE_LOGF(LogConvertContentBundleBuilder, Log, "Added new Action of type 'GameFeatureAction_AddWorldPartitionContent' to GameFeatureData %ls using External Data Layer Asset %ls while converting Content Bundle %ls.", *GameFeatureData->GetName(), *ExternalDataLayerAsset->GetPathName(), *ContentBundle->GetDisplayName());
			}

			// Set EDL Asset on Action if necessary
			if (AddWorldPartitionContent->ExternalDataLayerAsset != ExternalDataLayerAsset)
			{
				check(!AddWorldPartitionContent->ExternalDataLayerAsset);
				AddWorldPartitionContent->ExternalDataLayerAsset = ExternalDataLayerAsset;
				PackagesToSave.Add(GameFeatureData->GetPackage());
			}
		}
		
		// Backup the source ContentBundle Guid (to facilitate potential revert operation)
		AddWorldPartitionContent->ConvertedContentBundleGuid = ContentBundleDescriptor->GetGuid();
		PackagesToSave.Add(GameFeatureData->GetPackage());
		
		// Manually call OnExternalDataLayerAssetChanged to register the newly created GameFeatureAction_AddWorldPartitionContent
		AddWorldPartitionContent->OnExternalDataLayerAssetChanged(nullptr, ExternalDataLayerAsset);

		// First try to find existing ExternalDataLayerInstance
		UExternalDataLayerInstance* ExternalDataLayerInstance = ExternalDataLayerManager->GetExternalDataLayerInstance(ExternalDataLayerAsset);
		if (!ExternalDataLayerInstance)
		{
			// Create AWorldDataLayers and the ExternalDataLayerInstance for this EDL Asset
			FDataLayerCreationParameters CreationParams;
			CreationParams.DataLayerAsset = ExternalDataLayerAsset;
			CreationParams.WorldDataLayers = World->GetWorldDataLayers();
			ExternalDataLayerInstance = Cast<UExternalDataLayerInstance>(UDataLayerEditorSubsystem::Get()->CreateDataLayerInstance(CreationParams));
			UE_CLOGF(ExternalDataLayerInstance, LogConvertContentBundleBuilder, Log, "Create External Data Layer Instance %ls while converting Content Bundle %ls.", *ExternalDataLayerAsset->GetPathName(), *ContentBundle->GetDisplayName());
			if (!ExternalDataLayerInstance)
			{
				UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to create External Data Layer Instance for External Data Layer Asset %ls while converting Content Bundle %ls", *ExternalDataLayerAsset->GetName(), *ContentBundle->GetDisplayName());
				bIsSuccess = false;
				continue;
			}
			check(!ExternalDataLayerInstance->IsPackageExternal());
		}

		AWorldDataLayers* EDLWorldDataLayers = ExternalDataLayerInstance->GetDirectOuterWorldDataLayers();
		if (!EDLWorldDataLayers)
		{
			UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to find a valid WorldDataLayers actor for External Data Layer Asset %ls while converting Content Bundle %ls", *ExternalDataLayerAsset->GetName(), *ContentBundle->GetDisplayName());
			bIsSuccess = false;
			continue;
		}

		// By default, newly create AWorldDataLayers will be marked as read-only which would block assignation of actors to the External Data Layer
		// Temporarily remove package flag
		bool bIsNewEDLWorldDataLayers = EDLWorldDataLayers->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated);
		if (bIsNewEDLWorldDataLayers)
		{
			EDLWorldDataLayers->GetPackage()->ClearPackageFlags(PKG_NewlyCreated);
			PackagesToSave.Add(EDLWorldDataLayers->GetPackage());
		}

		// In case we want to delete HLODs
		TArray<UClass*> HLODActorClasses =
		{
			AWorldPartitionHLOD::StaticClass(),
			FindObject<UClass>(nullptr, TEXT("/Script/Engine.SpatialHashRuntimeGridInfo"))
		};

		TArray<FGuid> ConvertedActorGuids;
		TArray<FWorldPartitionReference> ActorReferences;
		bool bContentBundleActorConversionSuccess = true;
		// Convert actors from Content Bundle to EDL
		for (UActorDescContainerInstance::TIterator<> It(ContentBundleContainerInstance); It; ++It)
		{
			FWorldPartitionReference ActorRef(ContentBundleContainerInstance, It->GetGuid());
			if (!ActorRef.IsValid())
			{
				UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to load actor %ls(%ls). Actor won't be converted.", *It->GetActorLabelOrName().ToString(), *It->GetActorPackage().ToString());
				bContentBundleActorConversionSuccess = false;
				break;
			}

			ActorReferences.Add(ActorRef);
			AActor* Actor = ActorRef.GetActor();
			UPackage* OldActorPackage = Actor->GetExternalPackage();
			ResetLoaders(OldActorPackage);

			// Content bundles can contain HLODs, and since their names are partly based on the content bundle they were in, moving/renaming them over in an EDL is not straightforward.
			// It's easier to delete them and let the automated nightly recreate them properly.
			// Only need to use this option if a HLOD is preventing FExternalDataLayerHelper::MoveActorsToExternalDataLayer from succeeding, and you want to avoid a manual Remove HLOD step
			if (bDeleteHLODs)
			{
				if (HLODActorClasses.FindByPredicate([ActorClass = Actor->GetClass()](const UClass* HLODClass) { return ActorClass->IsChildOf(HLODClass); }))
				{
					PackagesToDelete.Add(Actor->GetExternalPackage());
					ConvertedActorGuids.Add(It->GetGuid());
					UE_LOGF(LogConvertContentBundleBuilder, Log, "Deleted HLOD-related Actor %ls.", *Actor->GetFullName());
					continue;
				}
			}

			FText FailureReason;
			if (!FExternalDataLayerHelper::MoveActorsToExternalDataLayer({ Actor }, ExternalDataLayerInstance, &FailureReason))
			{
				UE_LOGF(LogConvertContentBundleBuilder, Error, "Can't create package for actor %ls. %ls", *Actor->GetActorNameOrLabel(), *FailureReason.ToString());
				bContentBundleActorConversionSuccess = false;
				break;
			}

			check(Actor->GetExternalDataLayerAsset() == ExternalDataLayerAsset);
			check(!Actor->GetContentBundleGuid().IsValid());
			UPackage* NewPackage = Actor->GetPackage();
			PackagesToSave.Add(NewPackage);

			check(OldActorPackage->GetName() != NewPackage->GetName());
			PackagesToDelete.Add(OldActorPackage);
			
			UE_LOGF(LogConvertContentBundleBuilder, Log, "Converted Actor %ls(%ls) to %ls.", *Actor->GetName(), *NewPackage->GetName(), *ExternalDataLayerAsset->GetName());

			ConvertedActorGuids.Add(It->GetGuid());
		}

		// Remove actors from Content Bundle container (since we recycled the same Actor/guid)
		for (const FGuid& ActorGuid : ConvertedActorGuids)
		{
			ContentBundleContainerInstance->RemoveActor(ActorGuid);
		}
		
		if (bContentBundleActorConversionSuccess && bRemoveContentBundleAction)
		{
			// Remove Content Bundle GameFeatureData action
			GameFeatureData->GetMutableActionsInEditor().Remove(OldAddWPContentAction);
			PackagesToSave.Add(GameFeatureData->GetPackage());
		}

		// Restore package flag
		if (bIsNewEDLWorldDataLayers)
		{
			EDLWorldDataLayers->GetPackage()->SetPackageFlags(PKG_NewlyCreated);
		}
		
		// Log
		for (UPackage* PackageToSave : PackagesToSave)
		{
			UE_LOGF(LogConvertContentBundleBuilder, Log, "Package to save: %ls", *PackageToSave->GetPathName());
		}

		for (UPackage* PackageToDelete : PackagesToDelete)
		{
			UE_LOGF(LogConvertContentBundleBuilder, Log, "Package to delete: %ls", *PackageToDelete->GetPathName());
		}

		if (bContentBundleActorConversionSuccess && !bReportOnly)
		{
			if (!UWorldPartitionBuilder::SavePackages(PackagesToSave.Array(), PackageHelper))
			{
				UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to save packages. Conversion sanity is not guaranteed. Consult log for details.");
				bContentBundleActorConversionSuccess = false;
			}
			else if (!UWorldPartitionBuilder::DeletePackages(PackagesToDelete.Array(), PackageHelper))
			{
				UE_LOGF(LogConvertContentBundleBuilder, Error, "Failed to delete packages. Conversion sanity is not guaranteed. Consult log for details.");
				bContentBundleActorConversionSuccess = false;
			}
		}

		if (bContentBundleActorConversionSuccess)
		{
			ConvertedContentBundles.Add(ContentBundle);
			FinalReport.Reserve(FinalReport.Num() + PackagesToSave.Num() + PackagesToDelete.Num() + 5);
			FinalReport.Add(TEXT("------------------------------------------------------------------------"));
			FinalReport.Add(FString::Printf(TEXT("Converted Content Bundle '%s' (%s) to EDL Asset '%s'"), *ContentBundle->GetDisplayName(), *ContentBundle->GetDescriptor()->GetGuid().ToString(), *ExternalDataLayerAsset->GetPathName()));
			FinalReport.Add(TEXT("------------------------------------------------------------------------"));
			FinalReport.Add(FString::Printf(TEXT("[+] Added %d packages: "), PackagesToSave.Num()));
			for (UPackage* PackageToSave : PackagesToSave)
			{
				FinalReport.Add(FString::Printf(TEXT(" |- %s"), *PackageToSave->GetPathName()));
			}
			if (PackagesToDelete.Num())
			{
				FinalReport.Add(FString::Printf(TEXT("[+] Deleted %d packages: "), PackagesToDelete.Num()));
				for (UPackage* PackageToDelete : PackagesToDelete)
				{
					FinalReport.Add(FString::Printf(TEXT(" |- %s"), *PackageToDelete->GetPathName()));
				}
			}
		}
		else
		{
			bIsSuccess = false;
		}
	}

	if (SkippedEmptyContentBundles.Num())
	{
		FinalReport.Reserve(FinalReport.Num() + SkippedEmptyContentBundles.Num() + 4);
		FinalReport.Add(TEXT("------------------------------------------------------------------------"));
		FinalReport.Add(FString::Printf(TEXT("%d Skipped Empty Content Bundles: "), SkippedEmptyContentBundles.Num()));
		FinalReport.Add(TEXT("------------------------------------------------------------------------"));
		FinalReport.Add(TEXT("[+] Content Bundles: "));
		for (const TSharedPtr<FContentBundleEditor>& CB : SkippedEmptyContentBundles)
		{
			FinalReport.Add(FString::Printf(TEXT(" |- %s (%s)"), *CB->GetDisplayName(), *CB->GetDescriptor()->GetGuid().ToString()));
		}
	}

	TArray<TSharedPtr<FContentBundleEditor>> FailedContentBundles;
	for (TSharedPtr<FContentBundleEditor>& ContentBundle : ContentBundlesToProcess)
	{
		if (!ConvertedContentBundles.Contains(ContentBundle) && !SkippedEmptyContentBundles.Contains(ContentBundle))
		{
			FailedContentBundles.Add(ContentBundle);
		}
	}

	if (FailedContentBundles.Num())
	{
		FinalReport.Reserve(FinalReport.Num() + FailedContentBundles.Num() + 4);
		FinalReport.Add(TEXT("------------------------------------------------------------------------"));
		FinalReport.Add(FString::Printf(TEXT("%d Failed Content Bundle Conversions: "), FailedContentBundles.Num()));
		FinalReport.Add(TEXT("------------------------------------------------------------------------"));
		FinalReport.Add(TEXT("[+] Content Bundles: "));
		for (const TSharedPtr<FContentBundleEditor>& CB : FailedContentBundles)
		{
			FinalReport.Add(FString::Printf(TEXT(" |- %s (%s)"), *CB->GetDisplayName(), *CB->GetDescriptor()->GetGuid().ToString()));
		}
	}

	if (FinalReport.Num())
	{
		UE_LOGF(LogConvertContentBundleBuilder, Log, "=================================================================================================");
		UE_LOGF(LogConvertContentBundleBuilder, Log, "Content Bundle Conversion : Found(%d) | Converted(%d) | Empty(%d) | Failed(%d)", ContentBundlesToProcess.Num(), ConvertedContentBundles.Num(), SkippedEmptyContentBundles.Num(), FailedContentBundles.Num());
		UE_LOGF(LogConvertContentBundleBuilder, Log, "=================================================================================================");
		for (const FString& Str : FinalReport)
		{
			UE_LOGF(LogConvertContentBundleBuilder, Log, "%ls", *Str);
		}
		UE_LOGF(LogConvertContentBundleBuilder, Log, "=================================================================================================");
	}

	return bIsSuccess;
}

UExternalDataLayerAsset* UGameFeatureActionConvertContentBundleWorldPartitionBuilder::GetOrCreateExternalDataLayerAsset(const UContentBundleDescriptor* InContentBundleDescriptor, UExternalDataLayerFactory* InExternalDataLayerFactory, TSet<UPackage*>& OutPackagesToSave) const
{
	const FString ContentBundleDescriptorPackage = InContentBundleDescriptor->GetPackage()->GetName();
	const FString AssetPath = FString(FPathViews::GetMountPointNameFromPath(ContentBundleDescriptorPackage, nullptr, false)) / DestinationFolder;
	const FString AssetName = UPackageTools::SanitizePackageName(InContentBundleDescriptor->GetDisplayName());
	const FString PackageName = AssetPath / AssetName;
	const FSoftObjectPath Path(FTopLevelAssetPath(FName(PackageName), FName(AssetName)).ToString());
	const FString ObjectPath = Path.ToString();
	if (UExternalDataLayerAsset* ExistingExternalDataLayerAsset = LoadObject<UExternalDataLayerAsset>(nullptr, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn))
	{
		return ExistingExternalDataLayerAsset;
	}
	
	UObject* Asset = IAssetTools::Get().CreateAsset(AssetName, AssetPath, UExternalDataLayerAsset::StaticClass(), InExternalDataLayerFactory);
	UExternalDataLayerAsset* NewExternalDataLayerAsset = Asset ? CastChecked<UExternalDataLayerAsset>(Asset) : nullptr;
	UE_CLOGF(!NewExternalDataLayerAsset, LogConvertContentBundleBuilder, Error, "Failed to create external data layer asset for %ls.", *InContentBundleDescriptor->GetDisplayName());
	if (NewExternalDataLayerAsset)
	{
		OutPackagesToSave.Add(NewExternalDataLayerAsset->GetPackage());
	}
	return NewExternalDataLayerAsset;
}

