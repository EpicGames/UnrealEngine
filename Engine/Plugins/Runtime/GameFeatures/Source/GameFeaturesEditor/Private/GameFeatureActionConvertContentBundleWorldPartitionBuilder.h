// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "GameFeatureActionConvertContentBundleWorldPartitionBuilder.generated.h"

class UPackage;
class UExternalDataLayerFactory;
class UExternalDataLayerAsset;
class UContentBundleDescriptor;
class FContentBundleEditor;

UCLASS()
class UGameFeatureActionConvertContentBundleWorldPartitionBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()
public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return false; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	// UWorldPartitionBuilder interface end

private:
	UExternalDataLayerAsset* GetOrCreateExternalDataLayerAsset(const UContentBundleDescriptor* InContentBundleDescriptor, UExternalDataLayerFactory* InExternalDataLayerFactory, TSet<UPackage*>& OutPackagesToSave) const;

	TSet<TSharedPtr<FContentBundleEditor>> SkippedEmptyContentBundles;
	TSet<TSharedPtr<FContentBundleEditor>> ConvertedContentBundles;
	TArray<FString> FinalReport;
	TArray<FString> ContentBundlesToConvert;
	FString DestinationFolder;
	bool bReportOnly;
	bool bRemoveContentBundleAction;
	bool bDeleteHLODs;
};