// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureDataDetailsCustomization.h"
#include "GameFeaturePluginOperationResult.h"
#include "UObject/Package.h"
#include "GameFeaturesSubsystem.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "SGameFeatureStateWidget.h"
#include "Widgets/Notifications/SErrorText.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"

#include "Interfaces/IPluginManager.h"
#include "Features/IPluginsEditorFeature.h"
#include "Features/EditorFeatures.h"
#include "Features/IModularFeatures.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"

#include "GameFeatureData.h"
#include "GameFeatureTypes.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "GameFeatures"

//////////////////////////////////////////////////////////////////////////
// FGameFeatureDataDetailsCustomization

TSharedRef<IDetailCustomization> FGameFeatureDataDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FGameFeatureDataDetailsCustomization);
}

void FGameFeatureDataDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ErrorTextWidget = SNew(SErrorText)
		.ToolTipText(LOCTEXT("ErrorTooltip", "The error raised while attempting to change the state of this feature"));

	// Create a category so this is displayed early in the properties
	IDetailCategoryBuilder& TopCategory = DetailBuilder.EditCategory("Feature State", FText::GetEmpty(), ECategoryPriority::Important);

	PluginURL.Reset();
	ObjectsBeingCustomized.Empty();
	DetailBuilder.GetObjectsBeingCustomized(/*out*/ ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() == 1 && !ObjectsBeingCustomized[0]->GetPackage()->HasAnyPackageFlags(PKG_ForDiffing))
	{
		const UGameFeatureData* GameFeature = CastChecked<const UGameFeatureData>(ObjectsBeingCustomized[0]);

		TArray<FString> PathParts;
		GameFeature->GetOutermost()->GetName().ParseIntoArray(PathParts, TEXT("/"));

		UGameFeaturesSubsystem& Subsystem = UGameFeaturesSubsystem::Get();
		Subsystem.GetPluginURLByName(PathParts[0], /*out*/ PluginURL);
		PluginPtr = IPluginManager::Get().FindPlugin(PathParts[0]);

		const float Padding = 8.0f;

		if (PluginPtr.IsValid())
		{
			const FString ShortFilename = FPaths::GetCleanFilename(PluginPtr->GetDescriptorFileName());
			FDetailWidgetRow& EditPluginRow = TopCategory.AddCustomRow(LOCTEXT("InitialStateSearchText", "Initial State Edit Plugin"))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("InitialState", "Initial State"))
					.ToolTipText(LOCTEXT("InitialStateTooltip", "The initial or default state of this game feature (determines the state that it will be in at game/editor startup)"))
					.Font(DetailBuilder.GetDetailFont())
				]

				.ValueContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, Padding, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &FGameFeatureDataDetailsCustomization::GetInitialStateText)
						.Font(DetailBuilder.GetDetailFont())
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.Text(LOCTEXT("EditPluginButton", "Edit Plugin"))
						.OnClicked_Lambda([this]()
							{
								IModularFeatures& ModularFeatures = IModularFeatures::Get();
								if (ModularFeatures.IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
								{
									ModularFeatures.GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor).OpenPluginEditor(PluginPtr.ToSharedRef(), nullptr, FSimpleDelegate());
								}
								else
								{
									FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("CannotEditPlugin_PluginBrowserDisabled", "Cannot open plugin editor because the PluginBrowser plugin is disabled)"));
								}
								return FReply::Handled();
							})
					]

				];
		}

		FDetailWidgetRow& ControlRow = TopCategory.AddCustomRow(LOCTEXT("ControlSearchText", "Plugin State Control"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CurrentState", "Current State"))
				.ToolTipText(LOCTEXT("CurrentStateTooltip", "The current state of this game feature"))
				.Font(DetailBuilder.GetDetailFont())
			]

			.ValueContent()
			.MinDesiredWidth(400.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SGameFeatureStateWidget)
					.CurrentState(this, &FGameFeatureDataDetailsCustomization::GetCurrentState)
					.OnStateChanged(this, &FGameFeatureDataDetailsCustomization::ChangeDesiredState)
				]
				+SVerticalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(0.0f, 4.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &FGameFeatureDataDetailsCustomization::GetVisbililty)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(Padding)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Lock"))
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(FMargin(0.f, Padding, Padding, Padding))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.WrapTextAt(300.0f)
						.Text(LOCTEXT("Active_PreventingEditing", "Deactivate the feature before editing the Game Feature Data"))
						.Font(DetailBuilder.GetDetailFont())
						.ColorAndOpacity(FAppStyle::Get().GetSlateColor(TEXT("Colors.AccentYellow")))
					]
				]
				+SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				[
					ErrorTextWidget.ToSharedRef()
				]
			];

		FDetailWidgetRow& TagsRow = TopCategory.AddCustomRow(LOCTEXT("TagSearchText", "Gameplay Tag Config Path"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TagConfigPath", "Gameplay Tag Config Path"))
				.ToolTipText(LOCTEXT("TagConfigPathTooltip", "Path to search for Gameplay Tag ini files. To create feature-specific tags use Add New Tag Source with this path and then Add New Gameplay Tag with that Tag Source."))
				.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			.MinDesiredWidth(400.0f)
			[
				SNew(STextBlock)
				.Text(this, &FGameFeatureDataDetailsCustomization::GetTagConfigPathText)
				.Font(DetailBuilder.GetDetailFont())
		];

		if (FPlatformMisc::IsDebuggerPresent())
		{
			const FText Label = LOCTEXT("BreakStateLabel", "Break when State changes");
			const FText Tooltip = LOCTEXT("BreakStateTooltip", "When enabled, will trigger a breakpoint any time the state of this plugin changes");
			TopCategory.AddCustomRow(Label)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(Label)
				.ToolTipText(Tooltip)
				.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked( this, &FGameFeatureDataDetailsCustomization::GetDebugStateEnabled)
				.OnCheckStateChanged( this, &FGameFeatureDataDetailsCustomization::SetDebugStateEnabled)
				.ToolTipText(Tooltip)
			];
		}
//@TODO: This disables the mode switcher widget too (and it's a const cast hack...)
// 		if (IDetailsView* ConstHackDetailsView = const_cast<IDetailsView*>(DetailBuilder.GetDetailsView()))
// 		{
// 			ConstHackDetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([CapturedThis = this] { return CapturedThis->GetCurrentState() != EGameFeaturePluginState::Active; }));
// 		}
	}
}

void FGameFeatureDataDetailsCustomization::ChangeDesiredState(EGameFeaturePluginState DesiredState)
{
	EGameFeatureTargetState TargetState = EGameFeatureTargetState::Installed;
	switch (DesiredState)
	{
	case EGameFeaturePluginState::Installed:
		TargetState = EGameFeatureTargetState::Installed;
		break;
	case EGameFeaturePluginState::Registered:
		TargetState = EGameFeatureTargetState::Registered;
		break;
	case EGameFeaturePluginState::Loaded:
		TargetState = EGameFeatureTargetState::Loaded;
		break;
	case EGameFeaturePluginState::Active:
		TargetState = EGameFeatureTargetState::Active;
		break;
	}

	ErrorTextWidget->SetError(FText::GetEmpty());
	const TWeakPtr<FGameFeatureDataDetailsCustomization> WeakThisPtr = StaticCastSharedRef<FGameFeatureDataDetailsCustomization>(AsShared());

	UGameFeaturesSubsystem& Subsystem = UGameFeaturesSubsystem::Get();
	Subsystem.ChangeGameFeatureTargetState(PluginURL, TargetState, FGameFeaturePluginDeactivateComplete::CreateStatic(&FGameFeatureDataDetailsCustomization::OnOperationCompletedOrFailed, WeakThisPtr));
}


EGameFeaturePluginState FGameFeatureDataDetailsCustomization::GetCurrentState() const
{
	if (PluginURL.IsEmpty())
	{
		return EGameFeaturePluginState::Uninitialized;
	}
	return UGameFeaturesSubsystem::Get().GetPluginState(PluginURL);
}

EVisibility FGameFeatureDataDetailsCustomization::GetVisbililty() const
{
	return (GetCurrentState() == EGameFeaturePluginState::Active) ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState FGameFeatureDataDetailsCustomization::GetDebugStateEnabled() const
{
	if (PluginURL.IsEmpty())
	{
		return ECheckBoxState::Undetermined;
	}
	return UGameFeaturesSubsystem::Get().GetPluginDebugStateEnabled(PluginURL) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FGameFeatureDataDetailsCustomization::SetDebugStateEnabled(ECheckBoxState InCheckState)
{
	if (!PluginURL.IsEmpty())
	{
		return UGameFeaturesSubsystem::Get().SetPluginDebugStateEnabled(PluginURL, InCheckState == ECheckBoxState::Checked);
	}
}

FText FGameFeatureDataDetailsCustomization::GetInitialStateText() const
{
	const EBuiltInAutoState AutoState = UGameFeaturesSubsystem::DetermineBuiltInInitialFeatureState(PluginPtr->GetDescriptor().CachedJson, FString());
	const EGameFeaturePluginState InitialState = UGameFeaturesSubsystem::ConvertInitialFeatureStateToTargetState(AutoState);
	return SGameFeatureStateWidget::GetDisplayNameOfState(InitialState);
}

FText FGameFeatureDataDetailsCustomization::GetTagConfigPathText() const
{
	if (PluginURL.IsEmpty())
	{
		return LOCTEXT("TagConfigPathInvalid", "Invalid Plugin");
	}

	FString PluginFile = UGameFeaturesSubsystem::Get().GetPluginFilenameFromPluginURL(PluginURL);
	FString PluginFolder = FPaths::GetPath(PluginFile);
	FString TagFolder = PluginFolder / TEXT("Config") / TEXT("Tags");
	if (FPaths::IsUnderDirectory(TagFolder, FPaths::ProjectDir()))
	{
		FPaths::MakePathRelativeTo(TagFolder, *FPaths::ProjectDir());
	}
	return FText::AsCultureInvariant(TagFolder);
}

void FGameFeatureDataDetailsCustomization::OnOperationCompletedOrFailed(const UE::GameFeatures::FResult& Result, const TWeakPtr<FGameFeatureDataDetailsCustomization> WeakThisPtr)
{
	if (Result.HasError())
	{
		TSharedPtr<FGameFeatureDataDetailsCustomization> StrongThis = WeakThisPtr.Pin();
		if (StrongThis.IsValid())
		{
			StrongThis->ErrorTextWidget->SetError(FText::AsCultureInvariant(Result.GetError()));
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
