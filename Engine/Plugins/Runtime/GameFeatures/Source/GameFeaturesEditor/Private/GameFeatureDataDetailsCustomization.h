// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "GameFeatureTypesFwd.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::GameFeatures { struct FResult; }

class IDetailLayoutBuilder;
class SErrorText;
class IPlugin;
struct EVisibility;

enum class ECheckBoxState : uint8;

//////////////////////////////////////////////////////////////////////////
// FGameFeatureDataDetailsCustomization

class FGameFeatureDataDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

protected:
	void ChangeDesiredState(EGameFeaturePluginState State);

	EGameFeaturePluginState GetCurrentState() const;
	EVisibility GetVisbililty() const;

	ECheckBoxState GetDebugStateEnabled() const;
	void SetDebugStateEnabled(ECheckBoxState InCheckState);
	
	FText GetInitialStateText() const;
	FText GetTagConfigPathText() const;

	static void OnOperationCompletedOrFailed(const UE::GameFeatures::FResult& Result, const TWeakPtr<FGameFeatureDataDetailsCustomization> WeakThisPtr);
protected:
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	FString PluginURL;
	TSharedPtr<IPlugin> PluginPtr;

	TSharedPtr<SErrorText> ErrorTextWidget;
};
