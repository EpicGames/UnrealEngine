// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturePluginMetadataCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dom/JsonValue.h"
#include "Interfaces/IPluginManager.h"
#include "SGameFeatureStateWidget.h"

#include "GameFeaturesSubsystem.h"
#include "GameFeatureTypes.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "GameFeatures"

//////////////////////////////////////////////////////////////////////////
// FGameFeaturePluginMetadataCustomization

void FGameFeaturePluginMetadataCustomization::CustomizeDetails(FPluginEditingContext& InPluginContext, IDetailLayoutBuilder& DetailBuilder)
{
	Plugin = InPluginContext.PluginBeingEdited;
	
	const EBuiltInAutoState AutoState = UGameFeaturesSubsystem::DetermineBuiltInInitialFeatureState(Plugin->GetDescriptor().CachedJson, FString());
	InitialState = UGameFeaturesSubsystem::ConvertInitialFeatureStateToTargetState(AutoState);

	IDetailCategoryBuilder& TopCategory = DetailBuilder.EditCategory("Game Features", FText::GetEmpty(), ECategoryPriority::Important);

	FDetailWidgetRow& ControlRow = TopCategory.AddCustomRow(LOCTEXT("ControlSearchText", "Plugin State Control"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("InitialState", "Initial State"))
			.Font(DetailBuilder.GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SGameFeatureStateWidget)
			.ToolTipText(LOCTEXT("DefaultStateSwitcherTooltip", "Change the default initial state of this game feature"))
			.CurrentState(this, &FGameFeaturePluginMetadataCustomization::GetDefaultState)
			.OnStateChanged(this, &FGameFeaturePluginMetadataCustomization::ChangeDefaultState)
		];
}

void FGameFeaturePluginMetadataCustomization::CommitEdits(FPluginDescriptor& Descriptor)
{
	FString StateStr;
	switch (InitialState)
	{
	case EGameFeaturePluginState::Installed:
		StateStr = TEXT("Installed");
		break;
	case EGameFeaturePluginState::Registered:
		StateStr = TEXT("Registered");
		break;
	case EGameFeaturePluginState::Loaded:
		StateStr = TEXT("Loaded");
		break;
	case EGameFeaturePluginState::Active:
		StateStr = TEXT("Active");
		break;
	}

	if (ensure(!StateStr.IsEmpty()))
	{
		Descriptor.AdditionalFieldsToWrite.FindOrAdd(TEXT("BuiltInInitialFeatureState")) = MakeShared<FJsonValueString>(StateStr);
	}
}

EGameFeaturePluginState FGameFeaturePluginMetadataCustomization::GetDefaultState() const
{
	return InitialState;
}

void FGameFeaturePluginMetadataCustomization::ChangeDefaultState(EGameFeaturePluginState DesiredState)
{
	InitialState = DesiredState;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
