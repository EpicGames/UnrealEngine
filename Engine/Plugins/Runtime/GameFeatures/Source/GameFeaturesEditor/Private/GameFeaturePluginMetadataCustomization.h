// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IPluginsEditorFeature.h"
#include "GameFeatureTypesFwd.h"

class IDetailLayoutBuilder;
struct FPluginEditingContext;
class IPlugin;
struct FPluginDescriptor;

//////////////////////////////////////////////////////////////////////////
// FGameFeaturePluginMetadataCustomization

class FGameFeaturePluginMetadataCustomization : public FPluginEditorExtension
{
public:
	void CustomizeDetails(FPluginEditingContext& InPluginContext, IDetailLayoutBuilder& DetailBuilder);

	virtual void CommitEdits(FPluginDescriptor& Descriptor) override;
private:
	EGameFeaturePluginState GetDefaultState() const;

	void ChangeDefaultState(EGameFeaturePluginState DesiredState);

	TSharedPtr<IPlugin> Plugin;

	EGameFeaturePluginState InitialState;
};
