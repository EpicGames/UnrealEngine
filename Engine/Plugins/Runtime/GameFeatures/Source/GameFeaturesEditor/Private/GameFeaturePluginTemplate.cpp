// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeaturePluginTemplate.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "Dom/JsonValue.h"
#include "Editor.h"
#include "GameFeaturesSubsystem.h"
#include "GameFeaturesSubsystemSettings.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "GameFeatures"

FGameFeaturePluginTemplateDescription::FGameFeaturePluginTemplateDescription(FText InName, FText InDescription, FString InOnDiskPath, FString InDefaultSubfolder, FString InDefaultPluginName, TSubclassOf<UGameFeatureData> GameFeatureDataClassOverride, FString GameFeatureDataNameOverride, EPluginEnabledByDefault InEnabledByDefault, FString InPostCreatePythonScriptPath, FString InPostCreatePythonScriptArguments)
	: FPluginTemplateDescription(InName, InDescription, InOnDiskPath, /*bCanContainContent=*/ true, EHostType::Runtime, ELoadingPhase::Default, /*InSupportsContentOnlyProjects=*/ true, InPostCreatePythonScriptPath, InPostCreatePythonScriptArguments)
{
	SortPriority = 10;
	bCanBePlacedInEngine = false;
	DefaultSubfolder = InDefaultSubfolder;
	DefaultPluginName = InDefaultPluginName;
	GameFeatureDataName = !GameFeatureDataNameOverride.IsEmpty() ? GameFeatureDataNameOverride : FString();
	GameFeatureDataClass = GameFeatureDataClassOverride != nullptr ? GameFeatureDataClassOverride : TSubclassOf<UGameFeatureData>(UGameFeatureData::StaticClass());
	PluginEnabledByDefault = InEnabledByDefault;
}

bool FGameFeaturePluginTemplateDescription::ValidatePathForPlugin(const FString& ProposedAbsolutePluginPath, FText& OutErrorMessage)
{
	if (!IsRootedInGameFeaturesRoot(ProposedAbsolutePluginPath))
	{
		OutErrorMessage = LOCTEXT("InvalidPathForGameFeaturePlugin", "Game features must be inside the Plugins/GameFeatures folder");
		return false;
	}

	OutErrorMessage = FText::GetEmpty();
	return true;
}

void FGameFeaturePluginTemplateDescription::UpdatePathWhenTemplateSelected(FString& InOutPath)
{
	if (!IsRootedInGameFeaturesRoot(InOutPath))
	{
		InOutPath = GetGameFeatureRoot();
	}
}

void FGameFeaturePluginTemplateDescription::UpdatePathWhenTemplateUnselected(FString& InOutPath)
{
	InOutPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::ProjectPluginsDir());
	FPaths::MakePlatformFilename(InOutPath);
}

void FGameFeaturePluginTemplateDescription::UpdatePluginNameTextWhenTemplateSelected(FText& OutPluginNameText)
{
	OutPluginNameText = FText::FromString(DefaultPluginName);
}

void FGameFeaturePluginTemplateDescription::UpdatePluginNameTextWhenTemplateUnselected(FText& OutPluginNameText)
{
	OutPluginNameText = FText::GetEmpty();
}

void FGameFeaturePluginTemplateDescription::CustomizeDescriptorBeforeCreation(FPluginDescriptor& Descriptor)
{
	Descriptor.bExplicitlyLoaded = true;
	Descriptor.AdditionalFieldsToWrite.FindOrAdd(TEXT("BuiltInInitialFeatureState")) = MakeShared<FJsonValueString>(TEXT("Active"));
	Descriptor.Category = TEXT("Game Features");

	// Game features should not be enabled by default if the game wants to strictly manage default settings in the target settings
	Descriptor.EnabledByDefault = PluginEnabledByDefault;

	if (Descriptor.Modules.Num() > 0)
	{
		Descriptor.Modules[0].Name = FName(*(Descriptor.Modules[0].Name.ToString() + TEXT("Runtime")));
	}
}

void FGameFeaturePluginTemplateDescription::OnPluginCreated(TSharedPtr<IPlugin> NewPlugin)
{
	// If the template includes an existing game feature data, do not create a new one.
	TArray<FAssetData> ObjectList;
	FARFilter AssetFilter;
	AssetFilter.ClassPaths.Add(UGameFeatureData::StaticClass()->GetClassPathName());
	AssetFilter.PackagePaths.Add(FName(NewPlugin->GetMountedAssetPath()));
	AssetFilter.bRecursiveClasses = true;
	AssetFilter.bRecursivePaths = true;

	IAssetRegistry::GetChecked().GetAssets(AssetFilter, ObjectList);

	UObject* GameFeatureDataAsset = nullptr;

	if (ObjectList.Num() <= 0)
	{
		// Create the game feature data asset
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FString const& AssetName = !GameFeatureDataName.IsEmpty() ? GameFeatureDataName : NewPlugin->GetName();
		GameFeatureDataAsset = AssetToolsModule.Get().CreateAsset(AssetName, NewPlugin->GetMountedAssetPath(), GameFeatureDataClass, /*Factory=*/ nullptr);
	}
	else
	{
		GameFeatureDataAsset = ObjectList[0].GetAsset();
	}


	// Activate the new game feature plugin
	auto AdditionalFilter = [](const FString&, const FGameFeaturePluginDetails&, FBuiltInGameFeaturePluginBehaviorOptions&) -> bool { return true; };
	UGameFeaturesSubsystem::Get().LoadBuiltInGameFeaturePlugin(NewPlugin.ToSharedRef(), AdditionalFilter,
		FGameFeaturePluginLoadComplete::CreateLambda([GameFeatureDataAsset](const UE::GameFeatures::FResult&)
			{
				// Edit the new game feature data
				if (GameFeatureDataAsset != nullptr)
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(GameFeatureDataAsset);
				}
			}));
}

FString FGameFeaturePluginTemplateDescription::GetGameFeatureRoot() const
{
	FString Result = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(FPaths::ProjectPluginsDir() / TEXT("GameFeatures/")));

	// Append the optional subfolder if specified.
	if (!DefaultSubfolder.IsEmpty())
	{
		Result /= DefaultSubfolder + TEXT("/");
	}

	FPaths::MakePlatformFilename(Result);
	return Result;
}

bool FGameFeaturePluginTemplateDescription::IsRootedInGameFeaturesRoot(const FString& InStr) const
{
	const FString ConvertedPath = FPaths::ConvertRelativePathToFull(FPaths::CreateStandardFilename(InStr / TEXT("test.uplugin")));
	return GetDefault<UGameFeaturesSubsystemSettings>()->IsValidGameFeaturePlugin(ConvertedPath);
}

#undef LOCTEXT_NAMESPACE