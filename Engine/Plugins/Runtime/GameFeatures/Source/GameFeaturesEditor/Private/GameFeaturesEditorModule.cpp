// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/AssetManager.h"
#include "Engine/AssetManagerSettings.h"
#include "Features/EditorFeatures.h"
#include "Features/IModularFeatures.h"
#include "Features/IPluginsEditorFeature.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFeatureData.h"
#include "GameFeatureDataDetailsCustomization.h"
#include "GameFeaturePluginMetadataCustomization.h"
#include "GameFeaturePluginTemplate.h"
#include "GameFeaturesEditorSettings.h"
#include "GameFeaturesSubsystem.h"
#include "GameFeaturesSubsystemSettings.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/MessageLog.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "GameFeatures"

static FAutoConsoleCommand CCmdGFPPrintActionUsageStats(
	TEXT("GameFeaturePlugin.PrintActionUsage"),
	TEXT("List all uses of Game Feature Actions")
	TEXT("\nUsage: GameFeaturePlugin.PrintActionUsage [-Summary] [-Detailed [-Csv]]")
	TEXT("\n-Summary - Print the number of Game Feature Plugins and the number of uses found for each Game Feature Action")
	TEXT("\n-Details - Print the list of Game Feature Plugins and the list of plugins that use each Game Feature Action")
	TEXT("\n-Csv     - Prints the output in CSV format. Only affects -Detailed output."),
	FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
	{
		const UGameFeaturesSubsystem& GameFeatures = UGameFeaturesSubsystem::Get();

		// Gather disabled GFPs
		TArray<const FString*> DisabledPlugins;
		TArray<const FString*> DisallowedPlugins;
		{
			DisabledPlugins.Reserve(32);
			DisallowedPlugins.Reserve(32);

			IPluginManager& PluginManager = IPluginManager::Get();
			const UGameFeaturesSubsystemSettings& Settings = *GetDefault<UGameFeaturesSubsystemSettings>();

			TArray<TSharedRef<IPlugin>> Plugins = PluginManager.GetDiscoveredPlugins();
			for (const TSharedRef<IPlugin>& Plugin : Plugins)
			{
				const FString& DescriptorFileName = Plugin->GetDescriptorFileName();
				if (Settings.IsValidGameFeaturePlugin(DescriptorFileName))
				{
					if (!Plugin->IsEnabled())
					{
						DisabledPlugins.Add(&Plugin->GetName());
					}
					else
					{
						FString PluginURL;
						GameFeatures.GetBuiltInGameFeaturePluginURL(Plugin, PluginURL);
						if (!GameFeatures.IsPluginAllowed(PluginURL))
						{
							DisallowedPlugins.Add(&Plugin->GetName());
						}
					}
				}
			}
		}

		// Gather actions
		int32 RegisteredCount = 0;
		TMap<const UClass*, TArray<FString>> ActionUses;
		{
			ActionUses.Reserve(128);

			auto VisitGFP = [&RegisteredCount, &ActionUses](const UGameFeatureData* Data)
			{
				RegisteredCount++;

				FString PluginName;
				Data->GetPluginName(PluginName);

				const TArray<UGameFeatureAction*>& Actions = Data->GetActions();
				for (const UGameFeatureAction* Action : Actions)
				{
					const UClass* Class = Action->GetClass();
					TArray<FString>& Plugins = ActionUses.FindOrAdd(Class);
					Plugins.Add(PluginName);
				}
			};

			GameFeatures.ForEachRegisteredGameFeature<UGameFeatureData>(VisitGFP);

			ActionUses.ValueSort([](const TArray<FString>& A, const TArray<FString>& B) { return A.Num() > B.Num(); });
		}

		// Output
		{
			bool bDetails = Args.Contains(TEXT("-Details"));
			bool bSummary = Args.Contains(TEXT("-Summary")) || !bDetails;
			bool bCsv     = Args.Contains(TEXT("-Csv"));

			TStringBuilder<1024> Msg;
			Msg << TEXT("Game Feature Action usage\n");

			if (bSummary)
			{
				if (!DisabledPlugins.IsEmpty())
				{
					Msg << TEXT("Disabled Game Feature Plugins (Action usage for these will not be displayed): ") << DisabledPlugins.Num() << TEXT("\n");
				}

				if (!DisallowedPlugins.IsEmpty())
				{
					Msg << TEXT("Disallowed built-in Game Feature Plugins (Action usage for these will not be displayed): ") << DisallowedPlugins.Num() << TEXT("\n");
				}

				Msg << TEXT("Registered Game Feature Plugins: ") << RegisteredCount << TEXT("\n");
				if (!ActionUses.IsEmpty())
				{
					Msg << TEXT("Game Feature Action usage:\n{\n");
					for (TPair<const UClass*, TArray<FString>>& Pair : ActionUses)
					{
						Msg << TEXT("\t") << Pair.Key->GetFName() << TEXT(": ") << Pair.Value.Num() << TEXT("\n");
					}
					Msg << TEXT("}\n");
				}
				else
				{
					Msg << TEXT("No Game Feature Actions found\n");
				}
			}

			if (bDetails)
			{
				if (bCsv)
				{
					Msg << TEXT("Plugin Name,Action Class\n");

					if (!DisabledPlugins.IsEmpty())
					{
						for (const FString* PluginName : DisabledPlugins)
						{
							Msg << *PluginName << TEXT(",Disabled\n");
						}
					}

					if (!DisallowedPlugins.IsEmpty())
					{
						for (const FString* PluginName : DisallowedPlugins)
						{
							Msg << *PluginName << TEXT(",Disallowed\n");
						}
					}

					for (TPair<const UClass*, TArray<FString>>& Pair : ActionUses)
					{
						FName ActionName = Pair.Key->GetFName();
						for (const FString& PluginName : Pair.Value)
						{
							Msg << PluginName << TEXT(",") << ActionName << TEXT("\n");
						}
					}
				}
				else
				{
					if (!DisabledPlugins.IsEmpty())
					{
						Msg << TEXT("Disabled Game Feature Plugins (Action usage for these will not be displayed):\n{\n");
						for (const FString* PluginName : DisabledPlugins)
						{
							Msg << TEXT("\t") << *PluginName << TEXT("\n");
						}
						Msg << TEXT("}\n");
					}

					if (!DisallowedPlugins.IsEmpty())
					{
						Msg << TEXT("Disallowed built-in Game Feature Plugins (Action usage for these will not be displayed):\n{\n");
						for (const FString* PluginName : DisallowedPlugins)
						{
							Msg << TEXT("\t") << *PluginName << TEXT("\n");
						}
						Msg << TEXT("}\n");
					}

					Msg << TEXT("Game Feature Action usage:\n{\n");
					for (TPair<const UClass*, TArray<FString>>& Pair : ActionUses)
					{
						FName ActionName = Pair.Key->GetFName();

						Msg << TEXT("\t") << ActionName << TEXT(":\n\t{\n");
						for (const FString& PluginName : Pair.Value)
						{
							Msg << TEXT("\t\t") << PluginName << TEXT("\n");
						}
						Msg << TEXT("\t}\n");
					}
					Msg << TEXT("}\n");
				}
			}

			if (Msg.Len())
			{
				Msg.RemoveSuffix(1);
			}
			UE_LOGF(LogGameFeatures, Display, "%ls", Msg.ToString());
		}
	})
);

class FGameFeaturesEditorModule : public FDefaultModuleImpl
{
	virtual void StartupModule() override
	{
		// Register the details customizations
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomClassLayout(UGameFeatureData::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGameFeatureDataDetailsCustomization::MakeInstance));

			PropertyModule.NotifyCustomizationModuleChanged();
		}

		// Register to get a warning on startup if settings aren't configured correctly
		UAssetManager::CallOrRegister_OnAssetManagerCreated(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FGameFeaturesEditorModule::OnAssetManagerCreated));

		// Add templates to the new plugin wizard
		{
			GameFeaturesEditorSettingsWatcher = MakeShared<FGameFeaturesEditorSettingsWatcher>(this);
			GameFeaturesEditorSettingsWatcher->Init();

			CachePluginTemplates();

			IModularFeatures& ModularFeatures = IModularFeatures::Get();
			ModularFeatures.OnModularFeatureRegistered().AddRaw(this, &FGameFeaturesEditorModule::OnModularFeatureRegistered);
			ModularFeatures.OnModularFeatureUnregistered().AddRaw(this, &FGameFeaturesEditorModule::OnModularFeatureUnregistered);

			if (ModularFeatures.IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				OnModularFeatureRegistered(EditorFeatures::PluginsEditor, &ModularFeatures.GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor));
			}
		}
	}

	virtual void ShutdownModule() override
	{
		// Remove the customization
		if (UObjectInitialized() && FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout(UGameFeatureData::StaticClass()->GetFName());

			PropertyModule.NotifyCustomizationModuleChanged();
		}

		// Remove the plugin wizard override
		if (UObjectInitialized())
		{
			GameFeaturesEditorSettingsWatcher = nullptr;

			IModularFeatures& ModularFeatures = IModularFeatures::Get();
 			ModularFeatures.OnModularFeatureRegistered().RemoveAll(this);
 			ModularFeatures.OnModularFeatureUnregistered().RemoveAll(this);

			if (ModularFeatures.IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
			{
				OnModularFeatureUnregistered(EditorFeatures::PluginsEditor, &ModularFeatures.GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor));
			}
			UnregisterFunctionTemplates();
			PluginTemplates.Empty();
		}
	}

	void OnSettingsChanged(UObject* Settings, FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FName PropertyName = PropertyChangedEvent.GetPropertyName();
		const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
		const FName PluginTemplatePropertyName = GET_MEMBER_NAME_CHECKED(UGameFeaturesEditorSettings, PluginTemplates);
		if (PropertyName == PluginTemplatePropertyName
			|| MemberPropertyName == PluginTemplatePropertyName)
		{
			ResetPluginTemplates();
		}
	}

	void CachePluginTemplates()
	{
		PluginTemplates.Reset();
		if (const UGameFeaturesEditorSettings* GameFeatureEditorSettings = GetDefault<UGameFeaturesEditorSettings>())
		{
			for (const FPluginTemplateData& PluginTemplate : GameFeatureEditorSettings->PluginTemplates)
			{
				PluginTemplates.Add(MakeShareable(new FGameFeaturePluginTemplateDescription(
					PluginTemplate.Label,
					PluginTemplate.Description,
					PluginTemplate.Path.Path,
					PluginTemplate.DefaultSubfolder,
					PluginTemplate.DefaultPluginName,
					PluginTemplate.DefaultGameFeatureDataClass,
					PluginTemplate.DefaultGameFeatureDataName,
					PluginTemplate.bIsEnabledByDefault ? EPluginEnabledByDefault::Enabled : EPluginEnabledByDefault::Disabled,
					PluginTemplate.PostCreatePythonScriptPath.FilePath,
					PluginTemplate.PostCreatePythonScriptArguments)));
			}
		}
	}

	void ResetPluginTemplates()
	{
		UnregisterFunctionTemplates();
		CachePluginTemplates();
		RegisterPluginTemplates();
	}

	void RegisterPluginTemplates()
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
		{
			IPluginsEditorFeature& PluginEditor = IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
			for (const TSharedPtr<FGameFeaturePluginTemplateDescription, ESPMode::ThreadSafe>& TemplateDescription : PluginTemplates)
			{
				PluginEditor.RegisterPluginTemplate(TemplateDescription.ToSharedRef());
			}
			PluginEditorExtensionDelegate = PluginEditor.RegisterPluginEditorExtension(FOnPluginBeingEdited::CreateRaw(this, &FGameFeaturesEditorModule::CustomizePluginEditing));
		}
	}

	void UnregisterFunctionTemplates()
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(EditorFeatures::PluginsEditor))
		{
			IPluginsEditorFeature& PluginEditor = IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
			for (const TSharedPtr<FGameFeaturePluginTemplateDescription, ESPMode::ThreadSafe>& TemplateDescription : PluginTemplates)
			{
				PluginEditor.UnregisterPluginTemplate(TemplateDescription.ToSharedRef());
			}
			PluginEditor.UnregisterPluginEditorExtension(PluginEditorExtensionDelegate);
		}
		
	}

	void OnModularFeatureRegistered(const FName& Type, class IModularFeature* ModularFeature)
	{
		if (Type == EditorFeatures::PluginsEditor)
		{
			ResetPluginTemplates();
		}
	}

	void OnModularFeatureUnregistered(const FName& Type, class IModularFeature* ModularFeature)
	{
		if (Type == EditorFeatures::PluginsEditor)
		{
			UnregisterFunctionTemplates();
		}
	}

	void AddDefaultGameDataRule()
	{
		// Check out the ini or make it writable
		UAssetManagerSettings* Settings = GetMutableDefault<UAssetManagerSettings>();

		const FString& ConfigFileName = Settings->GetDefaultConfigFilename();

		bool bSuccess = false;

		FText NotificationOpText;
 		if (!SettingsHelpers::IsCheckedOut(ConfigFileName, true))
 		{
			FText ErrorMessage;
			bSuccess = SettingsHelpers::CheckOutOrAddFile(ConfigFileName, true, !IsRunningCommandlet(), &ErrorMessage);
			if (bSuccess)
			{
				NotificationOpText = LOCTEXT("CheckedOutAssetManagerIni", "Checked out {0}");
			}
			else
			{
				UE_LOGF(LogGameFeatures, Error, "%ls", *ErrorMessage.ToString());
				bSuccess = SettingsHelpers::MakeWritable(ConfigFileName);

				if (bSuccess)
				{
					NotificationOpText = LOCTEXT("MadeWritableAssetManagerIni", "Made {0} writable (you may need to manually add to revision control)");
				}
				else
				{
					NotificationOpText = LOCTEXT("FailedToTouchAssetManagerIni", "Failed to check out {0} or make it writable, so no rule was added");
				}
			}
		}
		else
		{
			NotificationOpText = LOCTEXT("UpdatedAssetManagerIni", "Updated {0}");
			bSuccess = true;
		}

		// Add the rule to project settings
		if (bSuccess)
		{
			FPrimaryAssetTypeInfo NewTypeInfo(
				UGameFeatureData::StaticClass()->GetFName(),
				UGameFeatureData::StaticClass(),
				false,
				false);
			NewTypeInfo.Rules.CookRule = EPrimaryAssetCookRule::AlwaysCook;

			Settings->Modify(true);

			Settings->PrimaryAssetTypesToScan.Add(NewTypeInfo);

 			Settings->PostEditChange();
			Settings->TryUpdateDefaultConfigFile();

			UAssetManager::Get().ReinitializeFromConfig();
		}

		// Show a message that the file was checked out/updated and must be submitted
		FNotificationInfo Info(FText::Format(NotificationOpText, FText::FromString(FPaths::GetCleanFilename(ConfigFileName))));
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	void OnAssetManagerCreated()
	{
		// Make sure the game has the appropriate asset manager configuration or we won't be able to load game feature data assets
		FPrimaryAssetId DummyGameFeatureDataAssetId(UGameFeatureData::StaticClass()->GetFName(), NAME_None);
		FPrimaryAssetRules GameDataRules = UAssetManager::Get().GetPrimaryAssetRules(DummyGameFeatureDataAssetId);
		if (FApp::HasProjectName() && GameDataRules.IsDefault())
		{
			FMessageLog("LoadErrors").Error()
				->AddToken(FTextToken::Create(FText::Format(NSLOCTEXT("GameFeatures", "MissingRuleForGameFeatureData", "Asset Manager settings do not include an entry for assets of type {0}, which is required for game feature plugins to function."), FText::FromName(UGameFeatureData::StaticClass()->GetFName()))))
				->AddToken(FActionToken::Create(NSLOCTEXT("GameFeatures", "AddRuleForGameFeatureData", "Add entry to PrimaryAssetTypesToScan?"), FText(),
					FOnActionTokenExecuted::CreateRaw(this, &FGameFeaturesEditorModule::AddDefaultGameDataRule), true));
		}
	}

	TSharedPtr<FPluginEditorExtension> CustomizePluginEditing(FPluginEditingContext& InPluginContext, IDetailLayoutBuilder& DetailBuilder)
	{
		const bool bIsGameFeaturePlugin = InPluginContext.PluginBeingEdited->GetDescriptorFileName().Contains(TEXT("/GameFeatures/"));
		if (bIsGameFeaturePlugin)
		{
			TSharedPtr<FGameFeaturePluginMetadataCustomization> Result = MakeShareable(new FGameFeaturePluginMetadataCustomization);
			Result->CustomizeDetails(InPluginContext, DetailBuilder);
			return Result;
		}

		return nullptr;
	}
private:

	struct FGameFeaturesEditorSettingsWatcher : public TSharedFromThis<FGameFeaturesEditorSettingsWatcher>
	{
		FGameFeaturesEditorSettingsWatcher(FGameFeaturesEditorModule* InParentModule)
			: ParentModule(InParentModule)
		{

		}

		void Init()
		{
			GetMutableDefault<UGameFeaturesEditorSettings>()->OnSettingChanged().AddSP(this, &FGameFeaturesEditorSettingsWatcher::OnSettingsChanged);
		}

		void OnSettingsChanged(UObject* Settings, FPropertyChangedEvent& PropertyChangedEvent)
		{
			if (ParentModule != nullptr)
			{
				ParentModule->OnSettingsChanged(Settings, PropertyChangedEvent);
			}
		}
	private:

		FGameFeaturesEditorModule* ParentModule;
	};

	TSharedPtr<FGameFeaturesEditorSettingsWatcher> GameFeaturesEditorSettingsWatcher;

	// Array of Plugin templates populated from GameFeatureDeveloperSettings. Allows projects to
	//	specify reusable plugin templates for the plugin creation wizard.
	TArray<TSharedPtr<FGameFeaturePluginTemplateDescription>> PluginTemplates;
	FPluginEditorExtensionHandle PluginEditorExtensionDelegate;
};

IMPLEMENT_MODULE(FGameFeaturesEditorModule, GameFeaturesEditor)

#undef LOCTEXT_NAMESPACE
