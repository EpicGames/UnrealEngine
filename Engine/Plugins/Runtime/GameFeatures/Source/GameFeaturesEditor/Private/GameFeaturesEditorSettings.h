// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Engine/DeveloperSettings.h"
#include "Templates/SubclassOf.h"
#include "GameFeaturesEditorSettings.generated.h"

class UGameFeatureData;

/*
* Data for specifying a usable plugin template. 
*	-Plugin templates are a folder/file structure that are duplicated and renamed
*	 by the plugin creation wizard to easily create new plugins with a standard
*	 format.
* See PluginUtils.h for more information.
*/
USTRUCT()
struct FPluginTemplateData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = PluginTemplate, meta = (RelativePath))
	FDirectoryPath Path;

	UPROPERTY(EditAnywhere, Category = PluginTemplate)
	FText Label;

	UPROPERTY(EditAnywhere, Category = PluginTemplate)
	FText Description;

	/** Optional sub folder that new plugins will be created in. */
	UPROPERTY(EditAnywhere, Category = PluginTemplate)
	FString DefaultSubfolder;

	/** Optional plugin name to default the new plugin to. */
	UPROPERTY(EditAnywhere, Category = PluginTemplate)
	FString DefaultPluginName;

	/** The default class of game feature data to create for new game feature plugins (if not set, UGameFeatureData will be used) */
	UPROPERTY(config, EditAnywhere, Category = Plugins)
	TSubclassOf<UGameFeatureData> DefaultGameFeatureDataClass;

	/** The default name of the created game feature data assets. If empty, will use the plugin name. */
	UPROPERTY(config, EditAnywhere, Category = Plugins)
	FString DefaultGameFeatureDataName;

	/** If true, the created plugin will be enabled by default without needing to be added to the project file. */
	UPROPERTY(config, EditAnywhere, Category = Plugins)
	bool bIsEnabledByDefault = false;
	
	/** Optional path to a post create python script that will run when new plugins are created from this template. */
	UPROPERTY(config, EditAnywhere, Category = Plugins, meta = (RelativePath))
	FFilePath PostCreatePythonScriptPath;
	
	/** Optional post create python script arguments. */
	UPROPERTY(config, EditAnywhere, Category = Plugins)
	FString PostCreatePythonScriptArguments;

};


UCLASS(MinimalAPI, config = Editor, defaultconfig)
class UGameFeaturesEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// Array of Plugin templates. Allows projects to specify reusable plugin templates for the plugin creation wizard.
	UPROPERTY(config, EditAnywhere, Category = Plugins)
	TArray<FPluginTemplateData> PluginTemplates;
};
