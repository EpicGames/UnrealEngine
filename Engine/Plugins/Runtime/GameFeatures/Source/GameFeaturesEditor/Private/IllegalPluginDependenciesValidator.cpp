// Copyright Epic Games, Inc. All Rights Reserved.

#include "IllegalPluginDependenciesValidator.h"

#include "DataValidationChangelist.h"
#include "GameFeaturesSubsystem.h"
#include "Interfaces/IPluginManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IllegalPluginDependenciesValidator)

#define LOCTEXT_NAMESPACE "IllegalPluginDependenciesValidator"

bool UIllegalPluginDependenciesValidator::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	return (InAsset->GetClass() == UDataValidationChangelist::StaticClass());
}

EDataValidationResult UIllegalPluginDependenciesValidator::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	UDataValidationChangelist* DataValidationChangelist = CastChecked<UDataValidationChangelist>(InAsset);

	bool bChangelistContainsPluginFiles = false;
	for (const FString& ModifiedFile : DataValidationChangelist->ModifiedFiles)
	{
		if (ModifiedFile.EndsWith(TEXT(".uplugin")))
		{
			bChangelistContainsPluginFiles = true;
			break;
		}
	}

	// Early out if the changelist doesn't contain any uplugin files.
	if (!bChangelistContainsPluginFiles)
	{
		AssetPasses(InAsset);
		return GetValidationResult();
	}

	TSet<FString> GFPs;
	TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();
	GFPs.Reserve(AllPlugins.Num());
	for (const TSharedRef<IPlugin>& Plugin : AllPlugins)
	{
		FGameFeaturePluginDetails PluginDetails;
		if (UGameFeaturesSubsystem::Get().GetBuiltInGameFeaturePluginDetails(Plugin, PluginDetails))
		{
			GFPs.Add(Plugin->GetName());
		}
	}

	for (const FString& ModifiedFile : DataValidationChangelist->ModifiedFiles)
	{
		if (ModifiedFile.EndsWith(TEXT(".uplugin")))
		{
			FString ModifiedPlugin = FPaths::GetBaseFilename(ModifiedFile);
			TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(ModifiedPlugin);
			if (Plugin.IsValid())
			{
				if (GFPs.Contains(ModifiedPlugin) || UGameFeaturesSubsystem::IsGameFeaturePlugin(Plugin))
				{
					continue;
				}
				if (!FPaths::IsUnderDirectory(Plugin->GetDescriptorFileName(), FPaths::EnginePluginsDir()) && !FPaths::IsUnderDirectory(Plugin->GetDescriptorFileName(), FPaths::ProjectPluginsDir()))
				{
					// do not validate plugins outside of this project.
					continue;
				}

				// Plugin is not a GFP so it can not have any GFP dependencies
				const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
				for (const FPluginReferenceDescriptor& Dependency : Descriptor.Plugins)
				{
					TSharedPtr<IPlugin> DependencyPlugin = IPluginManager::Get().FindPlugin(Dependency.Name);
					if (DependencyPlugin.IsValid())
					{
						if (GFPs.Contains(Dependency.Name))
						{
							FText NewError = FText::Format(
								LOCTEXT("ValidationError.IllegalPluginDependency", "Plugin {0} depends on {1}. Non GameFeaturePlugins are not allowed to depend on GameFeaturePlugins. This can create an issue where objects will fail to load"),
								FText::FromString(ModifiedPlugin),
								FText::FromString(Dependency.Name)
							);
							AssetFails(InAsset, NewError);
						}
					}
				}
			}
		}
	}

	if (GetValidationResult() != EDataValidationResult::Invalid)
	{
		AssetPasses(InAsset);
	}

	return GetValidationResult();
}
#undef LOCTEXT_NAMESPACE