// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"
#include "IllegalPluginDependenciesValidator.generated.h"

class FText;
class UObject;

/**
 * Ensures that non-GameFeaturePlugins do not depend on GameFeaturePlugins.
 * GameFeaturePlugins will load content later than non-GameFeaturePlugins which could cause linker load issues if they do not exist.
 */
UCLASS()
class UIllegalPluginDependenciesValidator : public UEditorValidatorBase
{
	GENERATED_BODY()

protected:
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
};