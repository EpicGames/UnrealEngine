// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGameFeatureStateWidget.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "GameFeatureTypes.h"

#define LOCTEXT_NAMESPACE "GameFeatures"

//////////////////////////////////////////////////////////////////////////
// FGameFeatureDataDetailsCustomization

void SGameFeatureStateWidget::Construct(const FArguments& InArgs)
{
	CurrentState = InArgs._CurrentState;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SSegmentedControl<EGameFeaturePluginState>)
			.Value(CurrentState)
			.OnValueChanged(InArgs._OnStateChanged)
			+SSegmentedControl<EGameFeaturePluginState>::Slot(EGameFeaturePluginState::Installed)
				.Text(GetDisplayNameOfState(EGameFeaturePluginState::Installed))
				.ToolTip(LOCTEXT("SwitchToInstalledTooltip", "Attempt to change the current state of this game feature to Installed.\n\nInstalled means that the plugin is in local storage (i.e., it is on the hard drive) but it has not been registered, loaded, or activated yet."))
			+ SSegmentedControl<EGameFeaturePluginState>::Slot(EGameFeaturePluginState::Registered)
				.Text(GetDisplayNameOfState(EGameFeaturePluginState::Registered))
				.ToolTip(LOCTEXT("SwitchToRegisteredTooltip", "Attempt to change the current state of this game feature to Registered.\n\nRegistered means that the assets in the plugin are known, but have not yet been loaded, except a few for discovery reasons (and it is not actively affecting gameplay yet)."))
			+ SSegmentedControl<EGameFeaturePluginState>::Slot(EGameFeaturePluginState::Loaded)
				.Text(GetDisplayNameOfState(EGameFeaturePluginState::Loaded))
				.ToolTip(LOCTEXT("SwitchToLoadedTooltip", "Attempt to change the current state of this game feature to Loaded.\n\nLoaded means that the plugin is loaded into memory and registered with some game systems, but not yet active and not affecting gameplay."))
			+ SSegmentedControl<EGameFeaturePluginState>::Slot(EGameFeaturePluginState::Active)
				.Text(GetDisplayNameOfState(EGameFeaturePluginState::Active))
				.ToolTip(LOCTEXT("SwitchToActiveTooltip", "Attempt to change the current state of this game feature to Active.\n\nActive means that the plugin is fully loaded and active. It is affecting the game."))
		]
		+SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SGameFeatureStateWidget::GetStateStatusDisplay)
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("ButtonText"))
			.ToolTipText(this, &SGameFeatureStateWidget::GetStateStatusDisplayTooltip)
			.ColorAndOpacity(FAppStyle::Get().GetSlateColor(TEXT("Colors.AccentYellow")))
		]
	];
}

FText SGameFeatureStateWidget::GetDisplayNameOfState(EGameFeaturePluginState State)
{
#define GAME_FEATURE_PLUGIN_STATE_TEXT(inEnum, inText) case EGameFeaturePluginState::inEnum: return inText;
	switch (State)
	{
	GAME_FEATURE_PLUGIN_STATE_LIST(GAME_FEATURE_PLUGIN_STATE_TEXT)
	}
#undef GAME_FEATURE_PLUGIN_STATE_TEXT

	const FString FallbackString = UE::GameFeatures::ToString(State);
	ensureMsgf(false, TEXT("Unknown EGameFeaturePluginState entry %d %s"), (int)State, *FallbackString);
	return FText::AsCultureInvariant(FallbackString);
}

FText SGameFeatureStateWidget::GetTooltipOfState(EGameFeaturePluginState State)
{
	static_assert((int32)EGameFeaturePluginState::MAX == 37, "");

	switch (State)
	{
	case EGameFeaturePluginState::Uninitialized:
		return LOCTEXT("StateTooltip_Uninitialized", "Unset. Not yet been set up.");
	case EGameFeaturePluginState::Terminal:
		return LOCTEXT("StateTooltip_Terminal", "Final State before removal of the state machine");
	case EGameFeaturePluginState::UnknownStatus:
		return LOCTEXT("StateTooltip_UnknownStatus", "Initialized, but the only thing known is the URL to query status.");
	case EGameFeaturePluginState::Uninstalled:
		return LOCTEXT("StateTooltip_Uninstalled", "All installed data for this plugin has now been uninstalled from local storage (i.e the hard drive)");
	case EGameFeaturePluginState::Uninstalling:
		return LOCTEXT("StateTooltip_Uninstalling", "Transition state between StatusKnown -> Terminal for any plugin that can have data that needs to have local datat uninstalled.");
	case EGameFeaturePluginState::ErrorUninstalling:
		return LOCTEXT("StateTooltip_Error Uninstalling", "Error state for Uninstalling -> Terminal transition.");
	case EGameFeaturePluginState::CheckingStatus:
		return LOCTEXT("StateTooltip_CheckingStatus", "Transition state UnknownStatus -> StatusKnown. The status is in the process of being queried.");
	case EGameFeaturePluginState::ErrorCheckingStatus:
		return LOCTEXT("StateTooltip_ErrorCheckingStatus", "Error state for UnknownStatus -> StatusKnown transition.");
	case EGameFeaturePluginState::ErrorUnavailable:
		return LOCTEXT("StateTooltip_ErrorUnavailable", "Error state for UnknownStatus -> StatusKnown transition.");
	case EGameFeaturePluginState::StatusKnown:
		return LOCTEXT("StateTooltip_StatusKnown", "The plugin's information is known, but no action has taken place yet.");
	case EGameFeaturePluginState::Releasing:
		return LOCTEXT("StateTooltip_Releasing", "Transition State for Installed -> StatusKnown. Releases local data from any relevant caches.");
	case EGameFeaturePluginState::ErrorManagingData:
		return LOCTEXT("StateTooltip_ErrorManagingData", "Error state for Installed -> StatusKnown and StatusKnown -> Installed transitions.");
	case EGameFeaturePluginState::Downloading:
		return LOCTEXT("StateTooltip_Downloading", "Transition state StatusKnown -> Installed. In the process of adding to local storage.");
	case EGameFeaturePluginState::Installed:
		return LOCTEXT("StateTooltip_Installed", "The plugin is in local storage (i.e. it is on the hard drive)");
	case EGameFeaturePluginState::ErrorMounting:
		return LOCTEXT("StateTooltip_ErrorMounting", "Error state for Installed -> Registered and Registered -> Installed transitions.");
	case EGameFeaturePluginState::ErrorWaitingForDependencies:
		return LOCTEXT("StateTooltip_ErrorWaitingForDependencies", "Error state for Installed -> Registered and Registered -> Installed transitions.");
	case EGameFeaturePluginState::ErrorRegistering:
		return LOCTEXT("StateTooltip_ErrorRegistering", "Error state for Installed -> Registered and Registered -> Installed transitions.");
	case EGameFeaturePluginState::WaitingForDependencies:
		return LOCTEXT("StateTooltip_WaitingForDependencies", "Transition state Installed -> Registered. In the process of loading code/content for all dependencies into memory.");
	case EGameFeaturePluginState::AssetDependencyStreamOut:
		return LOCTEXT("StateTooltip_AssetDependencyStreamout", "Transition state Registered -> Installed. In the process of streaming out individual assets from dependencies.");
	case EGameFeaturePluginState::ErrorAssetDependencyStreaming:
		return LOCTEXT("StateTooltip_ErrorAssetDependencyStreaming", "Error state for Installed -> Registered and Registered -> Installed transitions.");
	case EGameFeaturePluginState::AssetDependencyStreaming:
		return LOCTEXT("StateTooltip_AssetDependencyStreaming", "Transition state Installed -> Registered. In the process of streaming individual assets from dependencies.");
	case EGameFeaturePluginState::Unmounting:
		return LOCTEXT("StateTooltip_Unmounting", "Transition state Registered -> Installed. The content file(s) (i.e. pak file) for the plugin is unmounting.");
	case EGameFeaturePluginState::Mounting:
		return LOCTEXT("StateTooltip_Mounting", "Transition state Installed -> Registered. The content file(s) (i.e. pak file) for the plugin is getting mounted.");
	case EGameFeaturePluginState::Unregistering:
		return LOCTEXT("StateTooltip_Unregistering", "Transition state Registered -> Installed. Cleaning up data gathered in Registering.");
	case EGameFeaturePluginState::Registering:
		return LOCTEXT("StateTooltip_Registering", "Transition state Installed -> Registered. Discovering assets in the plugin, but not loading them, except a few for discovery reasons.");
	case EGameFeaturePluginState::Registered:
		return LOCTEXT("StateTooltip_Registered", "The assets in the plugin are known, but have not yet been loaded, except a few for discovery reasons.");
	case EGameFeaturePluginState::ErrorLoading:
		return LOCTEXT("StateTooltip_ErrorLoading", "Error state for Loading -> Loaded transition.");
	case EGameFeaturePluginState::Unloading:
		return LOCTEXT("StateTooltip_Unloading", "Transition state Loaded -> Registered. In the process of removing code/content from memory.");
	case EGameFeaturePluginState::Loading:
		return LOCTEXT("StateTooltip_Loading", "Transition state Registered -> Loaded. In the process of loading code/content into memory.");
	case EGameFeaturePluginState::Loaded:
		return LOCTEXT("StateTooltip_Loaded", "The plugin is loaded into memory and registered with some game systems but not yet active.");
	case EGameFeaturePluginState::ErrorActivatingDependencies:
		return LOCTEXT("StateTooltip_ErrorActivateDependencies", "Error state for Loaded -> Active and Active -> Loaded transitions.");
	case EGameFeaturePluginState::ActivatingDependencies:
		return LOCTEXT("StateTooltip_ActivateDependencies", "Transition state Loaded -> Active. In the process of selectively activating dependencies.");
	case EGameFeaturePluginState::ErrorDeactivatingDependencies:
		return LOCTEXT("StateTooltip_ErrorDeactivatingDependencies", "Error state for Active -> Loaded transition.");
	case EGameFeaturePluginState::DeactivatingDependencies:
		return LOCTEXT("StateTooltip_DeactivateDependencies", "Transition state Active -> Loaded. In the process of selectively deactivating dependencies.");
	case EGameFeaturePluginState::Deactivating:
		return LOCTEXT("StateTooltip_Deactivating", "Transition state Active -> Loaded. Currently unregistering with game systems.");
	case EGameFeaturePluginState::Activating:
		return LOCTEXT("StateTooltip_Activating", "Transition state Loaded -> Active. Currently registering plugin code/content with game systems.");
	case EGameFeaturePluginState::Active:
		return LOCTEXT("StateTooltip_Active", "Plugin is fully loaded and active. It is affecting the game.");
	}

	return GetDisplayNameOfState(State);
}

FText SGameFeatureStateWidget::GetStateStatusDisplay() const
{
	// Display the current state/transition for anything but the four acceptable destination states (which are already covered by the switcher)
	const EGameFeaturePluginState State = CurrentState.Get();
	switch (State)
	{
		case EGameFeaturePluginState::Active:
		case EGameFeaturePluginState::Installed:
		case EGameFeaturePluginState::Loaded:
		case EGameFeaturePluginState::Registered:
			return FText::GetEmpty();
		default:
			return GetDisplayNameOfState(State);
	}
}

FText SGameFeatureStateWidget::GetStateStatusDisplayTooltip() const
{
	const EGameFeaturePluginState State = CurrentState.Get();

	return FText::Format(
		LOCTEXT("OtherStateToolTip", "The current state of this game feature plugin\n\n{0}"),
		GetTooltipOfState(State));
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
