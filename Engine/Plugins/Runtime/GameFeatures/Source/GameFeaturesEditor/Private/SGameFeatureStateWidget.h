// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "GameFeatureTypesFwd.h"

//////////////////////////////////////////////////////////////////////////
// FGameFeatureDataDetailsCustomization

/**
 * A delegate that is invoked when widgets want to notify a user that they have been clicked.
 * Intended for use by buttons and other button-like widgets.
 */
DECLARE_DELEGATE_OneParam(FOnWidgetChangesGameFeatureState, EGameFeaturePluginState)

class SGameFeatureStateWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGameFeatureStateWidget) {}
		SLATE_ATTRIBUTE(EGameFeaturePluginState, CurrentState)
		SLATE_EVENT(FOnWidgetChangesGameFeatureState, OnStateChanged)
	SLATE_END_ARGS()

public:
	SGameFeatureStateWidget() {}

	void Construct(const FArguments& InArgs);

	static FText GetDisplayNameOfState(EGameFeaturePluginState State);
	static FText GetTooltipOfState(EGameFeaturePluginState StateID);

private:
	FText GetStateStatusDisplay() const;
	FText GetStateStatusDisplayTooltip() const;

private:
	TAttribute<EGameFeaturePluginState> CurrentState;
};
