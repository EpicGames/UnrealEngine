// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IPluginsEditorFeature.h"
#include "GameFeatureData.h"
#include "PluginDescriptor.h"

#define UE_API GAMEFEATURESEDITOR_API

/**
 * Used to create custom templates for GameFeaturePlugins.
 */
struct FGameFeaturePluginTemplateDescription : public FPluginTemplateDescription
{
	UE_API FGameFeaturePluginTemplateDescription(FText InName, FText InDescription, FString InOnDiskPath, FString InDefaultSubfolder, FString InDefaultPluginName
		, TSubclassOf<UGameFeatureData> GameFeatureDataClassOverride, FString GameFeatureDataNameOverride, EPluginEnabledByDefault InEnabledByDefault
		, FString InPostCreatePythonScriptPath, FString InPostCreatePythonScriptArguments);

	UE_API virtual bool ValidatePathForPlugin(const FString& ProposedAbsolutePluginPath, FText& OutErrorMessage) override;
	UE_API virtual void UpdatePathWhenTemplateSelected(FString& InOutPath) override;
	UE_API virtual void UpdatePathWhenTemplateUnselected(FString& InOutPath) override;

	UE_API virtual void UpdatePluginNameTextWhenTemplateSelected(FText& OutPluginNameText) override;
	UE_API virtual void UpdatePluginNameTextWhenTemplateUnselected(FText& OutPluginNameText) override;

	UE_API virtual void CustomizeDescriptorBeforeCreation(FPluginDescriptor& Descriptor) override;
	UE_API virtual void OnPluginCreated(TSharedPtr<IPlugin> NewPlugin) override;

	UE_API FString GetGameFeatureRoot() const;
	UE_API bool IsRootedInGameFeaturesRoot(const FString& InStr) const;

	FString DefaultSubfolder;
	FString DefaultPluginName;
	TSubclassOf<UGameFeatureData> GameFeatureDataClass;
	FString GameFeatureDataName;
	EPluginEnabledByDefault PluginEnabledByDefault = EPluginEnabledByDefault::Disabled;
};

#undef UE_API
